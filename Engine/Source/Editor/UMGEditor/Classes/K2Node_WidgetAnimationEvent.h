// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BlueprintNodeSignature.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node_Event.h"
#include "Animation/WidgetAnimation.h"
#include "Animation/WidgetAnimationDelegateBinding.h"
#include "WidgetBlueprint.h"
#include "K2Node_WidgetAnimationEvent.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UDynamicBlueprintBinding;
class UEdGraph;

UCLASS(MinimalAPI)
class UK2Node_WidgetAnimationEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

public:

	/** The action to bind to. */
	UPROPERTY()
	EWidgetAnimationEvent Action;

	/** Name of property in Blueprint class that pointer to component we want to bind to */
	UPROPERTY()
	FName AnimationPropertyName;

	/** Binds this to a specific user action. */
	UPROPERTY(EditAnywhere, Category = "Animation")
	FName UserTag;

	UPROPERTY()
	TObjectPtr<const UWidgetBlueprint> SourceWidgetBlueprint;

	virtual void PostDuplicate(bool bDuplicateForPIE) override;

	//~ Begin EdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual bool IsActionFilteredOut(FBlueprintActionFilter const& Filter) override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	//~ End UK2Node Interface

	void MarkDirty();

private:
	void Initialize(const UWidgetBlueprint* InSourceBlueprint, UWidgetAnimation* InAnimation, EWidgetAnimationEvent InAction);

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};
