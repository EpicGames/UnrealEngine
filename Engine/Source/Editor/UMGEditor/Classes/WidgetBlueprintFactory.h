// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "Engine/Blueprint.h"
#include "WidgetBlueprintFactory.generated.h"

UCLASS(HideCategories=Object, MinimalAPI)
class UWidgetBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category=WidgetBlueprintFactory)
	TEnumAsByte<enum EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category=WidgetBlueprintFactory, meta=(AllowAbstract = ""))
	TSubclassOf<class UUserWidget> ParentClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

private:
	UPROPERTY(Transient)
	TObjectPtr<UClass> RootWidgetClass;
};
