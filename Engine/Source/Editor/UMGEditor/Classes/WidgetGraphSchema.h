// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraphSchema_K2.h"
#include "WidgetGraphSchema.generated.h"

class UK2Node;
class UK2Node_CallFunction;

UCLASS(MinimalAPI)
class UWidgetGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_UCLASS_BODY()

public:
	virtual void BackwardCompatibilityNodeConversion(UEdGraph* Graph, bool bOnlySafeChanges) const override;

private:
	void ConvertAnimationEventNodes(UEdGraph* Graph) const;

	void ConvertAddAnimationDelegate(UEdGraph* Graph) const;
	void ConvertRemoveAnimationDelegate(UEdGraph* Graph) const;
	void ConvertClearAnimationDelegate(UEdGraph* Graph) const;

	void ReplaceAnimationFunctionAndAllocateDefaultPins(UEdGraph* Graph, UK2Node* OldNode, UK2Node_CallFunction* NewFunctionNode) const;

	void FixDefaultToSelfForAnimation(UEdGraph* Graph) const;
};
