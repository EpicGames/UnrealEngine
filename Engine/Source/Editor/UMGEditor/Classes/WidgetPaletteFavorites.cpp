// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetPaletteFavorites.h"

#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetPaletteFavorites)

UWidgetPaletteFavorites::UWidgetPaletteFavorites(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UWidgetPaletteFavorites::Add(const FString& InWidgetTemplateName)
{
	Favorites.AddUnique(InWidgetTemplateName);

	SaveConfig();

	OnFavoritesUpdated.Broadcast();
}

void UWidgetPaletteFavorites::Remove(const FString& InWidgetTemplateName)
{
	Favorites.RemoveSingle(InWidgetTemplateName);

	SaveConfig();

	OnFavoritesUpdated.Broadcast();
}
