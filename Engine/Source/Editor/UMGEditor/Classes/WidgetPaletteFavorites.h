// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "WidgetPaletteFavorites.generated.h"

#define UE_API UMGEDITOR_API

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UWidgetPaletteFavorites : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	UE_API void Add(const FString& InWidgetTemplateName);
	UE_API void Remove(const FString& InWidgetTemplateName);

	TArray<FString> GetFavorites() const { return Favorites; }
	
	DECLARE_MULTICAST_DELEGATE(FOnFavoritesUpdated)

	/** Fires after the list of favorites is updated */
	FOnFavoritesUpdated OnFavoritesUpdated;

private:
	UPROPERTY(config)
	TArray<FString> Favorites;
};

#undef UE_API
