// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MarginTrackEditor.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "KeyPropertyParams.h"
#include "Layout/Margin.h"
#include "Misc/Optional.h"
#include "PropertyPath.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

#define LOCTEXT_NAMESPACE "FMarginTrackEditor"

FName FMarginTrackEditor::LeftName( "Left" );
FName FMarginTrackEditor::TopName( "Top" );
FName FMarginTrackEditor::RightName( "Right" );
FName FMarginTrackEditor::BottomName( "Bottom" );

TSharedRef<ISequencerTrackEditor> FMarginTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FMarginTrackEditor( InSequencer ) );
}

FText FMarginTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MarginTrackEditor_DisplayName", "Margin");
}

void FMarginTrackEditor::GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	FPropertyPath StructPath = PropertyChangedParams.StructPathToKey;
	FName ChannelName = StructPath.GetNumProperties() != 0 ? StructPath.GetLeafMostProperty().Property->GetFName() : NAME_None;

	FMargin Margin = PropertyChangedParams.GetPropertyValue<FMargin>();

	const bool bKeyLeft   = ChannelName == NAME_None || ChannelName == LeftName;
	const bool bKeyTop    = ChannelName == NAME_None || ChannelName == TopName;
	const bool bKeyRight  = ChannelName == NAME_None || ChannelName == RightName;
	const bool bKeyBottom = ChannelName == NAME_None || ChannelName == BottomName;

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(0, Margin.Left,   bKeyLeft));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(1, Margin.Top,    bKeyTop));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(2, Margin.Right,  bKeyRight));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(3, Margin.Bottom, bKeyBottom));
}

#undef LOCTEXT_NAMESPACE
