// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Animation/MovieSceneMarginTrack.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FName;
class FPropertyChangedParams;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

class FMarginTrackEditor
	: public FPropertyTrackEditor<UMovieSceneMarginTrack>
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FMarginTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FPropertyTrackEditor( InSequencer, GetAnimatedPropertyTypes() )
	{
	}

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromStructType("Margin") });
	}

	/** Virtual destructor. */

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

protected:

	//~ FPropertyTrackEditor Interface
	virtual FText GetDisplayName() const override;
	virtual void GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;

private:

	static FName LeftName;
	static FName TopName;
	static FName RightName;
	static FName BottomName;
};
