// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequence.h"
#include "MovieSceneSequenceEditor.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Engine/Blueprint.h"
#include "K2Node_FunctionEntry.h"
#include "Modules/ModuleManager.h"

struct FMovieSceneSequenceEditor_WidgetAnimation : FMovieSceneSequenceEditor
{
	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const
	{
		return true;
	}

	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const override
	{
#if WITH_EDITOR
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
		const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
		FClassViewerInitializationOptions ClassViewerOptions = {};

		if (GlobalClassFilter.IsValid())
		{
			return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, UBlueprint::StaticClass(), ClassFilterFuncs);
		}
#endif
		return true;
	}

	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		return InSequence->GetTypedOuter<UBlueprint>();
	}
};