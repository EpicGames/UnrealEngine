// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/Sequencer2DTransformSection.h"

#include "Animation/MovieScene2DTransformSection.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "ScopedTransaction.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

#define LOCTEXT_NAMESPACE "F2DTransformSection"

void F2DTransformSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding)
{
	UMovieScene2DTransformSection* TransformSection = CastChecked<UMovieScene2DTransformSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	auto MakeUIAction = [=](EMovieScene2DTransformChannel ChannelsToToggle)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([=]
				{
					FScopedTransaction Transaction(LOCTEXT("SetActiveChannelsTransaction", "Set Active Channels"));
					TransformSection->Modify();
					EMovieScene2DTransformChannel Channels = TransformSection->GetMask().GetChannels();

					if (EnumHasAllFlags(Channels, ChannelsToToggle) || (Channels & ChannelsToToggle) == EMovieScene2DTransformChannel::None)
					{
						TransformSection->SetMask(TransformSection->GetMask().GetChannels() ^ ChannelsToToggle);
					}
					else
					{
						TransformSection->SetMask(TransformSection->GetMask().GetChannels() | ChannelsToToggle);
					}

					// Restore pre-animated state for the bound objects so that inactive channels will return to their default values.
					for (TWeakObjectPtr<> WeakObject : SequencerPtr->FindBoundObjects(InObjectBinding, SequencerPtr->GetFocusedTemplateID()))
					{
						if (UObject* Object = WeakObject.Get())
						{
							SequencerPtr->RestorePreAnimatedState();
						}
					}

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([=]
			{
				EMovieScene2DTransformChannel Channels = TransformSection->GetMask().GetChannels();
				if (EnumHasAllFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Checked;
				}
				else if (EnumHasAnyFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Undetermined;
				}
				return ECheckBoxState::Unchecked;
			})
		);
	};

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TransformChannelsText", "Active Channels"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AllTranslation", "Translation"), LOCTEXT("AllTranslation_ToolTip", "Causes this section to affect the translation of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("TranslationX", "X"), LOCTEXT("TranslationX_ToolTip", "Causes this section to affect the X channel of the transform's translation"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::TranslationX), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("TranslationY", "Y"), LOCTEXT("TranslationY_ToolTip", "Causes this section to affect the Y channel of the transform's translation"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::TranslationY), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieScene2DTransformChannel::Translation),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllRotation", "Rotation"), LOCTEXT("AllRotation_ToolTip", "Causes this section to affect the rotation of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("Angle", "Angle"), LOCTEXT("Angle_ToolTip", "Causes this section to affect the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::Rotation), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieScene2DTransformChannel::Rotation),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllScale", "Scale"), LOCTEXT("AllScale_ToolTip", "Causes this section to affect the scale of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("ScaleX", "X"), LOCTEXT("ScaleX_ToolTip", "Causes this section to affect the X channel of the transform's scale"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::ScaleX), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("ScaleY", "Y"), LOCTEXT("ScaleY_ToolTip", "Causes this section to affect the Y channel of the transform's scale"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::ScaleY), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieScene2DTransformChannel::Scale),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllShear", "Shear"), LOCTEXT("AllShear_ToolTip", "Causes this section to affect the shear of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("ShearX", "X"), LOCTEXT("ShearX_ToolTip", "Causes this section to affect the X channel of the transform's shear"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::ShearX), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("ShearY", "Y"), LOCTEXT("ShearY_ToolTip", "Causes this section to affect the Y channel of the transform's shear"),
					FSlateIcon(), MakeUIAction(EMovieScene2DTransformChannel::ShearY), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieScene2DTransformChannel::Shear),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

bool F2DTransformSection::RequestDeleteCategory(const TArray<FName>& CategoryNamePaths)
{
	UMovieScene2DTransformSection* TransformSection = CastChecked<UMovieScene2DTransformSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
		
	const FScopedTransaction Transaction( LOCTEXT( "DeleteTransformCategory", "Delete transform category" ) );

	if (TransformSection->TryModify())
	{
		FName CategoryName = CategoryNamePaths[CategoryNamePaths.Num()-1];
		
		EMovieScene2DTransformChannel Channel = TransformSection->GetMask().GetChannels();
		EMovieScene2DTransformChannel ChannelToRemove = TransformSection->GetMaskByName(CategoryName).GetChannels();

		Channel &= ~ChannelToRemove;

		TransformSection->SetMask(Channel);
			
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return false;
}

bool F2DTransformSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePaths)
{
	UMovieScene2DTransformSection* TransformSection = CastChecked<UMovieScene2DTransformSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	const FScopedTransaction Transaction( LOCTEXT( "DeleteTransformChannel", "Delete transform channel" ) );

	if (TransformSection->TryModify())
	{
		// Only delete the last key area path which is the channel. ie. TranslationX as opposed to Translation
		FName KeyAreaName = KeyAreaNamePaths[KeyAreaNamePaths.Num()-1];
			
		EMovieScene2DTransformChannel Channel = TransformSection->GetMask().GetChannels();
		EMovieScene2DTransformChannel ChannelToRemove = TransformSection->GetMaskByName(KeyAreaName).GetChannels();

		Channel &= ~ChannelToRemove;

		TransformSection->SetMask(Channel);
					
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE