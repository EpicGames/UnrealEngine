// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/Sequencer2DTransformTrackEditor.h"

#include "Animation/MovieScene2DTransformSection.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Animation/Sequencer2DTransformSection.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "KeyPropertyParams.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "PropertyPath.h"
#include "SequencerKeyParams.h"
#include "Slate/WidgetTransform.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ISequencerTrackEditor;
struct FMovieSceneChannel;

#define LOCTEXT_NAMESPACE "2DTransformTrackEditor"

FName F2DTransformTrackEditor::TranslationName( "Translation" );
FName F2DTransformTrackEditor::ScaleName( "Scale" );
FName F2DTransformTrackEditor::ShearName( "Shear" );
FName F2DTransformTrackEditor::AngleName( "Angle" );
FName F2DTransformTrackEditor::ChannelXName( "X" );
FName F2DTransformTrackEditor::ChannelYName( "Y" );

TSharedRef<ISequencerTrackEditor> F2DTransformTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new F2DTransformTrackEditor( InSequencer ) );
}

FText F2DTransformTrackEditor::GetDisplayName() const
{
	return LOCTEXT("2DTransformTrackEditor_DisplayName", "2D Transform");
}

TSharedRef<ISequencerSection> F2DTransformTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShared<F2DTransformSection>(SectionObject, GetSequencer());
}

void F2DTransformTrackEditor::GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys )
{
	FPropertyPath StructPath = PropertyChangedParams.StructPathToKey;

	bool bKeyTranslationX = false;
	bool bKeyTranslationY = false;
	bool bKeyAngle = false;
	bool bKeyScaleX = false;
	bool bKeyScaleY = false;
	bool bKeyShearX = false;
	bool bKeyShearY = false;

	if (StructPath.GetNumProperties() == 0)
	{
		bKeyTranslationX = bKeyTranslationY = bKeyAngle = bKeyScaleX = bKeyScaleY = bKeyShearX = bKeyShearY = true;
	}
	else
	{
		if (StructPath.GetRootProperty().Property->GetFName() == TranslationName)
		{
			if (StructPath.GetLeafMostProperty() != StructPath.GetRootProperty())
			{
				bKeyTranslationX = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelXName;
				bKeyTranslationY = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelYName;
			}
			else
			{
				bKeyTranslationX = bKeyTranslationY = true;
			}
		}

		if (StructPath.GetRootProperty().Property->GetFName() == AngleName)
		{
			bKeyAngle = true;
		}

		if (StructPath.GetRootProperty().Property->GetFName() == ScaleName)
		{
			if (StructPath.GetLeafMostProperty() != StructPath.GetRootProperty())
			{
				bKeyScaleX = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelXName;
				bKeyScaleY = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelYName;
			}
			else
			{
				bKeyScaleX = bKeyScaleY = true;
			}
		}

		if (StructPath.GetRootProperty().Property->GetFName() == ShearName)
		{
			if (StructPath.GetLeafMostProperty() != StructPath.GetRootProperty())
			{
				bKeyShearX = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelXName;
				bKeyShearY = StructPath.GetLeafMostProperty().Property->GetFName() == ChannelYName;
			}
			else
			{
				bKeyShearX = bKeyShearY = true;
			}
		}
	}
	
	FWidgetTransform CurrentTransform    = PropertyChangedParams.GetPropertyValue<FWidgetTransform>();
	FWidgetTransform RecomposedTransform = RecomposeTransform(CurrentTransform, PropertyChangedParams.ObjectsThatChanged[0], SectionToKey);

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(0, RecomposedTransform.Translation.X, bKeyTranslationX));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(1, RecomposedTransform.Translation.Y, bKeyTranslationY));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(2, RecomposedTransform.Angle,         bKeyAngle));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(3, RecomposedTransform.Scale.X,       bKeyScaleX));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(4, RecomposedTransform.Scale.Y,       bKeyScaleY));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(5, RecomposedTransform.Shear.X,       bKeyShearX));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(6, RecomposedTransform.Shear.Y,       bKeyShearY));
}

FWidgetTransform F2DTransformTrackEditor::RecomposeTransform(const FWidgetTransform& InTransform, UObject* AnimatedObject, UMovieSceneSection* Section)
{
	using namespace UE::MovieScene;

	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();

	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	FMovieSceneEntityID EntityID = EvaluationTemplate.FindEntityFromOwner(Section, 0, GetSequencer()->GetFocusedTemplateID());

	if (EntityLinker && EntityID)
	{
		UMovieScenePropertyInstantiatorSystem* System = EntityLinker->FindSystem<UMovieScenePropertyInstantiatorSystem>();
		if (System)
		{
			FDecompositionQuery Query;
			Query.Entities = MakeArrayView(&EntityID, 1);
			Query.Object   = AnimatedObject;

			return System->RecomposeBlend(FMovieSceneUMGComponentTypes::Get()->WidgetTransform, Query, InTransform).Values[0];
		}
	}

	return InTransform;
}

void F2DTransformTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult> *OutResults)
{
	using namespace UE::Sequencer;

	auto Iterator = [this, InKeyTime, &Operation, &InSequencer, OutResults](UMovieSceneTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> Operations)
	{
		FGuid ObjectBinding = Track->FindObjectBindingGuid();
		if (ObjectBinding.IsValid())
		{
			for (TWeakObjectPtr<> WeakObject : InSequencer.FindBoundObjects(ObjectBinding, InSequencer.GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					this->ProcessKeyOperation(Object, Operations, InSequencer, InKeyTime, OutResults);
					return;
				}
			}
		}

		// Default behavior
		FKeyOperation::ApplyOperations(InKeyTime, Operations, ObjectBinding, InSequencer, OutResults);
	};

	Operation.IterateOperations(Iterator);
}

void F2DTransformTrackEditor::ProcessKeyOperation(UObject* ObjectToKey, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult> *OutResults)
{
	using namespace UE::Sequencer;

	for (const FKeySectionOperation& Operation : SectionsToKey)
	{
		UMovieScene2DTransformSection* SectionObject = Cast<UMovieScene2DTransformSection>(Operation.Section->GetSectionObject());
		if (!SectionObject)
		{
			continue;
		}

		FWidgetTransform CurrentTransform    = SectionObject->GetCurrentValue(ObjectToKey);
		FWidgetTransform RecomposedTransform = RecomposeTransform(CurrentTransform, ObjectToKey, SectionObject);

		for (TSharedPtr<IKeyArea> KeyArea : Operation.KeyAreas)
		{
			FMovieSceneChannelHandle Handle  = KeyArea->GetChannel();
			FMovieSceneChannel*      Channel = Handle.Get();

			if (Channel)
			{
				float Value = 0.f;
				switch ( Handle.GetChannelIndex() )
				{
				case 0: Value = RecomposedTransform.Translation.X;               break;
				case 1: Value = RecomposedTransform.Translation.Y;               break;
				case 2: Value = RecomposedTransform.Angle;                       break;
				case 3: Value = RecomposedTransform.Scale.X;                     break;
				case 4: Value = RecomposedTransform.Scale.Y;                     break;
				case 5: Value = RecomposedTransform.Shear.X;                     break;
				case 6: Value = RecomposedTransform.Shear.Y;                     break;
				default: KeyArea->AddOrUpdateKey(KeyTime, FGuid(), InSequencer); continue;
				}

				if (Handle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
				{
					FKeyHandle KeyHandle = AddKeyToChannel(static_cast<FMovieSceneFloatChannel*>(Channel), KeyTime, Value, InSequencer.GetKeyInterpolation());

					if (OutResults)
					{
						FAddKeyResult Result;
						Result.KeyArea = KeyArea;
						Result.KeyHandle = KeyHandle;
						OutResults->Add(Result);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
