// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Animation/MovieScene2DTransformTrack.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Slate/WidgetTransform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FName;
class FPropertyChangedParams;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UMovieSceneTrack;
class UObject;
namespace UE::Sequencer { struct FKeyOperation; }
namespace UE::Sequencer { struct FKeySectionOperation; }
namespace UE::Sequencer { struct FAddKeyResult; }
struct FFrameNumber;
struct FGuid;

class F2DTransformTrackEditor
	: public FPropertyTrackEditor<UMovieScene2DTransformTrack>
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	F2DTransformTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FPropertyTrackEditor( InSequencer, GetAnimatedPropertyTypes() )
	{
	}

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromStructType("WidgetTransform") });
	}

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

protected:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) override;
	// FPropertyTrackEditor interface

	virtual void GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys ) override;

	FWidgetTransform RecomposeTransform(const FWidgetTransform& InTransform, UObject* AnimatedObject, UMovieSceneSection* Section);

	void ProcessKeyOperation(UObject* ObjectToKey, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr);

private:
	static FName TranslationName;
	static FName ScaleName;
	static FName ShearName;
	static FName AngleName;
	static FName ChannelXName;
	static FName ChannelYName;
};
