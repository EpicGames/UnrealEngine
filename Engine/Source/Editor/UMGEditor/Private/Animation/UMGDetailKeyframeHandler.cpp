// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/UMGDetailKeyframeHandler.h"
#include "Animation/WidgetAnimation.h"

#include "PropertyHandle.h"
#include "MovieScene.h"

FUMGDetailKeyframeHandler::FUMGDetailKeyframeHandler(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: BlueprintEditor( InBlueprintEditor )
{}

bool FUMGDetailKeyframeHandler::IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	return BlueprintEditor.Pin()->GetSequencer()->CanKeyProperty(FCanKeyPropertyParams(InObjectClass, InPropertyHandle));
}

bool FUMGDetailKeyframeHandler::IsPropertyKeyingEnabled() const
{

	UMovieSceneSequence* Sequence = BlueprintEditor.Pin()->GetSequencer()->GetRootMovieSceneSequence();
	return Sequence != nullptr && Sequence != UWidgetAnimation::GetNullAnimation();
}

bool FUMGDetailKeyframeHandler::IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject *ParentObject) const
{
	TSharedPtr<ISequencer> Sequencer = BlueprintEditor.Pin()->GetSequencer();
	if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence())
	{
		constexpr bool bCreateHandleIfMissing = false;
		FGuid ObjectHandle = Sequencer->GetHandleToObject(ParentObject, bCreateHandleIfMissing);
		if (ObjectHandle.IsValid()) 
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			FProperty* Property = PropertyHandle.GetProperty();
			TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
			PropertyPath->AddProperty(FPropertyInfo(Property));
			FName PropertyName(*PropertyPath->ToString(TEXT(".")));
			TSubclassOf<UMovieSceneTrack> TrackClass; //use empty @todo find way to get the UMovieSceneTrack from the Property type.
			return MovieScene->FindTrack(TrackClass, ObjectHandle, PropertyName) != nullptr;
		}
	}
	return false;
}

EPropertyKeyedStatus FUMGDetailKeyframeHandler::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	TSharedPtr<ISequencer> Sequencer = BlueprintEditor.Pin()->GetSequencer();
	if (Sequencer.IsValid())
	{
		return Sequencer->GetPropertyKeyedStatus(PropertyHandle);
	}
	return EPropertyKeyedStatus::NotKeyed;
}

void FUMGDetailKeyframeHandler::OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle)
{
	TArray<UObject*> Objects;
	KeyedPropertyHandle.GetOuterObjects( Objects );

	FKeyPropertyParams KeyPropertyParams(Objects, KeyedPropertyHandle, ESequencerKeyMode::ManualKeyForced);

	BlueprintEditor.Pin()->GetSequencer()->KeyProperty(KeyPropertyParams);
}
