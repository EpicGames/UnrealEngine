// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetBlueprintEditor.h"
#include "IDetailKeyframeHandler.h"

class IPropertyHandle;

class FUMGDetailKeyframeHandler : public IDetailKeyframeHandler
{
public:
	FUMGDetailKeyframeHandler( TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor );

	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const class IPropertyHandle& PropertyHandle) const override;

	virtual bool IsPropertyKeyingEnabled() const override;

	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle) override;

	virtual bool IsPropertyAnimated(const class IPropertyHandle& PropertyHandle, UObject *ParentObject) const override;

	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override;

private:
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
};
