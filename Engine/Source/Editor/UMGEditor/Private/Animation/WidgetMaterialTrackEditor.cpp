// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetMaterialTrackEditor.h"

#include "Animation/MovieSceneWidgetMaterialTrack.h"
#include "Animation/WidgetMaterialTrackUtilities.h"
#include "Components/Widget.h"
#include "Containers/ArrayView.h"
#include "ISequencer.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ISequencerTrackEditor;
class UMovieSceneTrack;


FWidgetMaterialTrackEditor::FWidgetMaterialTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMaterialTrackEditor( InSequencer )
{
}


TSharedRef<ISequencerTrackEditor> FWidgetMaterialTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable( new FWidgetMaterialTrackEditor( OwningSequencer ) );
}


bool FWidgetMaterialTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneWidgetMaterialTrack::StaticClass();
}


UMaterialInterface* FWidgetMaterialTrackEditor::GetMaterialInterfaceForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack )
{
	for (TWeakObjectPtr<> WeakObjectPtr : GetSequencer()->FindObjectsInCurrentSequence(ObjectBinding))
	{
		UWidget* Widget = Cast<UWidget>( WeakObjectPtr.Get() );
		UMovieSceneWidgetMaterialTrack* WidgetMaterialTrack = Cast<UMovieSceneWidgetMaterialTrack>( MaterialTrack );
		if ( Widget != nullptr && WidgetMaterialTrack != nullptr )
		{
			FWidgetMaterialHandle Handle = WidgetMaterialTrackUtilities::GetMaterialHandle( Widget, WidgetMaterialTrack->GetBrushPropertyNamePath() );
			if (Handle.IsValid() )
			{
				return Handle.GetMaterial();
			}
		}
	}
	return nullptr;
}
