// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "TrackEditors/MaterialTrackEditor.h"

class ISequencer;
class ISequencerTrackEditor;
class UMaterialInterface;
class UMovieSceneMaterialTrack;
class UMovieSceneTrack;
struct FGuid;

/**
 * A specialized material track editor for widget materials
 */
class FWidgetMaterialTrackEditor
	: public FMaterialTrackEditor
{
public:

	FWidgetMaterialTrackEditor( TSharedRef<ISequencer> InSequencer );

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface

	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;

protected:

	// FMaterialtrackEditor interface

	virtual UMaterialInterface* GetMaterialInterfaceForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack ) override;
};
