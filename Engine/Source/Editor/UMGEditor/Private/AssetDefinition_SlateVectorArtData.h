// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "Slate/SlateVectorArtData.h"
#include "AssetDefinition_SlateVectorArtData.generated.h"

UCLASS()
class UAssetDefinition_SlateVectorArtData : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SlateVectorArtData", "Slate Vector Art Data"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(105, 165, 60); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USlateVectorArtData::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, NSLOCTEXT("AssetDefinition", "AssetDefinition_SlateVectorArtDataSubMenu", "Slate"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
