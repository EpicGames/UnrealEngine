// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_WidgetBlueprint.h"

#include "WidgetBlueprintEditor.h"
#include "Misc/MessageDialog.h"
#include "SBlueprintDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_WidgetBlueprint)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_WidgetBlueprint::UAssetDefinition_WidgetBlueprint() = default;

UAssetDefinition_WidgetBlueprint::~UAssetDefinition_WidgetBlueprint() = default;

FText UAssetDefinition_WidgetBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_WidgetBlueprint", "Widget Blueprint");
}

FLinearColor UAssetDefinition_WidgetBlueprint::GetAssetColor() const
{
	return FLinearColor(FColor(44, 89, 180));
}

TSoftClassPtr<> UAssetDefinition_WidgetBlueprint::GetAssetClass() const
{
	return UWidgetBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_WidgetBlueprint::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, LOCTEXT("WidgetBlueprint_SubMenu", "Widget"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_WidgetBlueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.GetToolkitMode();

	EAssetCommandResult Result = EAssetCommandResult::Unhandled;

	for (UBlueprint* Blueprint : OpenArgs.LoadObjects<UBlueprint>())
	{
		if (Blueprint && Blueprint->SkeletonGeneratedClass && Blueprint->GeneratedClass)
		{
			TSharedRef<FWidgetBlueprintEditor> NewBlueprintEditor(new FWidgetBlueprintEditor);

			const bool bShouldOpenInDefaultsMode = false;
			TArray<UBlueprint*> Blueprints;
			Blueprints.Add(Blueprint);

			NewBlueprintEditor->InitWidgetBlueprintEditor(Mode, OpenArgs.ToolkitHost, Blueprints, bShouldOpenInDefaultsMode);
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("FailedToLoadWidgetBlueprint", "Widget Blueprint could not be loaded because it derives from an invalid class.\nCheck to make sure the parent class for this blueprint hasn't been removed!"));
		}

		Result = EAssetCommandResult::Handled;
	}

	return Result;
}

EAssetCommandResult UAssetDefinition_WidgetBlueprint::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UBlueprint* OldBlueprint = Cast<UBlueprint>(DiffArgs.OldAsset);
	const UBlueprint* NewBlueprint = Cast<UBlueprint>(DiffArgs.NewAsset);
	UClass* AssetClass = GetAssetClass().Get();
	SBlueprintDiff::CreateDiffWindow(OldBlueprint, NewBlueprint, DiffArgs.OldRevision, DiffArgs.NewRevision, AssetClass);
	return EAssetCommandResult::Handled;
}

FText UAssetDefinition_WidgetBlueprint::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>( GET_MEMBER_NAME_CHECKED( UBlueprint, BlueprintDescription ) );
	if ( !Description.IsEmpty() )
	{
		Description.ReplaceInline( TEXT( "\\n" ), TEXT( "\n" ) );
		return FText::FromString( MoveTemp(Description) );
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
