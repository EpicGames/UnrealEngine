// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetBlueprint.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_WidgetBlueprint.generated.h"

UCLASS()
class UAssetDefinition_WidgetBlueprint : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_WidgetBlueprint();
	virtual ~UAssetDefinition_WidgetBlueprint() override;
	
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	// UAssetDefinition End
};
