// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_WidgetBlueprintGeneratedClass.h"

#include "Algo/RemoveIf.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorUtilityLibrary.h"
#include "EditorUtilitySubsystem.h"
#include "EditorUtilityWidget.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_WidgetBlueprintGeneratedClass)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_WidgetBlueprintGeneratedClass::UAssetDefinition_WidgetBlueprintGeneratedClass() = default;

UAssetDefinition_WidgetBlueprintGeneratedClass::~UAssetDefinition_WidgetBlueprintGeneratedClass() = default;

FText UAssetDefinition_WidgetBlueprintGeneratedClass::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_WidgetBlueprintGeneratedClass", "Compiled Widget Blueprint");
}

FLinearColor UAssetDefinition_WidgetBlueprintGeneratedClass::GetAssetColor() const
{
	return FColor(121,149,207);
}

TSoftClassPtr<> UAssetDefinition_WidgetBlueprintGeneratedClass::GetAssetClass() const
{
	return UWidgetBlueprintGeneratedClass::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_WidgetBlueprintGeneratedClass::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::UI };
	return Categories;
}

namespace MenuExtension_WidgetBlueprintGeneratedClass
{
	UClass* GetParentClass(const FAssetData& AssetData)
	{
		UClass* ParentClass = nullptr;
		FString ParentClassName;
		if (!AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
		{
			AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
		}

		if (!ParentClassName.IsEmpty())
		{
			ParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(ParentClassName));
		}

		return ParentClass;
	}

	void ExecuteEditorUtilityEdit(const FToolMenuContext& InContext, TArray<FAssetData> AssetData)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			if(UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>())
			{
				TArray<UWidgetBlueprintGeneratedClass*> WidgetBlueprints = Context->LoadSelectedObjects<UWidgetBlueprintGeneratedClass>();

				for (UWidgetBlueprintGeneratedClass* WidgetBlueprint : WidgetBlueprints)
				{
					EditorUtilitySubsystem->SpawnAndRegisterTabGeneratedClass(WidgetBlueprint);
				}
			}
		}
	}

	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UWidgetBlueprintGeneratedClass::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					TArray<FAssetData> SelectedAssets(Context->SelectedAssets);

					const int32 EndOfRange = Algo::StableRemoveIf(SelectedAssets, [](const FAssetData& AssetData)
					{
						// Make sure this BP Generated Class is an actual EUW, and not just a UserWidget.
						return !(GetParentClass(AssetData) == UEditorUtilityWidget::StaticClass());
					});

					SelectedAssets.SetNum(EndOfRange);

					if (!SelectedAssets.IsEmpty())
					{
						const TAttribute<FText> Label = LOCTEXT("EditorUtilityWidget_Edit", "Run Editor Utility Widget");
						const TAttribute<FText> ToolTip = LOCTEXT("EditorUtilityWidget_EditTooltip", "Opens the tab built by this Editor Utility Widget Blueprint.");
						const FSlateIcon Icon = FSlateIcon();

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEditorUtilityEdit, MoveTemp(SelectedAssets));
						InSection.AddMenuEntry("EditorUtility_Run", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
