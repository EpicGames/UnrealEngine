// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetBlueprint.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_WidgetBlueprintGeneratedClass.generated.h"

UCLASS()
class UAssetDefinition_WidgetBlueprintGeneratedClass : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_WidgetBlueprintGeneratedClass();
	virtual ~UAssetDefinition_WidgetBlueprintGeneratedClass() override;
	
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	// UAssetDefinition End
};