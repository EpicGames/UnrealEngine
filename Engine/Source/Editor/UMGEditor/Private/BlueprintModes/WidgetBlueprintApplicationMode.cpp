// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/WidgetBlueprintApplicationMode.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"

/////////////////////////////////////////////////////
// FWidgetBlueprintApplicationMode

FWidgetBlueprintApplicationMode::FWidgetBlueprintApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor, FName InModeName)
	: FBlueprintEditorApplicationMode(InWidgetEditor, InModeName, FWidgetBlueprintApplicationModes::GetLocalizedMode, false, false)
	, MyWidgetBlueprintEditor(InWidgetEditor)
{
}

void FWidgetBlueprintApplicationMode::PreDeactivateMode()
{
	FBlueprintEditorApplicationMode::PreDeactivateMode();
	OnPreDeactivateMode.Broadcast(*this);
}

void FWidgetBlueprintApplicationMode::PostActivateMode()
{
	OnPostActivateMode.Broadcast(*this);
	FBlueprintEditorApplicationMode::PostActivateMode();
}

UWidgetBlueprint* FWidgetBlueprintApplicationMode::GetBlueprint() const
{
	if ( FWidgetBlueprintEditor* Editor = MyWidgetBlueprintEditor.Pin().Get() )
	{
		return Editor->GetWidgetBlueprintObj();
	}
	else
	{
		return NULL;
	}
}

TSharedPtr<FWidgetBlueprintEditor> FWidgetBlueprintApplicationMode::GetBlueprintEditor() const
{
	return MyWidgetBlueprintEditor.Pin();
}
