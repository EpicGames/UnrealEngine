// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/WidgetBlueprintApplicationModes.h"

#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"

// Mode constants
const FName FWidgetBlueprintApplicationModes::DesignerMode("DesignerName");
const FName FWidgetBlueprintApplicationModes::GraphMode("GraphName");
const FName FWidgetBlueprintApplicationModes::DebugMode("PreviewName");
const FName FWidgetBlueprintApplicationModes::PreviewMode("PreviewName");

FText FWidgetBlueprintApplicationModes::GetLocalizedMode(const FName InMode)
{
	if (InMode == FWidgetBlueprintApplicationModes::DesignerMode)
	{
		return NSLOCTEXT("WidgetBlueprintModes", "DesignerMode", "Designer");
	}
	else if (InMode == FWidgetBlueprintApplicationModes::GraphMode)
	{
		return NSLOCTEXT("WidgetBlueprintModes", "GraphMode", "Graph");
	}
	else if (InMode == FWidgetBlueprintApplicationModes::PreviewMode)
	{
		return NSLOCTEXT("WidgetBlueprintModes", "PreviewMode", "Preview");
	}
	return FText::GetEmpty();
}

static bool bEnablePreviewMode = false;
static FAutoConsoleVariableRef CVarEnablePreviewMode(
	TEXT("UMG.EnablePreviewMode"), 
	bEnablePreviewMode, 
	TEXT("Whether or not to enable the UMG Preview mode.")
);

bool FWidgetBlueprintApplicationModes::IsPreviewModeEnabled()
{
	return bEnablePreviewMode;
}
