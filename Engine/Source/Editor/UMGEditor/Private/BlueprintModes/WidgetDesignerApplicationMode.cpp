// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/WidgetDesignerApplicationMode.h"

#include "BlueprintEditorTabs.h"
#include "SBlueprintEditorToolbar.h"
#include "BlueprintEditorSharedTabFactories.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "WidgetBlueprintEditorToolbar.h"
#include "UMGEditorModule.h"
#include "WidgetEditingProjectSettings.h"
#include "StatusBarSubsystem.h"
#include "ToolMenus.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/Docking/SDockTab.h"

#include "TabFactory/PaletteTabSummoner.h"
#include "TabFactory/LibraryTabSummoner.h"
#include "TabFactory/HierarchyTabSummoner.h"
#include "TabFactory/BindWidgetTabSummoner.h"
#include "TabFactory/DesignerTabSummoner.h"
#include "TabFactory/DetailsTabSummoner.h"
#include "TabFactory/AnimationTabSummoner.h"
#include "TabFactory/NavigationTabSummoner.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "WidgetBlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "WidgetDesignerMode"

/////////////////////////////////////////////////////
// FWidgetDesignerApplicationMode

FWidgetDesignerApplicationMode::FWidgetDesignerApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor)
	: FWidgetBlueprintApplicationMode(InWidgetEditor, FWidgetBlueprintApplicationModes::DesignerMode)
{
	// Override the default created category here since "Designer Editor" sounds awkward
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_WidgetDesigner", "Widget Designer"));

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);

	const double DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left), static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Top));

	const float CenterScale = 0.4f;
	const FVector2D DisplaySize(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);
	const FVector2D WindowSize = (CenterScale * DisplaySize) / DPIScale;

	TSharedPtr<FTabManager::FStack> PaletteTab;
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnablePaletteWindow || FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableLibraryWindow)
	{
		PaletteTab = FTabManager::NewStack()
			->SetSizeCoefficient(0.5f);

		if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnablePaletteWindow)
		{
			PaletteTab->SetForegroundTab(FPaletteTabSummoner::TabID)
				->AddTab(FPaletteTabSummoner::TabID, ETabState::OpenedTab);
		}
		if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableLibraryWindow)
		{
			PaletteTab->AddTab(FLibraryTabSummoner::TabID, ETabState::OpenedTab);
		}
	}

	TSharedPtr<FTabManager::FStack> EnableHierarchyTab;
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableHierarchyWindow || FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableBindWidgetWindow)
	{
		EnableHierarchyTab = FTabManager::NewStack()
			->SetSizeCoefficient(0.5f);

		if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableHierarchyWindow)
		{
			EnableHierarchyTab->SetForegroundTab(FHierarchyTabSummoner::TabID)
				->AddTab(FHierarchyTabSummoner::TabID, ETabState::OpenedTab);
		}

		if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableBindWidgetWindow)
		{
			EnableHierarchyTab->AddTab(FBindWidgetTabSummoner::TabID, ETabState::OpenedTab);
		}
	}

	TSharedRef<FTabManager::FArea> MainArea = FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Horizontal);
	if (PaletteTab || EnableHierarchyTab)
	{
		TSharedRef<FTabManager::FSplitter> LeftSplit = FTabManager::NewSplitter()
			->SetSizeCoefficient(0.15f)
			->SetOrientation(Orient_Vertical);
		if (PaletteTab)
		{
			LeftSplit->Split(PaletteTab.ToSharedRef());
		}
		if (EnableHierarchyTab)
		{
			LeftSplit->Split(EnableHierarchyTab.ToSharedRef());
		}
		MainArea->Split(LeftSplit);
	}

	MainArea->Split
	(
		FTabManager::NewSplitter()
		->SetSizeCoefficient(0.85f)
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->SetSizeCoefficient(0.7f)
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.05f)
				->AddTab(FDesignerTabSummoner::ToolPaletteTabID, ETabState::ClosedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.80f)
				->AddTab(FDesignerTabSummoner::TabID, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.35f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FDetailsTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.3f)
			->SetForegroundTab(FAnimationTabSummoner::TabID)
			->AddTab(FAnimationTabSummoner::TabID, ETabState::ClosedTab)
			->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
		)
	);

	TabLayout = FTabManager::NewLayout("WidgetBlueprintEditor_Designer_Layout_v4_8")
	->AddArea
	(
		MainArea
	)
	->AddArea
	(
		// Sequencer popup
		FTabManager::NewArea(WindowSize)
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(1.0f)
			->AddTab("SequencerGraphEditor", ETabState::ClosedTab)
		)
	);

	// Add Tab Spawners
	//TabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InWidgetEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FDetailsTabSummoner(InWidgetEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FDesignerTabSummoner(InWidgetEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FCompilerResultsSummoner(InWidgetEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FAnimationTabSummoner(InWidgetEditor)));
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableHierarchyWindow)
	{
		TabFactories.RegisterFactory(MakeShareable(new FHierarchyTabSummoner(InWidgetEditor)));
	}
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableBindWidgetWindow)
	{
		TabFactories.RegisterFactory(MakeShareable(new FBindWidgetTabSummoner(InWidgetEditor)));
	}
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnablePaletteWindow)
	{
		TabFactories.RegisterFactory(MakeShareable(new FPaletteTabSummoner(InWidgetEditor)));
	}
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableLibraryWindow)
	{
		TabFactories.RegisterFactory(MakeShareable(new FLibraryTabSummoner(InWidgetEditor)));
	}
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(InWidgetEditor)->bEnableNavigationSimulationWindow)
	{
		TabFactories.RegisterFactory(MakeShareable(new FNavigationTabSummoner(InWidgetEditor)));
	}

	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterTabsForEditor().Broadcast(*this, TabFactories);

	// Add any extenders specified by the UMG Editor Module
	// Note: Used by WidgetEditorModeUILayer to register the toolbox tab
	if (LayoutExtender)
	{
		UMGEditorModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
		TabLayout->ProcessExtensions(*LayoutExtender);
	}

	//Make sure we start with our existing list of extenders instead of creating a new one
	ToolbarExtender = UMGEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders();
	
	InWidgetEditor->GetWidgetToolbarBuilder()->AddWidgetBlueprintEditorModesToolbar(ToolbarExtender);
	InWidgetEditor->RegisterModeToolbarIfUnregistered(GetModeName());

	FName OutParentToolbarName;
	FName ToolBarname = InWidgetEditor->GetToolMenuToolbarNameForMode(GetModeName(), OutParentToolbarName);
	if (UToolMenu* Toolbar = UToolMenus::Get()->FindMenu(ToolBarname))
	{
		InWidgetEditor->GetWidgetToolbarBuilder()->AddWidgetReflector(Toolbar);
		InWidgetEditor->GetWidgetToolbarBuilder()->AddToolPalettes(Toolbar);
		InWidgetEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InWidgetEditor->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}
}

void FWidgetDesignerApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FWidgetBlueprintEditor> BP = GetBlueprintEditor();

	BP->RegisterToolbarTab(InTabManager.ToSharedRef());
	BP->PushTabFactories(TabFactories);
}

void FWidgetDesignerApplicationMode::PreDeactivateMode()
{
	//FWidgetBlueprintApplicationMode::PreDeactivateMode();

	TSharedPtr<FWidgetBlueprintEditor> BP = GetBlueprintEditor();
	if (TSharedPtr<FTabManager> TabManager = BP->GetTabManager())
	{
		if (TSharedPtr<SDockTab> PaletteTab = TabManager->FindExistingLiveTab(FDesignerTabSummoner::ToolPaletteTabID))
		{
			PaletteTab->RequestCloseTab();
		}
	}

	OnPreDeactivateMode.Broadcast(*this);
}

void FWidgetDesignerApplicationMode::PostActivateMode()
{
	// FWidgetBlueprintApplicationMode::PostActivateMode();
	OnPostActivateMode.Broadcast(*this);

	TSharedPtr<FWidgetBlueprintEditor> BP = GetBlueprintEditor();

	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(BP)->bEnableWidgetAnimationEditor)
	{
		FWidgetDrawerConfig WidgetAnimSequencerDrawer(FAnimationTabSummoner::WidgetAnimSequencerDrawerID);
		WidgetAnimSequencerDrawer.GetDrawerContentDelegate.BindSP(BP.Get(), &FWidgetBlueprintEditor::OnGetWidgetAnimSequencer);
		WidgetAnimSequencerDrawer.OnDrawerOpenedDelegate.BindSP(BP.Get(), &FWidgetBlueprintEditor::OnWidgetAnimDrawerSequencerOpened);
		WidgetAnimSequencerDrawer.OnDrawerDismissedDelegate.BindSP(BP.Get(), &FWidgetBlueprintEditor::OnWidgetAnimDrawerSequencerDismissed);
		WidgetAnimSequencerDrawer.ButtonText = LOCTEXT("StatusBar_WidgetAnimSequencer", "Animations");
		WidgetAnimSequencerDrawer.ToolTipText = LOCTEXT("StatusBar_WidgetAnimSequencerToolTip", "Opens animation sequencer (Ctrl+Shift+Space Bar).");
		WidgetAnimSequencerDrawer.Icon = FAppStyle::Get().GetBrush("UMGEditor.AnimTabIcon");
		BP->RegisterDrawer(MoveTemp(WidgetAnimSequencerDrawer), 1);
	}

	// Toggle any active tool palette modes
	for (TSharedPtr<FUICommandInfo>& Command : BP->ToolPaletteCommands)
	{
		if (Command && BP->GetToolkitCommands()->GetCheckState(Command.ToSharedRef()) == ECheckBoxState::Checked)
		{
			BP->GetToolkitCommands()->TryExecuteAction(Command.ToSharedRef());

			// @FIXME: DarenC - Executing twice since a single action is a toggle, 
			// need toggle twice if in incorrect state. This is not relevant when we move to drop downs.
			BP->GetToolkitCommands()->TryExecuteAction(Command.ToSharedRef());
		}
	}

	BP->OnEnteringDesigner();
}

#undef LOCTEXT_NAMESPACE
