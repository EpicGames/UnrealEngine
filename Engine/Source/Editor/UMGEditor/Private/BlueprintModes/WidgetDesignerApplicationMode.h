// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintModes/WidgetBlueprintApplicationMode.h"

/////////////////////////////////////////////////////
// FWidgetDesignerApplicationMode

class FWidgetDesignerApplicationMode : public FWidgetBlueprintApplicationMode
{
public:
	FWidgetDesignerApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;
	// End of FApplicationMode interface
};
