// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/WidgetGraphApplicationMode.h"

#include "BlueprintEditorTabs.h"
#include "SBlueprintEditorToolbar.h"

#include "UMGEditorModule.h"
#include "WidgetBlueprintEditorToolbar.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "ToolMenus.h"

/////////////////////////////////////////////////////
// FWidgetGraphApplicationMode

FWidgetGraphApplicationMode::FWidgetGraphApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor)
	: FWidgetBlueprintApplicationMode(InWidgetEditor, FWidgetBlueprintApplicationModes::GraphMode)
{
	TabLayout = FTabManager::NewLayout( "WidgetBlueprintEditor_Graph_Layout_v2x1" )
		->AddArea
		(
			FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
					)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.70f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.80f )
						->AddTab( "Document", ETabState::ClosedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.20f )
						->AddTab( FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					)
				)
			)
		);
	
	IUMGEditorModule& UMGEditorModule = FModuleManager::GetModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterTabsForEditor().Broadcast(*this, TabFactories);

	// Add any extenders specified by the UMG Editor Module
	// Note: Used by WidgetEditorModeUILayer to register the toolbox tab
	if (LayoutExtender)
	{
		UMGEditorModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
		TabLayout->ProcessExtensions(*LayoutExtender);
	}

	ToolbarExtender = MakeShareable(new FExtender);
	InWidgetEditor->GetWidgetToolbarBuilder()->AddWidgetBlueprintEditorModesToolbar(ToolbarExtender);
	InWidgetEditor->RegisterModeToolbarIfUnregistered(GetModeName());

	FName OutParentToolbarName;
	FName ToolBarname = InWidgetEditor->GetToolMenuToolbarNameForMode(GetModeName(), OutParentToolbarName);
	if (UToolMenu* Toolbar = UToolMenus::Get()->FindMenu(ToolBarname))
	{
		InWidgetEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InWidgetEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InWidgetEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
		InWidgetEditor->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}
}

void FWidgetGraphApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = GetBlueprintEditor();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
	BP->PushTabFactories(TabFactories);
}

void FWidgetGraphApplicationMode::PostActivateMode()
{
	FWidgetBlueprintApplicationMode::PostActivateMode();
	TSharedPtr<FWidgetBlueprintEditor> BP = GetBlueprintEditor();
	
	// Select associated widget variable in 'My Blueprint'.
	const TSet<FWidgetReference>& Selected = BP->GetSelectedWidgets();
	if (Selected.Num() == 1)
	{
		for (const FWidgetReference& WidgetRef : Selected)
		{
			if (WidgetRef.IsValid())
			{
				BP->SelectGraphActionItemByName(WidgetRef.GetPreview()->GetFName());
			}
		}
	}
}
