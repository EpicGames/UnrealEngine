// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/WidgetPreviewApplicationMode.h"

#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "SBlueprintEditorToolbar.h"
#include "TabFactory/PreviewDetailsTabSummoner.h"
#include "TabFactory/DetailsTabSummoner.h"
#include "TabFactory/PreviewTabSummoner.h"
#include "ToolMenus.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprintEditorToolbar.h"

namespace UE::UMG::Editor
{

FWidgetPreviewApplicationMode::FWidgetPreviewApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor)
	: FWidgetBlueprintApplicationMode(InWidgetEditor, FWidgetBlueprintApplicationModes::PreviewMode)
{
	TabLayout = FTabManager::NewLayout( "WidgetBlueprintEditor_Preview_Layout_v1" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.8f)
				->AddTab(FWidgetPreviewTabSummoner::TabID, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.15f)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FPreviewDetailsTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
		);
	
	TabFactories.RegisterFactory(MakeShareable(new FWidgetPreviewTabSummoner(InWidgetEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FPreviewDetailsTabSummoner(InWidgetEditor)));
	//TabFactories.RegisterFactory(MakeShareable(new FPreviewSettingsTabSummoner(InWidgetEditor))); Named: Parameters? contains: localization, background, size

	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterTabsForEditor().Broadcast(*this, TabFactories);

	ToolbarExtender = UMGEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders();
	InWidgetEditor->GetWidgetToolbarBuilder()->AddWidgetBlueprintEditorModesToolbar(ToolbarExtender);

	if (UToolMenu* Toolbar = InWidgetEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InWidgetEditor->GetWidgetToolbarBuilder()->AddWidgetReflector(Toolbar);
	}
}

void FWidgetPreviewApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = GetBlueprintEditor();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());
	BP->PushTabFactories(TabFactories);
}

void FWidgetPreviewApplicationMode::PostActivateMode()
{

}

} //namespace
