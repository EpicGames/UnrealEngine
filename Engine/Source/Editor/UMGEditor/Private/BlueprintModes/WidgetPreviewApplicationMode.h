// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintModes/WidgetBlueprintApplicationMode.h"

namespace UE::UMG::Editor
{

class FWidgetPreviewApplicationMode : public FWidgetBlueprintApplicationMode
{
public:
	FWidgetPreviewApplicationMode(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor);

	static bool IsDebugModeEnabled();

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PostActivateMode() override;
	// End of FApplicationMode interface
};

} // namespace