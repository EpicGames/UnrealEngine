// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/AssetThumbnailWidget.h"

#include "AssetThumbnail.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Styling/AppStyle.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetThumbnailWidget)

#define LOCTEXT_NAMESPACE "UMG"

FAssetThumbnailConfig FAssetThumbnailWidgetSettings::ToThumbnailConfig() const
{
	FAssetThumbnailConfig Result;
	Result.bForceGenericThumbnail				= bForceGenericThumbnail;
	Result.bAllowHintText						= bAllowHintText;
	Result.bAllowRealTimeOnHovered				= bAllowRealTimeOnHovered;
	Result.AllowAssetSpecificThumbnailOverlay	= bAllowAssetSpecificThumbnailOverlay;
	Result.ThumbnailLabel						= static_cast<EThumbnailLabel::Type>(ThumbnailLabel);
	Result.HighlightedText						= TAttribute<FText>::CreateLambda([Delegate = HighlightedTextDelegate]()
	{
		return Delegate.IsBound() ? Delegate.Execute() : FText::GetEmpty();
	});
	Result.HintColorAndOpacity					= HintColorAndOpacity;
	Result.AssetTypeColorOverride				= bOverrideAssetTypeColor ? AssetTypeColorOverride : TOptional<FLinearColor>();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Result.Padding								= Padding;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Result.GenericThumbnailSize					= GenericThumbnailSize;
	Result.ColorStripOrientation				= static_cast<EThumbnailColorStripOrientation>(ColorStripOrientation);
	return Result;
}
void UAssetThumbnailWidget::SetThumbnailSettings(const FAssetThumbnailWidgetSettings& InThumbnailSettings)
{
	ThumbnailSettings = InThumbnailSettings;
	UpdateNativeAssetThumbnailWidget();
}

void UAssetThumbnailWidget::SetAsset(const FAssetData& AssetData)
{
	AssetToShow = AssetData;
	UpdateNativeAssetThumbnailWidget();
}

void UAssetThumbnailWidget::SetAssetByObject(UObject* Object)
{
	AssetToShow = Object;
	UpdateNativeAssetThumbnailWidget();
}

void UAssetThumbnailWidget::SetResolution(const FIntPoint& InResolution)
{
	Resolution = InResolution;
	UpdateNativeAssetThumbnailWidget();
}

void UAssetThumbnailWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();
	UpdateNativeAssetThumbnailWidget();
}

void UAssetThumbnailWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	DisplayedWidget.Reset();
	ThumbnailRenderer.Reset();
}

TSharedRef<SWidget> UAssetThumbnailWidget::RebuildWidget()
{
	DisplayedWidget = SNew(SBorder)
		.Padding(0.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.BorderImage(FAppStyle::GetBrush("NoBorder"));
	UpdateNativeAssetThumbnailWidget();
	return DisplayedWidget.ToSharedRef();
}

const FText UAssetThumbnailWidget::GetPaletteCategory()
{
	return LOCTEXT("Editor", "Editor");
}

void UAssetThumbnailWidget::UpdateNativeAssetThumbnailWidget()
{
	if (!DisplayedWidget)
	{
		return;
	}

	const bool bCreateNewThumbnailRenderer = AssetToShow.IsValid() && (!ThumbnailRenderer || ThumbnailRenderer->GetSize() != Resolution);
	if (bCreateNewThumbnailRenderer)
	{
		ThumbnailRenderer = MakeShared<FAssetThumbnail>(AssetToShow, FMath::Clamp(Resolution.X, 1, 1024), FMath::Clamp(Resolution.Y, 1, 1024), UThumbnailManager::Get().GetSharedThumbnailPool());
	}
	else if (ThumbnailRenderer && (AssetToShow.IsValid() && ThumbnailRenderer->GetAssetData() != AssetToShow))
	{
		ThumbnailRenderer->SetAsset(AssetToShow);
	}
	
	if (ThumbnailRenderer)
	{
		DisplayedWidget->SetContent(ThumbnailRenderer->MakeThumbnailWidget(ThumbnailSettings.ToThumbnailConfig()));
	}
	else
	{
		DisplayedWidget->SetContent(SNew(STextBlock).Text(LOCTEXT("NoAssetData", "Call SetThumbnailAsset")));
	}
}

#undef LOCTEXT_NAMESPACE
