// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CanvasSlotCustomization.h"

#include "Animation/CurveSequence.h"
#include "Components/CanvasPanelSlot.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformMath.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/TypeHash.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/Anchors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class FActiveTimerHandle;
class SWidget;
class UObject;
struct FGeometry;
struct FPointerEvent;



#define LOCTEXT_NAMESPACE "UMG"

class SAnchorPreviewWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAnchorPreviewWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, 
		TSharedPtr<IPropertyHandle> AnchorsHandle,
		TSharedPtr<IPropertyHandle> AlignmentHandle,
		TSharedPtr<IPropertyHandle> OffsetsHandle,
		FText LabelText,
		FAnchors Anchors)
	{
		bIsHovered = false;
		ResizeCurve = FCurveSequence(0, 0.40f);

		RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SAnchorPreviewWidget::UpdateAnimation));

		ChildSlot
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleSharpButton")
			.ButtonColorAndOpacity(FLinearColor(FColor(40, 40, 40)))
			.OnClicked(this, &SAnchorPreviewWidget::OnAnchorClicked, AnchorsHandle, AlignmentHandle, OffsetsHandle, Anchors)
			.AddMetaData<FTagMetaData>(FTagMetaData(FName{ LabelText.ToString() }))
			.ContentPadding(FMargin(2.0f, 2.0f))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("UMGEditor.AnchorGrid"))
					.Padding(0)
					[
						SNew(SBox)
						.WidthOverride(64.0f)
						.HeightOverride(64.0f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(this, &SAnchorPreviewWidget::GetCurrentWidth)
							.HeightOverride(this, &SAnchorPreviewWidget::GetCurrentHeight)
							.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AnimatedAnchorPreviewBox")))
							[
								SNew(SBorder)
								.Padding(1)
								[
									SNew(SConstraintCanvas)

									+ SConstraintCanvas::Slot()
									.Anchors(Anchors)
									.Offset(FMargin(0.0f, 0.0f, Anchors.IsStretchedHorizontal() ? 0.0f : 15.0f, Anchors.IsStretchedVertical() ? 0.0f : 15.0f))
									.Alignment(FVector2D(Anchors.IsStretchedHorizontal() ? 0.0f : Anchors.Minimum.X, Anchors.IsStretchedVertical() ? 0.0f : Anchors.Minimum.Y))
									[
										SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("UMGEditor.AnchoredWidget"))
									]
								]
							]
						]
					]
				]
			]
		];
	}
	
	EActiveTimerReturnType UpdateAnimation(double InCurrentTime, float InDeltaTime)
	{
		if ( bIsHovered )
		{
			if ( !ResizeCurve.IsPlaying() )
			{
				if ( ResizeCurve.IsAtStart() )
				{
					ResizeCurve.Play( this->AsShared() );
				}
				else if ( ResizeCurve.IsAtEnd() )
				{
					ResizeCurve.Reverse();
				}
			}
		}
		//Make sure the preview animation goes all the way back to the initial position before we stop ticking
		else if ( !ResizeCurve.IsAtStart() && !ResizeCurve.IsInReverse() )
		{
			ResizeCurve.Reverse();
		}

		return EActiveTimerReturnType::Continue;
	}

	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		bIsHovered = true;
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent )
	{
		bIsHovered = false;
	}

private:
	FOptionalSize GetCurrentWidth() const
	{
		return 48 + ( 16 * ResizeCurve.GetLerp() );
	}

	FOptionalSize GetCurrentHeight() const //-V524
	{
		return 48 + ( 16 * ResizeCurve.GetLerp() );
	}

	FReply OnAnchorClicked(
		TSharedPtr<IPropertyHandle> AnchorsHandle,
		TSharedPtr<IPropertyHandle> AlignmentHandle,
		TSharedPtr<IPropertyHandle> OffsetsHandle,
		FAnchors Anchors)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAnchors", "Changed Anchors"));

		{
			const FString Value = FString::Printf(TEXT("(Minimum=(X=%f,Y=%f),Maximum=(X=%f,Y=%f))"), Anchors.Minimum.X, Anchors.Minimum.Y, Anchors.Maximum.X, Anchors.Maximum.Y);
			AnchorsHandle->SetValueFromFormattedString(Value);
		}

		// If shift is down, update the alignment/pivot point to match the anchor position.
		if ( FSlateApplication::Get().GetModifierKeys().IsShiftDown() )
		{
			const FString Value = FString::Printf(TEXT("(X=%f,Y=%f)"), Anchors.IsStretchedHorizontal() ? 0.0f : Anchors.Minimum.X, Anchors.IsStretchedVertical() ? 0.0f : Anchors.Minimum.Y);
			AlignmentHandle->SetValueFromFormattedString(Value);
		}

		// If control is down, update position to be reset to 0 in the directions where it's appropriate.
		if ( FSlateApplication::Get().GetModifierKeys().IsControlDown() )
		{
			TArray<void*> RawOffsetData;
			OffsetsHandle->AccessRawData(RawOffsetData);
			FMargin* Offsets = reinterpret_cast<FMargin*>( RawOffsetData[0] );

			const FString Value = FString::Printf(TEXT("(Left=%f,Top=%f,Right=%f,Bottom=%f)"), 0.0f, 0.0f, Anchors.IsStretchedHorizontal() ? 0.0f : Offsets->Right, Anchors.IsStretchedVertical() ? 0.0f : Offsets->Bottom);
			OffsetsHandle->SetValueFromFormattedString(Value);
		}

		// Close the menu
		FSlateApplication::Get().DismissAllMenus();

		return FReply::Handled();
	}

private:
	FCurveSequence ResizeCurve;
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;
	bool bIsHovered;
};


// FCanvasSlotCustomization
////////////////////////////////////////////////////////////////////////////////

void FCanvasSlotCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FCanvasSlotCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CustomizeLayoutData(PropertyHandle, ChildBuilder, CustomizationUtils);

	FillOutChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
}

void FCanvasSlotCustomization::FillOutChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Generate all the other children
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for ( uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++ )
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		if ( ChildHandle->IsCustomized() )
		{
			continue;
		}

		if ( ChildHandle->GetProperty() == NULL )
		{
			FillOutChildren(ChildHandle, ChildBuilder, CustomizationUtils);
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle);
		}
	}
}

void FCanvasSlotCustomization::CustomizeLayoutData(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName LayoutDataName = GET_MEMBER_NAME_CHECKED(UCanvasPanelSlot, LayoutData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSharedPtr<IPropertyHandle> LayoutData = PropertyHandle->GetChildHandle(LayoutDataName);
	if ( LayoutData.IsValid() )
	{
		LayoutData->MarkHiddenByCustomization();

		CustomizeAnchors(LayoutData, ChildBuilder, CustomizationUtils);
		CustomizeOffsets(LayoutData, ChildBuilder, CustomizationUtils);

		TSharedPtr<IPropertyHandle> AlignmentHandle = LayoutData->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Alignment));
		AlignmentHandle->MarkHiddenByCustomization();
		ChildBuilder.AddProperty(AlignmentHandle.ToSharedRef());
	}
}

void FCanvasSlotCustomization::CustomizeOffsets(TSharedPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> OffsetsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Offsets));
	TSharedPtr<IPropertyHandle> AnchorsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Anchors));

	OffsetsHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> LeftHandle = OffsetsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMargin, Left));
	TSharedPtr<IPropertyHandle> TopHandle = OffsetsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMargin, Top));
	TSharedPtr<IPropertyHandle> RightHandle = OffsetsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMargin, Right));
	TSharedPtr<IPropertyHandle> BottomHandle = OffsetsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMargin, Bottom));

	IDetailPropertyRow& LeftRow = ChildBuilder.AddProperty(LeftHandle.ToSharedRef());
	IDetailPropertyRow& TopRow = ChildBuilder.AddProperty(TopHandle.ToSharedRef());
	IDetailPropertyRow& RightRow = ChildBuilder.AddProperty(RightHandle.ToSharedRef());
	IDetailPropertyRow& BottomRow = ChildBuilder.AddProperty(BottomHandle.ToSharedRef());

	TAttribute<FText> LeftLabel = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FCanvasSlotCustomization::GetOffsetLabel, PropertyHandle, Orient_Horizontal, LOCTEXT("PositionX", "Position X"), LOCTEXT("OffsetLeft", "Offset Left")));
	TAttribute<FText> TopLabel = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FCanvasSlotCustomization::GetOffsetLabel, PropertyHandle, Orient_Vertical, LOCTEXT("PositionY", "Position Y"), LOCTEXT("OffsetTop", "Offset Top")));
	TAttribute<FText> RightLabel = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FCanvasSlotCustomization::GetOffsetLabel, PropertyHandle, Orient_Horizontal, LOCTEXT("SizeX", "Size X"), LOCTEXT("OffsetRight", "Offset Right")));
	TAttribute<FText> BottomLabel = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FCanvasSlotCustomization::GetOffsetLabel, PropertyHandle, Orient_Vertical, LOCTEXT("SizeY", "Size Y"), LOCTEXT("OffsetBottom", "Offset Bottom")));

	CreateEditorWithDynamicLabel(LeftRow, LeftLabel);
	CreateEditorWithDynamicLabel(TopRow, TopLabel);
	CreateEditorWithDynamicLabel(RightRow, RightLabel);
	CreateEditorWithDynamicLabel(BottomRow, BottomLabel);
}

void FCanvasSlotCustomization::CreateEditorWithDynamicLabel(IDetailPropertyRow& PropertyRow, TAttribute<FText> TextAttribute)
{
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	PropertyRow.CustomWidget(/*bShowChildren*/ true)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(TextAttribute)
	]
	.ValueContent()
	[
		ValueWidget.ToSharedRef()
	];
}

FText FCanvasSlotCustomization::GetOffsetLabel(TSharedPtr<IPropertyHandle> PropertyHandle, EOrientation Orientation, FText NonStretchingLabel, FText StretchingLabel)
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	
	if ( Objects.Num() == 1 && Objects[0] != nullptr )
	{
		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);

		FAnchorData* AnchorData = reinterpret_cast<FAnchorData*>( RawData[0] );

		const bool bStretching =
			( Orientation == Orient_Horizontal && AnchorData->Anchors.IsStretchedHorizontal() ) ||
			( Orientation == Orient_Vertical && AnchorData->Anchors.IsStretchedVertical() );

		return bStretching ? StretchingLabel : NonStretchingLabel;
	}
	else
	{
		return StretchingLabel;
	}
}

void FCanvasSlotCustomization::CustomizeAnchors(TSharedPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> AnchorsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Anchors));
	TSharedPtr<IPropertyHandle> AlignmentHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Alignment));
	TSharedPtr<IPropertyHandle> OffsetsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnchorData, Offsets));

	AnchorsHandle->MarkHiddenByCustomization();

	IDetailPropertyRow& AnchorsPropertyRow = ChildBuilder.AddProperty(AnchorsHandle.ToSharedRef());

	const float FillDividePadding = 1;

	FInputChord ShiftKey(EKeys::Invalid, true, false, false, false);
	FInputChord CtrlKey(EKeys::Invalid, false, true, false, false);

	AnchorsPropertyRow.CustomWidget(/*bShowChildren*/ true)
		.NameContent()
		[
			SNew( STextBlock )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text( LOCTEXT("Anchors", "Anchors") )
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AnchorsText", "Anchors"))
			]
			.MenuContent()
			[
				SNew(SBorder)
				//.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))FAppStyle::GetBrush("WhiteBrush")/*FAppStyle::GetBrush("ToolPanel.GroupBorder")*/)
				.Padding(5)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FLinearColor(FColor(66, 139, 202)))
					.Padding(0)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SUniformGridPanel)

								// Top Row
								+ SUniformGridPanel::Slot(0, 0)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("TopLeft", "Top/Left"), FAnchors(0, 0, 0, 0))
								]

								+ SUniformGridPanel::Slot(1, 0)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("TopCenter", "Top/Center"), FAnchors(0.5, 0, 0.5, 0))
								]

								+ SUniformGridPanel::Slot(2, 0)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("TopRight", "Top/Right"), FAnchors(1, 0, 1, 0))
								]

								// Center Row
								+ SUniformGridPanel::Slot(0, 1)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("CenterLeft", "Center/Left"), FAnchors(0, 0.5, 0, 0.5))
								]

								+ SUniformGridPanel::Slot(1, 1)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("CenterCenter", "Center/Center"), FAnchors(0.5, 0.5, 0.5, 0.5))
								]

								+ SUniformGridPanel::Slot(2, 1)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("CenterRight", "Center/Right"), FAnchors(1, 0.5, 1, 0.5))
								]

								// Bottom Row
								+ SUniformGridPanel::Slot(0, 2)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("BottomLeft", "Bottom/Left"), FAnchors(0, 1, 0, 1))
								]

								+ SUniformGridPanel::Slot(1, 2)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("BottomCenter", "Bottom/Center"), FAnchors(0.5, 1, 0.5, 1))
								]

								+ SUniformGridPanel::Slot(2, 2)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("BottomRight", "Bottom/Right"), FAnchors(1, 1, 1, 1))
								]
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(FillDividePadding, 0, 0, 0)
							[
								SNew(SUniformGridPanel)

								+ SUniformGridPanel::Slot(0, 0)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("TopFill", "Top/Fill"), FAnchors(0, 0, 1, 0))
								]

								+ SUniformGridPanel::Slot(0, 1)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("CenterFill", "Center/Fill"), FAnchors(0, 0.5f, 1, 0.5f))
								]

								+ SUniformGridPanel::Slot(0, 2)
								[
									SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("BottomFill", "Bottom/Fill"), FAnchors(0, 1, 1, 1))
								]
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, FillDividePadding, 0, 0)
						[
							SNew(SHorizontalBox)

							// Fill Row
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("FillLeft", "Fill/Left"), FAnchors(0, 0, 0, 1))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("FillCenter", "Fill/Center"), FAnchors(0.5, 0, 0.5, 1))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("FillRight", "Fill/Right"), FAnchors(1, 0, 1, 1))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(FillDividePadding, 0, 0, 0)
							[
								SNew(SAnchorPreviewWidget, AnchorsHandle, AlignmentHandle, OffsetsHandle, LOCTEXT("FillFill", "Fill/Fill"), FAnchors(0, 0, 1, 1))
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
							.BorderBackgroundColor(FLinearColor(0.016f, 0.016f, 0.016f))
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(FText::Format(LOCTEXT("ShiftResetsAlignment", "Hold {0} to update the alignment to match."), ShiftKey.GetInputText()))
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(FText::Format(LOCTEXT("ControlResetsPosition", "Hold {0} to update the position to match."), CtrlKey.GetInputText()))
								]
							]
						]
					]
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
