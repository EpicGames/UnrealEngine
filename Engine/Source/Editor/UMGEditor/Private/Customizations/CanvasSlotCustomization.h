// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "WidgetBlueprint.h"

class IDetailChildrenBuilder;
class IDetailPropertyRow;
class IPropertyHandle;

class FCanvasSlotCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(UBlueprint* Blueprint)
	{
		return MakeShareable(new FCanvasSlotCustomization(Blueprint));
	}

	FCanvasSlotCustomization(UBlueprint* InBlueprint)
		: Blueprint(Cast<UWidgetBlueprint>(InBlueprint))
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
private:

	void CustomizeLayoutData(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
	void CustomizeAnchors(TSharedPtr<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
	void CustomizeOffsets(TSharedPtr<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
	
	void FillOutChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);

	void CreateEditorWithDynamicLabel(IDetailPropertyRow& PropertyRow, TAttribute<FText> TextAttribute);

	static FText GetOffsetLabel(TSharedPtr<IPropertyHandle> AnchorStructureHandle, EOrientation Orientation, FText NonStretchingLabel, FText StretchingLabel);

private:
	UWidgetBlueprint* Blueprint;
	
};
