// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DynamicEntryBoxDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Components/DynamicEntryBox.h"
#include "PropertyCustomizationHelpers.h"

//////////////////////////////////////////////////////////////////////////
// FDynamicEntryBoxBaseDetails
//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FDynamicEntryBoxBaseDetails::MakeInstance()
{
	return MakeShareable(new FDynamicEntryBoxBaseDetails());
}

void FDynamicEntryBoxBaseDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}
	EntryBox = Cast<UDynamicEntryBox>(Objects[0].Get());
	if (!EntryBox.IsValid())
	{
		return;
	}

	IDetailCategoryBuilder& EntryLayoutCategory = DetailLayout.EditCategory(TEXT("EntryLayout"));

	const TAttribute<bool> CanEditAignmentAttribute(this, &FDynamicEntryBoxDetails::CanEditAlignment);
	EntryLayoutCategory.AddProperty(DetailLayout.GetProperty(TEXT("EntryHorizontalAlignment")))
		.IsEnabled(CanEditAignmentAttribute);
	EntryLayoutCategory.AddProperty(DetailLayout.GetProperty(TEXT("EntryVerticalAlignment")))
		.IsEnabled(CanEditAignmentAttribute);

	EntryLayoutCategory.AddProperty(DetailLayout.GetProperty(TEXT("MaxElementSize")))
		.IsEnabled(TAttribute<bool>(this, &FDynamicEntryBoxDetails::CanEditMaxElementSize));
	EntryLayoutCategory.AddProperty(DetailLayout.GetProperty(TEXT("EntrySpacing")))
		.IsEnabled(TAttribute<bool>(this, &FDynamicEntryBoxDetails::CanEditEntrySpacing));
	EntryLayoutCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDynamicEntryBoxBase, SpacingPattern)))
		.IsEnabled(TAttribute<bool>(this, &FDynamicEntryBoxDetails::CanEditSpacingPattern));
}

bool FDynamicEntryBoxBaseDetails::CanEditSpacingPattern() const
{
	if (EntryBox.IsValid())
	{
		return EntryBox->GetBoxType() == EDynamicBoxType::Overlay;
	}
	return false;
}

bool FDynamicEntryBoxBaseDetails::CanEditEntrySpacing() const
{
	if (EntryBox.IsValid())
	{
		return EntryBox->SpacingPattern.Num() == 0;
	}
	return false;
}

bool FDynamicEntryBoxBaseDetails::CanEditAlignment() const
{
	if (EntryBox.IsValid())
	{
		return EntryBox->GetBoxType() != EDynamicBoxType::Overlay || CanEditEntrySpacing();
	}
	return false;
}

bool FDynamicEntryBoxBaseDetails::CanEditMaxElementSize() const
{
	if (EntryBox.IsValid())
	{
		const EDynamicBoxType BoxType = EntryBox->GetBoxType();
		return BoxType == EDynamicBoxType::Horizontal || BoxType == EDynamicBoxType::Vertical;
	}
	return false;
}

//////////////////////////////////////////////////////////////////////////
// FDynamicEntryBoxDetails
//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FDynamicEntryBoxDetails::MakeInstance()
{
	return MakeShareable(new FDynamicEntryBoxDetails());
}

void FDynamicEntryBoxDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FDynamicEntryBoxBaseDetails::CustomizeDetails(DetailLayout);

	if (EntryBox.IsValid())
	{
		IDetailCategoryBuilder& EntryLayoutCategory = DetailLayout.EditCategory(TEXT("EntryLayout"));
		AddEntryClassPicker(*EntryBox, EntryLayoutCategory, DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDynamicEntryBox, EntryWidgetClass)));
	}
}
