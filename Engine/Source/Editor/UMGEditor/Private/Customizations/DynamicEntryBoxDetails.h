// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/DynamicEntryWidgetDetailsBase.h"

class IPropertyHandle;
class UDynamicEntryBoxBase;

//////////////////////////////////////////////////////////////////////////
// FDynamicEntryBoxBaseDetails
//////////////////////////////////////////////////////////////////////////

class FDynamicEntryBoxBaseDetails : public FDynamicEntryWidgetDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/* Main customization of details */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	TWeakObjectPtr<UDynamicEntryBoxBase> EntryBox;

private:
	bool CanEditSpacingPattern() const;
	bool CanEditEntrySpacing() const;
	bool CanEditAlignment() const;
	bool CanEditMaxElementSize() const;
};

//////////////////////////////////////////////////////////////////////////
// FDynamicEntryBoxDetails
//////////////////////////////////////////////////////////////////////////

class FDynamicEntryBoxDetails : public FDynamicEntryBoxBaseDetails
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/* Main customization of details */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
