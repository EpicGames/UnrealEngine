// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DynamicEntryWidgetDetailsBase.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"

const UClass* FDynamicEntryWidgetDetailsBase::GetSelectedEntryClass() const
{
	if (EntryClassHandle.IsValid())
	{
		const UObject* SelectedClassObj = nullptr;
		EntryClassHandle->GetValue(SelectedClassObj);
		return Cast<UClass>(SelectedClassObj);

	}
	return nullptr;
}

void FDynamicEntryWidgetDetailsBase::HandleNewEntryClassSelected(const UClass* NewEntryClass) const
{
	if (EntryClassHandle.IsValid())
	{
		EntryClassHandle->SetValueFromFormattedString(NewEntryClass->GetPathName());
	}
}

void FDynamicEntryWidgetDetailsBase::AddEntryClassPickerInternal(const UClass* EntryBaseClass, const UClass* RequiredEntryInterface, IDetailCategoryBuilder& CategoryBuilder) const
{
	// Create a custom class picker here that filters according to the EntryClass 
	IDetailPropertyRow& EntryClassRow = CategoryBuilder.AddProperty(EntryClassHandle);
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	EntryClassRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	EntryClassRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SClassPropertyEntryBox)
			.AllowNone(false)
			.IsBlueprintBaseOnly(true)
			.RequiredInterface(RequiredEntryInterface)
			.MetaClass(EntryBaseClass ? EntryBaseClass : UUserWidget::StaticClass())
			.SelectedClass(this, &FDynamicEntryWidgetDetailsBase::GetSelectedEntryClass)
			.OnSetClass(this, &FDynamicEntryWidgetDetailsBase::HandleNewEntryClassSelected)
		];
}
