// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Blueprint/UserWidget.h"

class IDetailCategoryBuilder;
class IPropertyHandle;

/** Base helper for customizations on widget classes that dynamically generate entries of a given widget class */
class FDynamicEntryWidgetDetailsBase : public IDetailCustomization
{
protected:
	template <typename WidgetT>
	void AddEntryClassPicker(WidgetT& WidgetInstance, IDetailCategoryBuilder& CategoryBuilder, TSharedRef<IPropertyHandle> InEntryClassHandle)
	{
		static const FName EntryInterfaceMetaKey = TEXT("EntryInterface");
		static const FName EntryClassMetaKey = TEXT("EntryClass");

		EntryClassHandle = InEntryClassHandle;

		const UClass* RequiredInterface = nullptr;
		const UClass* BaseClass = nullptr; 

		if (UUserWidget* OwningUserWidget = WidgetInstance.template GetTypedOuter<UUserWidget>())
		{
			// Find the native BindWidget FProperty that corresponds to this table view
			for (TFieldIterator<FObjectProperty> PropertyIter(OwningUserWidget->GetClass()); PropertyIter; ++PropertyIter)
			{
				FObjectProperty* Property = *PropertyIter;
				if (Property->PropertyClass && Property->PropertyClass->IsChildOf<WidgetT>())
				{
					WidgetT** WidgetPropertyInstance = Property->ContainerPtrToValuePtr<WidgetT*>(OwningUserWidget);
					if (*WidgetPropertyInstance == &WidgetInstance)
					{
						BaseClass = Property->GetClassMetaData(EntryClassMetaKey);
						RequiredInterface = Property->GetClassMetaData(EntryInterfaceMetaKey);
						break;
					}
				}
			}
		}

		// If the property binding didn't exist or didn't specify a class/interface, check with the class itself (including parents as needed)
		const UClass* WidgetClass = WidgetInstance.GetClass();
		while (WidgetClass && WidgetClass->IsChildOf<UWidget>() && (!BaseClass || !RequiredInterface))
		{
			if (!BaseClass)
			{
				BaseClass = WidgetClass->GetClassMetaData(EntryClassMetaKey);
			}
			if (!RequiredInterface)
			{
				RequiredInterface = WidgetClass->GetClassMetaData(EntryInterfaceMetaKey);
			}
			WidgetClass = WidgetClass->GetSuperClass();
		}
		
		// If a valid base class has been specified, create a custom class picker that filters accordingly
		if (BaseClass || RequiredInterface)
		{
			AddEntryClassPickerInternal(BaseClass, RequiredInterface, CategoryBuilder);
		}
	}

private:
	void AddEntryClassPickerInternal(const UClass* EntryBaseClass, const UClass* RequiredEntryInterface, IDetailCategoryBuilder& CategoryBuilder) const;

	const UClass* GetSelectedEntryClass() const;
	void HandleNewEntryClassSelected(const UClass* NewEntryClass) const;
	
	TSharedPtr<IPropertyHandle> EntryClassHandle;
};