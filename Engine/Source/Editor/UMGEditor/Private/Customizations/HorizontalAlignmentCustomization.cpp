// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/HorizontalAlignmentCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSegmentedControl.h"

class IDetailChildrenBuilder;


#define LOCTEXT_NAMESPACE "UMG"

void FHorizontalAlignmentCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	const FMargin OuterPadding(2);
	const FMargin ContentPadding(2);

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0)
	[
		SNew(SSegmentedControl<EHorizontalAlignment>)
		.Value(this, &FHorizontalAlignmentCustomization::GetCurrentAlignment, PropertyHandle)
		.OnValueChanged(this, &FHorizontalAlignmentCustomization::OnCurrentAlignmentChanged, PropertyHandle)
		+SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Left)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Left"))
			.ToolTip(LOCTEXT("HAlignLeft", "Left Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Center)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Center"))
			.ToolTip(LOCTEXT("HAlignCenter", "Center Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Right)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Right"))
			.ToolTip(LOCTEXT("HAlignRight", "Right Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Fill)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Fill"))
			.ToolTip(LOCTEXT("HAlignFill", "Fill Horizontally"))
	];
}

void FHorizontalAlignmentCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

EHorizontalAlignment FHorizontalAlignmentCustomization::GetCurrentAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return (EHorizontalAlignment)Value;
	}

	return HAlign_Fill;
}

void FHorizontalAlignmentCustomization::OnCurrentAlignmentChanged(EHorizontalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)NewAlignment);
}

#undef LOCTEXT_NAMESPACE
