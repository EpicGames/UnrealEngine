// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;

class FHorizontalAlignmentCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FHorizontalAlignmentCustomization());
	}

	FHorizontalAlignmentCustomization()
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
	void HandleCheckStateChanged(ECheckBoxState InCheckboxState, TSharedRef<IPropertyHandle> PropertyHandle, EHorizontalAlignment ToAlignment);

	ECheckBoxState GetCheckState(TSharedRef<IPropertyHandle> PropertyHandle, EHorizontalAlignment ForAlignment) const;

	EHorizontalAlignment GetCurrentAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const;

	void OnCurrentAlignmentChanged(EHorizontalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle);
};
