// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/ListViewBaseDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Components/ListViewBase.h"
#include "PropertyCustomizationHelpers.h"

TSharedRef<IDetailCustomization> FListViewBaseDetails::MakeInstance()
{
	return MakeShareable(new FListViewBaseDetails());
}

void FListViewBaseDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}
	UListViewBase* ListView = Cast<UListViewBase>(Objects[0].Get());
	if (!ListView)
	{
		return;
	}

	IDetailCategoryBuilder& ListEntriesCategory = DetailLayout.EditCategory(TEXT("ListEntries"));
	AddEntryClassPicker(*ListView, ListEntriesCategory, DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UListViewBase, EntryWidgetClass)));
}
