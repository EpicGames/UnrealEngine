// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SlateChildSizeCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/SBoxPanel.h"

class IDetailChildrenBuilder;

#define LOCTEXT_NAMESPACE "UMG"

void FSlateChildSizeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> ValueHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateChildSize, Value));
	TSharedPtr<IPropertyHandle> RuleHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateChildSize, SizeRule));

	if (!ValueHandle.IsValid() || !RuleHandle.IsValid())
	{
		return;
	}

	const FMargin OuterPadding(2, 0);

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(OuterPadding)
		[
			SNew(SSegmentedControl<ESlateSizeRule::Type>)
			.Value(this, &FSlateChildSizeCustomization::GetCurrentSizeRule, RuleHandle)
			.OnValueChanged(this, &FSlateChildSizeCustomization::OnSizeRuleChanged, RuleHandle)
			+ SSegmentedControl<ESlateSizeRule::Type>::Slot(ESlateSizeRule::Automatic)
			.Text(LOCTEXT("Auto", "Auto"))
			.ToolTip(LOCTEXT("Auto_ToolTip", "Only requests as much room as it needs based on the widgets desired size."))
			+ SSegmentedControl<ESlateSizeRule::Type>::Slot(ESlateSizeRule::Fill)
			.Text(LOCTEXT("Fill", "Fill"))
			.ToolTip(LOCTEXT("Fill_ToolTip", "Greedily attempts to fill all available room based on the percentage value 0..1"))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(OuterPadding)
		[
			SNew( SNumericEntryBox<float> )
			.LabelVAlign(VAlign_Center)
			.Visibility(this, &FSlateChildSizeCustomization::GetValueVisiblity, RuleHandle)
			.Value(this, &FSlateChildSizeCustomization::GetValue, ValueHandle)
			.OnValueCommitted(this, &FSlateChildSizeCustomization::HandleValueComitted, ValueHandle)
			.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
		]
	];
}

void FSlateChildSizeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	
}

void FSlateChildSizeCustomization::OnSizeRuleChanged(ESlateSizeRule::Type ToRule, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)ToRule);
}

ESlateSizeRule::Type FSlateChildSizeCustomization::GetCurrentSizeRule(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return (ESlateSizeRule::Type)Value;
	}

	return ESlateSizeRule::Automatic;
}

TOptional<float> FSlateChildSizeCustomization::GetValue(TSharedPtr<IPropertyHandle> ValueHandle) const
{
	float Value;
	if ( ValueHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return Value;
	}

	return TOptional<float>();
}

void FSlateChildSizeCustomization::HandleValueComitted(float NewValue, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> ValueHandle)
{
	ValueHandle->SetValue(NewValue);
}

EVisibility FSlateChildSizeCustomization::GetValueVisiblity(TSharedPtr<IPropertyHandle> RuleHandle) const
{
	uint8 Value;
	if ( RuleHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return Value == ESlateSizeRule::Fill ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
