// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SlateWrapperTypes.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;

class FSlateChildSizeCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FSlateChildSizeCustomization());
	}

	FSlateChildSizeCustomization()
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	void OnSizeRuleChanged(ESlateSizeRule::Type ToRule, TSharedPtr<IPropertyHandle> PropertyHandle);

	ESlateSizeRule::Type GetCurrentSizeRule(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	TOptional<float> GetValue(TSharedPtr<IPropertyHandle> ValueHandle) const;

	void HandleValueComitted(float NewValue, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> ValueHandle);

	EVisibility GetValueVisiblity(TSharedPtr<IPropertyHandle> RuleHandle) const;
};
