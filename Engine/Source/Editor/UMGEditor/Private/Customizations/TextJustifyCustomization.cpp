// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/TextJustifyCustomization.h"

#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSegmentedControl.h"

class IDetailChildrenBuilder;


#define LOCTEXT_NAMESPACE "UMG"

void FTextJustifyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{	
	HeaderRow
	.IsEnabled(TAttribute<bool>(PropertyHandle, &IPropertyHandle::IsEditable))
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SSegmentedControl<ETextJustify::Type>)
		.Value(this, &FTextJustifyCustomization::GetCurrentJustification, PropertyHandle)
		.OnValueChanged(this, &FTextJustifyCustomization::OnJustificationChanged, PropertyHandle)
		+ SSegmentedControl<ETextJustify::Type>::Slot(ETextJustify::Left)
		.Icon(FAppStyle::GetBrush("HorizontalAlignment_Left"))
		.ToolTip(LOCTEXT("AlignTextLeft", "Align Text Left"))
		+ SSegmentedControl<ETextJustify::Type>::Slot(ETextJustify::Center)
		.Icon(FAppStyle::GetBrush("HorizontalAlignment_Center"))
		.ToolTip(LOCTEXT("AlignTextCenter", "Align Text Center"))
		+ SSegmentedControl<ETextJustify::Type>::Slot(ETextJustify::Right)
		.Icon(FAppStyle::GetBrush("HorizontalAlignment_Right"))
		.ToolTip(LOCTEXT("AlignTextRight", "Align Text Right"))
	];
}

void FTextJustifyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FTextJustifyCustomization::OnJustificationChanged(ETextJustify::Type NewState, TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)NewState);
}

ETextJustify::Type FTextJustifyCustomization::GetCurrentJustification(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if ( PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return ETextJustify::Type(Value);
	}

	return ETextJustify::Left;
}

#undef LOCTEXT_NAMESPACE
