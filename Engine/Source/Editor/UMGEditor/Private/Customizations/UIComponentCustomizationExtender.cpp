// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIComponentCustomizationExtender.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "Extensions/UIComponents/NavigationUIComponentCustomization.h"
#include "IDetailsView.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "UIComponentUtils.h"
#include "WidgetBlueprint.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "UIComponentCustomizationExtender"


TSharedPtr<FUIComponentCustomizationExtender> FUIComponentCustomizationExtender::MakeInstance()
{
	return MakeShared<FUIComponentCustomizationExtender>();
}

void FUIComponentCustomizationExtender::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor)
{
	const UUserWidget* PreviewUserWidget = InWidgetBlueprintEditor->GetPreview();
	if (InWidgets.Num() != 1 || !PreviewUserWidget)
	{
		return;
	}

	Widget = InWidgets[0];
	WidgetBlueprintEditor = InWidgetBlueprintEditor;

	// We use the UserWidgetExtension on the preview and it will be migrated to the WBP extension in MigrateFromChain
	if (const UUIComponentUserWidgetExtension* Extension = PreviewUserWidget->GetExtension<UUIComponentUserWidgetExtension>())
	{
		CustomizeComponentPropertyTypes(InDetailLayout.GetDetailsViewSharedPtr());

		TArray<UUIComponent*> ComponentsOnWidget = Extension->GetComponentsFor(Widget.Get());
		for (int32 Index = ComponentsOnWidget.Num() - 1; Index >= 0; Index--)
		{
			if (UUIComponent* Component = ComponentsOnWidget[Index])
			{
				IDetailCategoryBuilder& ComponentCategory = InDetailLayout.EditCategory(Component->GetClass()->GetFName(), FText::GetEmpty(), ECategoryPriority::Important);

				IDetailPropertyRow* PropertyRow = ComponentCategory.AddExternalObjects({ Component }, EPropertyLocation::Default,
					FAddPropertyParams()
					.CreateCategoryNodes(false)
					.AllowChildren(true)
					.HideRootObjectNode(true));

				const FName WidgetName = Widget->GetFName();
				const UClass* ComponentClass = Component->GetClass();
				
				TSharedRef<SWidget> RemoveButton =
					PropertyCustomizationHelpers::MakeDeleteButton(
					FSimpleDelegate::CreateLambda([this, WidgetName, ComponentClass]()
					{
						if (TSharedPtr<FWidgetBlueprintEditor> Editor = WidgetBlueprintEditor.Pin())
						{
							UWidgetBlueprint* WidgetBlueprint = Editor->GetWidgetBlueprintObj();
							FText RemoveErrorMessage;
							if (!FUIComponentUtils::RemoveComponent(Editor.Get(), WidgetBlueprint, ComponentClass, WidgetName, RemoveErrorMessage))
							{
								UE_LOGF(LogUMGEditor, Error,  "%ls", *RemoveErrorMessage.ToString());
							}
						}
					})
					, LOCTEXT("RemoveUIComponent", "Remove Component from Widget.")
				);				
				
				TSharedRef<SWidget> ComponentHeaderContent =
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.Padding(4.0f, 4.0f)
					.VAlign(VAlign_Center) 
					[
						SNew(STextBlock)
						.Text(Component->GetClass()->GetDisplayNameText())
						.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					]
					+ SHorizontalBox::Slot()					
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(4.0f, 4.0f)
					.VAlign(VAlign_Center) 
					[
						RemoveButton
					];				
				ComponentCategory.HeaderContent(ComponentHeaderContent, true);
			}
		}
	}
	else
	{
		TWeakPtr<IDetailsView> DetailsView = InDetailLayout.GetDetailsViewSharedPtr();
		if (!UpdateQueuedForDetailsView.Contains(DetailsView))
		{
			TFunction<void()> UpdateDetailsPanel = [this, DetailsView]()
				{
					if (TSharedPtr<IDetailsView> PinnedDetailsView = DetailsView.Pin())
					{
						PinnedDetailsView->ForceRefresh();
					}
					UpdateQueuedForDetailsView.Remove(DetailsView);
				};
			InWidgetBlueprintEditor->AddPostDesignerLayoutAction(UpdateDetailsPanel);
			UpdateQueuedForDetailsView.Add(DetailsView);
		}
	}
}

void FUIComponentCustomizationExtender::CustomizeComponentPropertyTypes(TSharedPtr<IDetailsView> InDetailsView)
{
	if (!InDetailsView.IsValid())
	{
		return;
	}

	InDetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("NavigationUIComponent"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNavigationUIComponentCustomization::MakeInstance, WidgetBlueprintEditor, Widget->GetFName()));
}

#undef LOCTEXT_NAMESPACE