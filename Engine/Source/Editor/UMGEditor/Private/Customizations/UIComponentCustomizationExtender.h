// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/IBlueprintWidgetCustomizationExtender.h"
#include "Blueprint/UserWidget.h"
#include "UObject/WeakObjectPtr.h"

class SWidget;
class IDetailsView;

class FUIComponentCustomizationExtender : public IBlueprintWidgetCustomizationExtender
{
public:
	static TSharedPtr<FUIComponentCustomizationExtender> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor) override;

private:

	void CustomizeComponentPropertyTypes(TSharedPtr<IDetailsView> InDetailView);

	/** The selected widget in the details panel. */
	TWeakObjectPtr<class UWidget> Widget;
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor;
	TArray<TWeakPtr<IDetailsView>> UpdateQueuedForDetailsView;
};
