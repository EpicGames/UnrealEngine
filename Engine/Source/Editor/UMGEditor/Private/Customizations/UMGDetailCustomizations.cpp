// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/UMGDetailCustomizations.h"
#include "Components/StaticMeshComponent.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/Pawn.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "WidgetBlueprintOperationUtils.h"

#include "Algo/Transform.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Customizations/IBlueprintWidgetCustomizationExtender.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "UMGEditorProjectSettings.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Components/PanelSlot.h"
#include "IPropertyAccessEditor.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "IDetailsView.h"
#include "IDetailPropertyExtensionHandler.h"
#include "IHasPropertyBindingExtensibility.h"
#include "Binding/PropertyBinding.h"
#include "Components/WidgetComponent.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprintEditor.h"
#include "Animation/WidgetAnimation.h"
#include "Styling/SlateTypes.h"

#define LOCTEXT_NAMESPACE "UMG"

class SGraphSchemaActionButton : public SCompoundWidget, public FGCObject
{
public:

	SLATE_BEGIN_ARGS(SGraphSchemaActionButton) {}
		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InEditor, TSharedPtr<FEdGraphSchemaAction> InClickAction)
	{
		Editor = InEditor;
		Action = InClickAction;

		ChildSlot
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.TextStyle(FAppStyle::Get(), "NormalText")
			.HAlign(HAlign_Center)
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(Action->GetTooltipDescription())
			.OnClicked(this, &SGraphSchemaActionButton::AddOrViewEventBinding)
			[
				InArgs._Content.Widget
			]
		];
	}

private:
	FReply AddOrViewEventBinding()
	{
		UBlueprint* Blueprint = Editor.Pin()->GetBlueprintObj();

		UEdGraph* TargetGraph = Blueprint->GetLastEditedUberGraph();
		
		if ( TargetGraph != nullptr )
		{
			Editor.Pin()->SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);

			// Figure out a decent place to stick the node
			const FVector2f NewNodePos = TargetGraph->GetGoodPlaceForNewNode();

			Action->PerformAction(TargetGraph, nullptr, NewNodePos);
		}

		return FReply::Handled();
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Action->AddReferencedObjects(Collector);
	}

private:
	TWeakPtr<FWidgetBlueprintEditor> Editor;

	TSharedPtr<FEdGraphSchemaAction> Action;
};

bool FBlueprintWidgetCustomization::IsBindableEvent(const FProperty* Property)
{
	static const FName IsBindableEventName(TEXT("IsBindableEvent"));

	if (const FDelegateProperty* DelegateProperty = CastField<FDelegateProperty>(Property))
	{
		//TODO Remove the code to use ones that end with "Event".  Prefer metadata flag.
		if (DelegateProperty->HasMetaData(IsBindableEventName) || DelegateProperty->GetName().EndsWith(TEXT("Event")))
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SWidget> FBlueprintWidgetCustomization::MakePropertyBindingWidget(TWeakPtr<FWidgetBlueprintEditor> InEditor, UFunction* SignatureFunction, TSharedRef<IPropertyHandle> InPropertyHandle, TWeakPtr<IDetailsView> InDetailsView, bool bInGeneratePureBindings, bool bAllowDetailsPanelLegacyBinding, EPropertyBindingMode Mode)
{
	if (!IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		return SNullWidget::NullWidget;
	}

	TArray<TWeakObjectPtr<UObject>> Objects;
	{
		TArray<UObject*> RawObjects;
		InPropertyHandle->GetOuterObjects(RawObjects);

		Objects.Reserve(RawObjects.Num());
		for (UObject* RawObject : RawObjects)
		{
			Objects.Add(RawObject);
		}
	}

	UWidget* Widget = Objects.Num() ? Cast<UWidget>(Objects[0]) : nullptr;

	FString WidgetName;
	if (Widget && !Widget->IsGeneratedName())
	{
		WidgetName = TEXT("_") + Widget->GetName() + TEXT("_");
	}

	UWidgetBlueprint* WidgetBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

	const bool bIsBindableEvent = IsBindableEvent(InPropertyHandle->GetProperty());
	UUMGEditorProjectSettings* UMGEditorProjectSettings = GetMutableDefault<UUMGEditorProjectSettings>();
	const bool bCanUseNewBinding = UMGEditorProjectSettings->bUseNewBindingFromDetailsView && !bIsBindableEvent;

	TArray<TSharedPtr<FExtender>> MenuExtenders;

	// cached list of extensions for which CanExtend() returned true
	TArray<TSharedPtr<IPropertyBindingExtension>> ActiveExtensions;

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	for (const TSharedPtr<IPropertyBindingExtension>& Extension : EditorModule.GetPropertyBindingExtensibilityManager()->GetExtensions())
	{
		if (Extension->CanExtend(WidgetBlueprint, Widget, InPropertyHandle))
		{
			MenuExtenders.Add(Extension->CreateMenuExtender(WidgetBlueprint, Widget, InPropertyHandle));
			ActiveExtensions.Add(Extension);
		}
	}

	FPropertyBindingWidgetArgs Args;
	Args.MenuExtender = FExtender::Combine(MenuExtenders);
	Args.Property = InPropertyHandle->GetProperty();
	Args.BindableSignature = SignatureFunction;
	Args.BindButtonStyle = &FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

	Args.OnGenerateBindingName = FOnGenerateBindingName::CreateLambda([WidgetName]()
	{
		return WidgetName;
	});

	Args.OnGotoBinding = FOnGotoBinding::CreateLambda([InEditor, Objects](FName InPropertyName)
	{
		TSharedPtr<FWidgetBlueprintEditor> EditorPinned = InEditor.Pin();
		UWidgetBlueprint* ThisBlueprint = EditorPinned->GetWidgetBlueprintObj();

		//TODO UMG O(N) Isn't good for this, needs to be map, but map isn't serialized, need cached runtime map for fast lookups.

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			for ( const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings )
			{
				if ( Binding.ObjectName == Object->GetName() && Binding.PropertyName == InPropertyName )
				{
					if ( Binding.Kind == EBindingKind::Function )
					{
						TArray<UEdGraph*> AllGraphs;
						ThisBlueprint->GetAllGraphs(AllGraphs);

						FGuid SearchForGuid = Binding.MemberGuid;
						if ( !Binding.SourcePath.IsEmpty() )
						{
							SearchForGuid = Binding.SourcePath.Segments.Last().GetMemberGuid();
						}

						for ( UEdGraph* Graph : AllGraphs )
						{
							if ( Graph->GraphGuid == SearchForGuid )
							{
								EditorPinned->SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);
								EditorPinned->OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
							}
						}

						// Either way return
						return true;
					}
				}
			}
		}

		return false;
	});

	Args.OnCanGotoBinding = FOnCanGotoBinding::CreateLambda([InEditor, Objects](FName InPropertyName)
	{
		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			for ( const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings )
			{
				if ( Binding.ObjectName == Object->GetName() && Binding.PropertyName == InPropertyName )
				{
					if ( Binding.Kind == EBindingKind::Function )
					{
						return true;
					}
				}
			}
		}

		return false;
	});

	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda([SignatureFunction](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
	{
		if (SignatureFunction != nullptr)
		{
			if (FProperty* ReturnProperty = SignatureFunction->GetReturnProperty() )
			{
				// Find the binder that can handle the delegate return type.
				TSubclassOf<UPropertyBinding> Binder = UWidget::FindBinderClassForDestination(ReturnProperty);
				if ( Binder != nullptr )
				{
					// Ensure that the binder also can handle binding from the property we care about.
					return ( Binder->GetDefaultObject<UPropertyBinding>()->IsSupportedSource(InProperty) );
				}
			}
		}

		return false;
	});
	
	Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda([SignatureFunction](UFunction* InFunction)
	{
		auto HasFunctionBinder = [InFunction](UFunction* InBindableSignature)
		{
			if ( InFunction->NumParms == 1 && InBindableSignature->NumParms == 1 )
			{
				if ( FProperty* FunctionReturn = InFunction->GetReturnProperty() )
				{
					if ( FProperty* DelegateReturn = InBindableSignature->GetReturnProperty() )
					{
						// Find the binder that can handle the delegate return type.
						TSubclassOf<UPropertyBinding> Binder = UWidget::FindBinderClassForDestination(DelegateReturn);
						if ( Binder != nullptr )
						{
							// Ensure that the binder also can handle binding from the property we care about.
							if ( Binder->GetDefaultObject<UPropertyBinding>()->IsSupportedSource(FunctionReturn) )
							{
								return true;
							}
						}
					}
				}
			}

			return false;
		};

		if (SignatureFunction == nullptr)
		{
			return false;
		}

		// We ignore CPF_ReturnParm because all that matters for binding to script functions is that the number of out parameters match.
		return ( InFunction->IsSignatureCompatibleWith(SignatureFunction, UFunction::GetDefaultIgnoredSignatureCompatibilityFlags() | CPF_ReturnParm) ||
				HasFunctionBinder(SignatureFunction) );
	});

	Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass)
	{
		if (InClass == UUserWidget::StaticClass() ||
			InClass == AActor::StaticClass() ||
			InClass == APawn::StaticClass() ||
			InClass == UObject::StaticClass() ||
			InClass == UPrimitiveComponent::StaticClass() ||
			InClass == USceneComponent::StaticClass() ||
			InClass == UActorComponent::StaticClass() ||
			InClass == UWidgetComponent::StaticClass() ||
			InClass == UStaticMeshComponent::StaticClass() ||
			InClass == UWidgetAnimation::StaticClass() )
		{
			return false;
		}
		
		return true;
	});

	Args.OnCanBindToSubObjectClass = FOnCanBindToSubObjectClass::CreateLambda([](UClass* InClass)
	{
		// Ignore any properties that are widgets, we don't want users binding widgets to other widgets.
		return InClass->IsChildOf(UWidget::StaticClass());
	});

	Args.OnHasAnyBindings = FOnHasAnyBindings::CreateLambda([InEditor, InPropertyHandle]()
		{
			return HasPropertyBindings(InEditor, InPropertyHandle);
		});

	Args.OnCanUseCustomGenerateMenu = FOnHasAnyBindings::CreateLambda([InEditor, InPropertyHandle, Mode, bCanUseNewBinding]()
		{
			UUMGEditorProjectSettings* UMGEditorProjectSettings = GetMutableDefault<UUMGEditorProjectSettings>();
			return bCanUseNewBinding &&
					((Mode == EPropertyBindingMode::Value) || !HasPropertyBindings(InEditor, InPropertyHandle));
		});

	Args.CurrentBindingColor = MakeAttributeLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions]()
		{
			UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();
			for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
			{
				UObject* Object = ObjectPtr.Get();

				// Ignore null outer objects
				if (Object == nullptr)
				{
					continue;
				}
				if (UWidget* Widget = Cast<UWidget>(Object))
				{
					for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
					{
						TOptional<FLinearColor> Color = Extension->GetCurrentIconColor(ThisBlueprint, Widget, InPropertyHandle);
						if (Color.IsSet())
						{
							return Color.GetValue();
						}
					}
				}
			}

			return FLinearColor::White;
		});

	Args.OnAddBinding = FOnAddBinding::CreateLambda([InEditor, Objects](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
	{
		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();
		UBlueprintGeneratedClass* SkeletonClass = Cast<UBlueprintGeneratedClass>(ThisBlueprint->SkeletonGeneratedClass);

		ThisBlueprint->Modify();

		TArray<FFieldVariant> FieldChain;
		Algo::Transform(InBindingChain, FieldChain, [](const FBindingChainElement& InElement)
		{
			return InElement.Field;
		});

		UFunction* Function = FieldChain.Last().Get<UFunction>();
		FProperty* Property = FieldChain.Last().Get<FProperty>();

		check(Function != nullptr || Property != nullptr);

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			FDelegateEditorBinding Binding;
			Binding.ObjectName = Object->GetName();
			Binding.PropertyName = InPropertyName;
			Binding.SourcePath = FEditorPropertyPath(FieldChain);

			if ( Function != nullptr)
			{
				Binding.FunctionName = Function->GetFName();

				UBlueprint::GetGuidFromClassByFieldName<UFunction>(
					Function->GetOwnerClass(),
					Function->GetFName(),
					Binding.MemberGuid);

				Binding.Kind = EBindingKind::Function;
			}
			else if( Property != nullptr )
			{
				Binding.SourceProperty = Property->GetFName();

				UBlueprint::GetGuidFromClassByFieldName<FProperty>(
					SkeletonClass,
					Property->GetFName(),
					Binding.MemberGuid);

				Binding.Kind = EBindingKind::Property;
			}

			ThisBlueprint->Bindings.Remove(Binding);
			ThisBlueprint->Bindings.AddUnique(Binding);
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ThisBlueprint);	
	});

	Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions](FName InPropertyName)
	{
		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

		ThisBlueprint->Modify();

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			FDelegateEditorBinding Binding;
			Binding.ObjectName = Object->GetName();
			Binding.PropertyName = InPropertyName;

			ThisBlueprint->Bindings.Remove(Binding);

			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
				{
					Extension->ClearCurrentValue(ThisBlueprint, Widget, InPropertyHandle);
				}
			}
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ThisBlueprint);
	});

	Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions](FName InPropertyName)
	{
		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			for ( const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings )
			{
				if ( Binding.ObjectName == Object->GetName() && Binding.PropertyName == InPropertyName )
				{
					return true;
				}
			}

			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
				{
					TOptional<FName> Name = Extension->GetCurrentValue(ThisBlueprint, Widget, InPropertyHandle);
					if (Name.IsSet())
					{
						return true;
					}
				}
			}
		}

		return false;
	});

	Args.OnDrop = FOnDrop::CreateLambda([ActiveExtensions, InEditor, Objects, InPropertyHandle](const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
	{
		bool bDropHandled = false;
		UWidgetBlueprint* WidgetBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();
		UWidget* Widget = Objects.Num() ? Cast<UWidget>(Objects[0]) : nullptr;

		if (Widget && WidgetBlueprint)
		{
			for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
			{
				IPropertyBindingExtension::EDropResult Result = Extension->OnDrop(Geometry, DragDropEvent, WidgetBlueprint, Widget, InPropertyHandle);
				if (Result != IPropertyBindingExtension::EDropResult::Unhandled)
				{
					bDropHandled = true;
				}
				if (Result == IPropertyBindingExtension::EDropResult::HandledBreak)
				{
					break;
				}
			}
		}
		return bDropHandled ? FReply::Handled() : FReply::Unhandled();
	});

	Args.CurrentBindingText = MakeAttributeLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions]()
	{
		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

		//TODO UMG O(N) Isn't good for this, needs to be map, but map isn't serialized, need cached runtime map for fast lookups.

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			//TODO UMG handle multiple things selected

			for ( const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings )
			{
				if ( Binding.ObjectName == Object->GetName() && Binding.PropertyName == InPropertyHandle->GetProperty()->GetFName())
				{
					if ( !Binding.SourcePath.IsEmpty() )
					{
						return Binding.SourcePath.GetDisplayText();
					}
					else
					{
						if ( Binding.Kind == EBindingKind::Function )
						{
							if ( Binding.MemberGuid.IsValid() )
							{
								// Graph function, look up by Guid
								FName FoundName = ThisBlueprint->GetFieldNameFromClassByGuid<UFunction>(ThisBlueprint->GeneratedClass, Binding.MemberGuid);
								return FText::FromString(FName::NameToDisplayString(FoundName.ToString(), false));
							}
							else
							{
								// No GUID, native function, return function name.
								return FText::FromName(Binding.FunctionName);
							}
						}
						else // Property
						{
							if ( Binding.MemberGuid.IsValid() )
							{
								FName FoundName = ThisBlueprint->GetFieldNameFromClassByGuid<FProperty>(ThisBlueprint->GeneratedClass, Binding.MemberGuid);
								return FText::FromString(FName::NameToDisplayString(FoundName.ToString(), false));
							}
							else
							{
								// No GUID, native property, return source property.
								return FText::FromName(Binding.SourceProperty);
							}
						}
					}
				}
			}

			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
				{
					TOptional<FName> Name = Extension->GetCurrentValue(ThisBlueprint, Widget, InPropertyHandle);
					if (Name.IsSet())
					{
						return FText::FromName(Name.GetValue());
					}
				}
			}

			//TODO UMG Do something about missing functions, little exclamation points if they're missing and such.

			break;
		}

		return FText::GetEmpty();
	});

	Args.CurrentBindingImage = MakeAttributeLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions]() -> const FSlateBrush*
	{
		static FName PropertyIcon(TEXT("Kismet.Tabs.Variables"));
		static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));

		UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

		//TODO UMG O(N) Isn't good for this, needs to be map, but map isn't serialized, need cached runtime map for fast lookups.

		for (const TWeakObjectPtr<UObject>& ObjectPtr : Objects)
		{
			UObject* Object = ObjectPtr.Get();

			// Ignore null outer objects
			if ( Object == nullptr )
			{
				continue;
			}

			//TODO UMG handle multiple things selected

			for ( const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings )
			{
				if ( Binding.ObjectName == Object->GetName() && Binding.PropertyName == InPropertyHandle->GetProperty()->GetFName() )
				{
					if ( Binding.Kind == EBindingKind::Function )
					{
						return FAppStyle::Get().GetBrush(FunctionIcon);
					}
					else // Property
					{
						return FAppStyle::Get().GetBrush(PropertyIcon);
					}
				}
			}

			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
				{
					const FSlateBrush* Brush = Extension->GetCurrentIcon(ThisBlueprint, Widget, InPropertyHandle);
					if (Brush != nullptr)
					{
						return Brush;
					}
				}
			}
		}

		return nullptr;
	});

	Args.bGeneratePureBindings = bInGeneratePureBindings;
	Args.bAllowNewBindings = bAllowDetailsPanelLegacyBinding || bIsBindableEvent;
	Args.bUseLinkIconStyle = true;
	Args.bAllowFunctionBindings = !bCanUseNewBinding;
	Args.bAllowPropertyBindings = !bCanUseNewBinding;

	if (Mode == EPropertyBindingMode::Extension && !bIsBindableEvent)
	{
		Args.bLinkIconStyleCompactMode = UMGEditorProjectSettings->bUseCompactLinkButtonInExtension;
		Args.bCanDisplayLinkIcon = true;
	}
	else
	{
		Args.bLinkIconStyleCompactMode = false;
		Args.bCanDisplayLinkIcon = bIsBindableEvent;
	}

	Args.OnGenerateMenu = FOnGetContent::CreateLambda([InEditor, Objects, InPropertyHandle, ActiveExtensions, InDetailsView]() -> TSharedRef<SWidget>
	{
		UWidgetBlueprint* WidgetBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();
		UWidget* Widget = Objects.Num() ? Cast<UWidget>(Objects[0]) : nullptr;

		if (Widget && WidgetBlueprint)
		{
			for (const TSharedPtr<IPropertyBindingExtension>& Extension : ActiveExtensions)
			{
				TSharedRef<SWidget> GeneratedWidget = Extension->OnGenerateMenu(WidgetBlueprint, Widget, InPropertyHandle, InDetailsView);
				if (GeneratedWidget != SNullWidget::NullWidget)
				{
					return GeneratedWidget;
				}
			}
		}

		return SNullWidget::NullWidget;
	}
	);

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	return PropertyAccessEditor.MakePropertyBindingWidget(InEditor.Pin()->GetBlueprintObj(), Args);
}

bool FBlueprintWidgetCustomization::HasPropertyBindings(TWeakPtr<FWidgetBlueprintEditor> InEditor, const TSharedRef<IPropertyHandle>& InPropertyHandle, bool bOnlyEditableValue)
{
	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	UWidgetBlueprint* ThisBlueprint = InEditor.Pin()->GetWidgetBlueprintObj();

	// In the UI, we treat a child property of a struct/array as bound if the parent is bound
	// E.g., we'll disable the child value widgets as well.
	// So find the parent property and check if it's bound below.
	FName ParentPropertyName;
	for (TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = InPropertyHandle;
		CurrentPropertyHandle && CurrentPropertyHandle->GetProperty();
		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle())
	{
		ParentPropertyName = CurrentPropertyHandle->GetProperty()->GetFName();
	}

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");

	//TODO UMG O(N) Isn't good for this, needs to be map, but map isn't serialized, need cached runtime map for fast lookups.
	for (const UObject* Object : Objects)
	{
		// Ignore null outer objects
		if (Object == nullptr)
		{
			continue;
		}

		for (const FDelegateEditorBinding& Binding : ThisBlueprint->Bindings)
		{
			if (Binding.ObjectName == Object->GetName() && Binding.PropertyName == ParentPropertyName)
			{
				return true;
			}
		}
	}

	// check property binding extensions
	for (const UObject* Object : Objects)
	{
		const UWidget* Widget = Cast<UWidget>(Object);
		if (Widget == nullptr)
		{
			continue;
		}

		for (const TSharedPtr<IPropertyBindingExtension>& BindingExtension : EditorModule.GetPropertyBindingExtensibilityManager()->GetExtensions())
		{
			if (BindingExtension->CanExtend(ThisBlueprint, Widget, InPropertyHandle))
			{
				TOptional<FName> CurrentValue = BindingExtension->GetCurrentValue(ThisBlueprint, Widget, InPropertyHandle);
				if (CurrentValue.IsSet())
				{
					if (bOnlyEditableValue)
					{
						return BindingExtension->ShouldAllowEditingValue(ThisBlueprint, Widget, InPropertyHandle);
					}
					else
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FBlueprintWidgetCustomization::CreateEventCustomization( IDetailLayoutBuilder& DetailLayout, FDelegateProperty* Property, UWidget* Widget )
{
	TSharedRef<IPropertyHandle> DelegatePropertyHandle = DetailLayout.GetProperty(Property->GetFName(), Property->GetOwnerChecked<UClass>());
	UWidgetBlueprint* BlueprintObj = Blueprint.Get();
	if (!BlueprintObj)
	{
		return;
	}

	const bool bHasValidHandle = DelegatePropertyHandle->IsValidHandle();
	if(!bHasValidHandle)
	{
		return;
	}

	IDetailCategoryBuilder& PropertyCategory = DetailLayout.EditCategory(FObjectEditorUtils::GetCategoryFName(Property), FText::GetEmpty(), ECategoryPriority::Uncommon);

	IDetailPropertyRow& PropertyRow = PropertyCategory.AddProperty(DelegatePropertyHandle);
	PropertyRow.OverrideResetToDefault(FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateSP(this, &FBlueprintWidgetCustomization::ResetToDefault_RemoveBinding)));

	FString LabelStr = Property->GetDisplayNameText().ToString();
	LabelStr.RemoveFromEnd(TEXT("Event"));

	FText Label = FText::FromString(LabelStr);

	const bool bShowChildren = true;
	PropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		[
			SNew(SHorizontalBox)
			.ToolTipText(Property->GetToolTipText())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0,0,5,0)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("GraphEditor.Event_16x"))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
			]
		]
		.ValueContent()
		.MinDesiredWidth(200)
		.MaxDesiredWidth(250)
		[
			MakePropertyBindingWidget(Editor.Pin(), Property->SignatureFunction, DelegatePropertyHandle, DetailLayout.GetDetailsViewSharedPtr(), false, true)
		];
}

void FBlueprintWidgetCustomization::ResetToDefault_RemoveBinding(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	const FScopedTransaction Transaction(LOCTEXT("UnbindDelegate", "Remove Binding"));

	UWidgetBlueprint* BlueprintObj = Blueprint.Get();
	if (BlueprintObj)
	{
		BlueprintObj->Modify();
		
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		for (UObject* SelectedObject : OuterObjects)
		{
			FDelegateEditorBinding Binding;
			Binding.ObjectName = SelectedObject->GetName();
			Binding.PropertyName = PropertyHandle->GetProperty()->GetFName();

			BlueprintObj->Bindings.Remove(Binding);
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);
	}


}


FReply FBlueprintWidgetCustomization::HandleAddOrViewEventForVariable(const FName EventName, FName PropertyName, TWeakObjectPtr<UClass> PropertyClass)
{
	if (UWidgetBlueprint* BlueprintObj = Blueprint.Get())
	{
		FText ErrorMessage;
		FWidgetBlueprintOperationUtils::BindToEventProperty(BlueprintObj, EventName, PropertyName, PropertyClass.Get(), true, ErrorMessage);
	}

	return FReply::Handled();
}

int32 FBlueprintWidgetCustomization::HandleAddOrViewIndexForButton(const FName EventName, FName PropertyName) const
{
	if (UBlueprint* BlueprintObj = Blueprint.Get())
	{
		if (FKismetEditorUtilities::FindBoundEventForComponent(BlueprintObj, EventName, PropertyName))
		{
			return 0; // View
		}

		return 1; // Add
	}
	return 0;
}

void FBlueprintWidgetCustomization::CreateMulticastEventCustomization(IDetailLayoutBuilder& DetailLayout, FName ThisComponentName, UClass* PropertyClass, FMulticastDelegateProperty* DelegateProperty)
{
	UBlueprint* BlueprintObj = Blueprint.Get();
	if (BlueprintObj == nullptr)
	{
		return;
	}

	const FString AddString = FString(TEXT("Add "));
	const FString ViewString = FString(TEXT("View "));

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	if ( !K2Schema->CanUserKismetAccessVariable(DelegateProperty, PropertyClass, UEdGraphSchema_K2::MustBeDelegate) )
	{
		return;
	}

	FText PropertyTooltip = DelegateProperty->GetToolTipText();
	if ( PropertyTooltip.IsEmpty() )
	{
		PropertyTooltip = FText::FromString(DelegateProperty->GetName());
	}

	static FText EventCategoryText = LOCTEXT("Events", "Events");
	IDetailCategoryBuilder& EventCategory = DetailLayout.EditCategory(TEXT("Events"), EventCategoryText, ECategoryPriority::Uncommon);
	FObjectProperty* ComponentProperty = FindFProperty<FObjectProperty>(BlueprintObj->SkeletonGeneratedClass, ThisComponentName);
	if (ComponentProperty)
	{
		FName PropertyName = ComponentProperty->GetFName();
		FName EventName = DelegateProperty->GetFName();
		FText EventText = DelegateProperty->GetDisplayNameText();

		EventCategory.AddCustomRow(EventText)
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
				.ToolTipText(DelegateProperty->GetToolTipText())
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 5.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("GraphEditor.Event_16x"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(EventText)
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					SNew(SButton)
					.ContentPadding(FMargin(3.0f, 2.0f))
					.OnClicked(this, &FBlueprintWidgetCustomization::HandleAddOrViewEventForVariable, EventName, PropertyName, MakeWeakObjectPtr(PropertyClass))
					[
						SNew(SWidgetSwitcher)
						.WidgetIndex(this, &FBlueprintWidgetCustomization::HandleAddOrViewIndexForButton, EventName, PropertyName)
						+ SWidgetSwitcher::Slot()
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush("Icons.SelectInViewport"))
						]
						+ SWidgetSwitcher::Slot()
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
						]
					]
				]
			];
	}
	else if (!bCreateMulticastEventCustomizationErrorAdded)
	{
		bCreateMulticastEventCustomizationErrorAdded = true;
		EventCategory.AddCustomRow(FText::GetEmpty())
			.WholeRowContent()
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("EventAvailableButNotVariable", "To see available events, enable the Is Variable setting for this widget."))
			];
	}
}

void FBlueprintWidgetCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	static const FName LayoutCategoryKey(TEXT("Layout"));
	static const FName LocalizationCategoryKey(TEXT("Localization"));

	DetailLayout.EditCategory(LocalizationCategoryKey, FText::GetEmpty(), ECategoryPriority::Uncommon);

	TArray< TWeakObjectPtr<UObject> > OutObjects;
	DetailLayout.GetObjectsBeingCustomized(OutObjects);

	FMemMark Mark(FMemStack::Get());
	TArray<UWidget*, TMemStackAllocator<>> CustomizedWidgets;
	CustomizedWidgets.Reserve(OutObjects.Num());

	UClass* SlotBaseClasses = nullptr;
	for (const TWeakObjectPtr<UObject>& Obj : OutObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(Obj.Get()))
		{
			CustomizedWidgets.Add(Widget);
			if (Widget->Slot)
			{
				UClass* SlotClass = Widget->Slot->GetClass();
				if (!SlotBaseClasses)
				{
					SlotBaseClasses = SlotClass;
				}
				else if (SlotBaseClasses != SlotClass)
				{
					SlotBaseClasses = nullptr;
					break;
				}
			}
			else
			{
				SlotBaseClasses = nullptr;
				break;
			}
		}
	}
	
	if (SlotBaseClasses)
	{
		FText LayoutCatName = FText::Format(LOCTEXT("SlotNameFmt", "Slot ({0})"), SlotBaseClasses->GetDisplayNameText());
		DetailLayout.EditCategory(LayoutCategoryKey, LayoutCatName, ECategoryPriority::TypeSpecific);
	}
	else
	{
		DetailLayout.EditCategory(LayoutCategoryKey, FText(), ECategoryPriority::TypeSpecific);
	}

	PerformAccessibilityCustomization(DetailLayout);
	PerformBindingCustomization(DetailLayout, CustomizedWidgets);
	PerformCustomizationExtenders(DetailLayout, CustomizedWidgets);
}

void FBlueprintWidgetCustomization::PerformBindingCustomization(IDetailLayoutBuilder& DetailLayout, const TArrayView<UWidget*> Widgets)
{
	bCreateMulticastEventCustomizationErrorAdded = false;
	if ( Widgets.Num() == 1 )
	{
		UWidget* Widget = Widgets[0];
		UClass* PropertyClass = Widget->GetClass();

		for ( TFieldIterator<FProperty> PropertyIt(PropertyClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt )
		{
			FProperty* Property = *PropertyIt;

			if ( FDelegateProperty* DelegateProperty = CastField<FDelegateProperty>(*PropertyIt) )
			{
				if ( IsBindableEvent(Property))
				{
					CreateEventCustomization(DetailLayout, DelegateProperty, Widget);
				}
			}
			else if ( FMulticastDelegateProperty* MulticastDelegateProperty = CastField<FMulticastDelegateProperty>(Property) )
			{
				CreateMulticastEventCustomization(DetailLayout, Widget->GetFName(), PropertyClass, MulticastDelegateProperty);
			}
		}
	}
}

void FBlueprintWidgetCustomization::PerformAccessibilityCustomization(IDetailLayoutBuilder& DetailLayout)
{
	// We have to add these properties even though we're not customizing to preserve UI ordering
	DetailLayout.EditCategory("Accessibility").AddProperty("bOverrideAccessibleDefaults");
	DetailLayout.EditCategory("Accessibility").AddProperty("bCanChildrenBeAccessible");
	CustomizeAccessibilityProperty(DetailLayout, "AccessibleBehavior", "AccessibleText");
	CustomizeAccessibilityProperty(DetailLayout, "AccessibleSummaryBehavior", "AccessibleSummaryText");
}

void FBlueprintWidgetCustomization::CustomizeAccessibilityProperty(IDetailLayoutBuilder& DetailLayout, const FName& BehaviorPropertyName, const FName& TextPropertyName)
{
	UWidgetBlueprint* BlueprintObj = Blueprint.Get();
	if (!BlueprintObj)
	{
		return;
	}

	// Treat AccessibleBehavior as the "base" property for the row, and then add the AccessibleText binding to the end of it.
	TSharedRef<IPropertyHandle> AccessibleBehaviorPropertyHandle = DetailLayout.GetProperty(BehaviorPropertyName);
	IDetailPropertyRow& AccessibilityRow = DetailLayout.EditCategory("Accessibility").AddProperty(AccessibleBehaviorPropertyHandle);

	TSharedRef<IPropertyHandle> AccessibleTextPropertyHandle = DetailLayout.GetProperty(TextPropertyName);
	const FName DelegateName(*(TextPropertyName.ToString() + "Delegate"));
	FDelegateProperty* AccessibleTextDelegateProperty = FindFieldChecked<FDelegateProperty>(CastChecked<UClass>(AccessibleTextPropertyHandle->GetProperty()->GetOwner<UObject>()), DelegateName);
	// Make sure the old AccessibleText properties are hidden so we don't get duplicate widgets
	DetailLayout.HideProperty(AccessibleTextPropertyHandle);

	TSharedRef<SWidget> ValueWidget = AccessibleTextPropertyHandle->CreatePropertyValueWidget();
	TWeakPtr<FWidgetBlueprintEditor> ThisEditor = Editor;
	ValueWidget->SetEnabled(TAttribute<bool>::Create(
		[ThisEditor, AccessibleTextPropertyHandle]() {
			return !HasPropertyBindings(ThisEditor, AccessibleTextPropertyHandle);
		}));

	TSharedRef<SWidget> BindingWidget = MakePropertyBindingWidget(Editor, AccessibleTextDelegateProperty->SignatureFunction, AccessibleTextPropertyHandle, DetailLayout.GetDetailsViewSharedPtr(), false, BlueprintObj->ArePropertyBindingsAllowed());
	TSharedRef<SHorizontalBox> CustomTextLayout = SNew(SHorizontalBox)
	.Visibility(TAttribute<EVisibility>::Create([AccessibleBehaviorPropertyHandle]() -> EVisibility
	{
		uint8 Behavior = 0;
		AccessibleBehaviorPropertyHandle->GetValue(Behavior);
		return (ESlateAccessibleBehavior)Behavior == ESlateAccessibleBehavior::Custom ? EVisibility::Visible : EVisibility::Hidden;
	}))
	+ SHorizontalBox::Slot()
	.Padding(FMargin(4.0f, 0.0f))
	[
		ValueWidget
	]
	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		BindingWidget
	];

	TSharedPtr<SWidget> AccessibleBehaviorNameWidget, AccessibleBehaviorValueWidget;
	AccessibilityRow.GetDefaultWidgets(AccessibleBehaviorNameWidget, AccessibleBehaviorValueWidget);

	AccessibilityRow.CustomWidget()
	.NameContent()
	[
		AccessibleBehaviorNameWidget.ToSharedRef()
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			AccessibleBehaviorValueWidget.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		[
			CustomTextLayout
		]
	];
}

void FBlueprintWidgetCustomization::PerformCustomizationExtenders(IDetailLayoutBuilder& DetailLayout, const TArrayView<UWidget*> Widgets)
{
	if (IUMGEditorModule* UMGEditorModule = FModuleManager::GetModulePtr<IUMGEditorModule>("UMGEditor"))
	{
		if (TSharedPtr<FWidgetBlueprintEditor> EditorPtr = Editor.Pin())
		{
			for (TSharedRef<IBlueprintWidgetCustomizationExtender>& CustomizationExtender : UMGEditorModule->GetAllWidgetCustomizationExtenders())
			{
				CustomizationExtender->CustomizeDetails(DetailLayout, Widgets, EditorPtr.ToSharedRef());
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
