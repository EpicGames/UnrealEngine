// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UFunction;

/**
 * Provides the customization for all UWidgets.  Bindings, style disabling...etc.
 */
class FBlueprintWidgetCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TSharedRef<FWidgetBlueprintEditor> InEditor, UBlueprint* InBlueprint)
	{
		return MakeShareable(new FBlueprintWidgetCustomization(InEditor, InBlueprint));
	}

	FBlueprintWidgetCustomization(TSharedRef<FWidgetBlueprintEditor> InEditor, UBlueprint* InBlueprint)
		: Editor(InEditor)
		, Blueprint(CastChecked<UWidgetBlueprint>(InBlueprint))
	{
	}
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	/** Make a property binding widget */
	enum class EPropertyBindingMode
	{
		Value,
		Extension
	};
	static TSharedRef<SWidget> MakePropertyBindingWidget(TWeakPtr<FWidgetBlueprintEditor> InEditor, UFunction* SignatureFunction, TSharedRef<IPropertyHandle> InDelegatePropertyHandle, TWeakPtr<IDetailsView> InDetailsView, bool bInGeneratePureBindings, bool bAllowDetailsPanelLegacyBinding, EPropertyBindingMode Mode = EPropertyBindingMode::Extension);

	/** 
	 * Whether the property (or its parent property, if this is an array/struct element) currently has bindings.
	 * When bOnlyCompilableBindingsFilter is set to true, only bindings which have there Compile flag enable will be considered.
	 */
	static bool HasPropertyBindings(TWeakPtr<FWidgetBlueprintEditor> InEditor, const TSharedRef<IPropertyHandle>& InPropertyHandle, bool bOnlyEditableValue = false);

	static bool IsBindableEvent(const FProperty* Property);

private:
	void PerformBindingCustomization(IDetailLayoutBuilder& DetailLayout, const TArrayView<UWidget*> Widgets);

	void CreateEventCustomization( IDetailLayoutBuilder& DetailLayout, FDelegateProperty* Property, UWidget* Widget );

	void CreateMulticastEventCustomization(IDetailLayoutBuilder& DetailLayout, FName ThisComponentName, UClass* PropertyClass, FMulticastDelegateProperty* Property);

	void ResetToDefault_RemoveBinding(TSharedPtr<IPropertyHandle> PropertyHandle);

	FReply HandleAddOrViewEventForVariable(const FName EventName, FName PropertyName, TWeakObjectPtr<UClass> PropertyClass);

	int32 HandleAddOrViewIndexForButton(const FName EventName, FName PropertyName) const;

	void PerformAccessibilityCustomization(IDetailLayoutBuilder& DetailLayout);
	void CustomizeAccessibilityProperty(IDetailLayoutBuilder& DetailLayout, const FName& BehaviorPropertyName, const FName& TextPropertyName);
	void PerformCustomizationExtenders(IDetailLayoutBuilder& DetailLayout, const TArrayView<UWidget*> Widgets);

private:
	TWeakPtr<FWidgetBlueprintEditor> Editor;
	TWeakObjectPtr<UWidgetBlueprint> Blueprint;
	bool bCreateMulticastEventCustomizationErrorAdded = false;
};
