// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/VerticalAlignmentCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSegmentedControl.h"

class IDetailChildrenBuilder;


#define LOCTEXT_NAMESPACE "UMG"

void FVerticalAlignmentCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	const FMargin OuterPadding(2);
	const FMargin ContentPadding(2);

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0)
	[
		SNew(SSegmentedControl<EVerticalAlignment>)
		.Value(this, &FVerticalAlignmentCustomization::GetCurrentAlignment, PropertyHandle)
		.OnValueChanged(this, &FVerticalAlignmentCustomization::OnCurrentAlignmentChanged, PropertyHandle)
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Top)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Top"))
			.ToolTip(LOCTEXT("VAlignTop", "Top Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Center)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Center"))
			.ToolTip(LOCTEXT("VAlignCenter", "Center Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Bottom)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Bottom"))
			.ToolTip(LOCTEXT("VAlignBottom", "Bottom Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Fill)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Fill"))
			.ToolTip(LOCTEXT("VAlignFill", "Fill Vertically"))
	];
}

void FVerticalAlignmentCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

EVerticalAlignment FVerticalAlignmentCustomization::GetCurrentAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return (EVerticalAlignment)Value;
	}

	return VAlign_Fill;
}

void FVerticalAlignmentCustomization::OnCurrentAlignmentChanged(EVerticalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)NewAlignment);
}

#undef LOCTEXT_NAMESPACE
