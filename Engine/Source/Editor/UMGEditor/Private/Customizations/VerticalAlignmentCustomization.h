// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;

enum class ECheckBoxState : uint8;

class FVerticalAlignmentCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FVerticalAlignmentCustomization());
	}

	FVerticalAlignmentCustomization()
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	EVerticalAlignment GetCurrentAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const;

	void OnCurrentAlignmentChanged(EVerticalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle);
};
