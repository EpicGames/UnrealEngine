// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/WidgetChildTypeCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Hierarchy/SReadOnlyHierarchyView.h"

#include "Blueprint/WidgetNavigation.h"
#include "Blueprint/WidgetChild.h"
#include "Blueprint/WidgetTree.h"

#include "WidgetBlueprint.h"

#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "UMG"

// FWidgetChildTypeCustomization
////////////////////////////////////////////////////////////////////////////////

void FWidgetChildTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandlePtr = PropertyHandle;

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(FDetailWidgetRow::DefaultValueMaxWidth * 2)
	[
		SAssignNew(WidgetListComboButton, SComboButton)
		.ForegroundColor(FSlateColor::UseForeground())
		.OnGetMenuContent(this, &FWidgetChildTypeCustomization::GetPopupContent)
		.ContentPadding(2.0f)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FWidgetChildTypeCustomization::GetCurrentValueText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

void FWidgetChildTypeCustomization::SetWidgetChild(UUserWidget* OwnerUserWidget, FName WidgetChildName)
{
	if (ensure(OwnerUserWidget))
	{
		// We Get the value from the Preview and add it to the CDO.
		if (TSharedPtr<IPropertyHandle> PropertyHandlePinnedPtr = PropertyHandlePtr.Pin())
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandlePinnedPtr->GetProperty()))
			{
				TArray<void*> RawData;
				PropertyHandlePinnedPtr->AccessRawData(RawData);
				FWidgetChild* PreviousWidgetChild = reinterpret_cast<FWidgetChild*>(RawData[0]);

				// When setting a WidgetChild we know it will affect the preview and the CDO. Mark them transactional.
				{
					OwnerUserWidget->SetFlags(RF_Transactional);
					OwnerUserWidget->Modify();

					if (UWidgetBlueprintGeneratedClass* BGClass = OwnerUserWidget->GetWidgetTreeOwningClass())
					{
						if (UUserWidget* UserWidgetCDO = BGClass->GetDefaultObject<UUserWidget>())
						{
							UserWidgetCDO->SetFlags(RF_Transactional);
							UserWidgetCDO->Modify();
						}
					}
				}

				FString TextValue;
				FWidgetChild WidgetChild(OwnerUserWidget, WidgetChildName);
				StructProperty->Struct->ExportText(TextValue, &WidgetChild, PreviousWidgetChild, OwnerUserWidget, EPropertyPortFlags::PPF_None, nullptr);
				ensure(PropertyHandlePinnedPtr->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
			}
		}
	}
}

void FWidgetChildTypeCustomization::OnWidgetSelectionChanged(FName SelectedName, ESelectInfo::Type SelectionType)
{
	if (TSharedPtr<IPropertyHandle> PropertyHandlePinnedPtr = PropertyHandlePtr.Pin())
	{
		TArray<UObject*> OuterObjects;
		PropertyHandlePinnedPtr->GetOuterObjects(OuterObjects);

		for (UObject* OuterObject : OuterObjects)
		{
			if (UUserWidget* UserWidget = Cast<UUserWidget>(OuterObject))
			{
				const FScopedTransaction Transaction(
					FText::Format(
						LOCTEXT("SetWidgetChildProperty", "Set {0}"),
						PropertyHandlePinnedPtr->GetPropertyDisplayName()));
				if (UWidgetBlueprint* WidgetBlueprint = Editor.Pin()->GetWidgetBlueprintObj())
				{
					if (SelectedName == WidgetBlueprint->GetFName())
					{
						SetWidgetChild(UserWidget, FName());
						break;
					}
				}
				SetWidgetChild(UserWidget, SelectedName);
				break;
			}
		}
	}
	WidgetListComboButton->SetIsOpen(false);
}

TSharedRef<SWidget> FWidgetChildTypeCustomization::GetPopupContent()
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor = Editor.Pin();
	UWidgetBlueprint* WidgetBlueprint = BlueprintEditor->GetWidgetBlueprintObj();

	const float MinPopupWidth = 250.0f;
	const float MinPopupHeight = 200.0f;

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(4.0f)
		[
			SNew(SBox)
			.MinDesiredWidth(MinPopupWidth)
			.MinDesiredHeight(MinPopupHeight)
			[
				SNew(SReadOnlyHierarchyView, WidgetBlueprint)
					.OnSelectionChanged(this, &FWidgetChildTypeCustomization::OnWidgetSelectionChanged)
					.ShowSearch(true)
					.RootSelectionMode(ERootSelectionMode::Self)
					.InheritedNamedSlotSelectionMode(EInheritedNamedSlotSelectionMode::Disabled)
			]
		];
}

void FWidgetChildTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

UWidget* FWidgetChildTypeCustomization::GetCurrentValue() const
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin())
	{
		UObject* Object;
		switch (PropertyHandle->GetValue(Object))
		{
		case FPropertyAccess::Success:
			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				return Widget;
			}
			break;
		}
	}

	return nullptr;
}

FText FWidgetChildTypeCustomization::GetCurrentValueText() const
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin())
	{
		UObject* Object;
		switch (PropertyHandle->GetValue(Object))
		{
		case FPropertyAccess::MultipleValues:
			return LOCTEXT("MultipleValues", "Multiple Values");
		case FPropertyAccess::Success:
			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				return Widget->GetLabelText();
			}
			return FText::GetEmpty();
		case FPropertyAccess::Fail:
			break;

		}
		
		if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
		{
			TArray<void*> RawData;
			PropertyHandle->AccessRawData(RawData);
			if (FWidgetChild* ChildWidget = reinterpret_cast<FWidgetChild*>(RawData[0]))
			{
				FName WidgetName = ChildWidget->GetFName();
				return WidgetName.IsNone()? LOCTEXT("SelfText", "Self") : FText::FromName(WidgetName);
			}
		}		
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE

