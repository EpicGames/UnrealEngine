// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "WidgetBlueprintEditor.h"
#include "IPropertyTypeCustomization.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class IDetailChildrenBuilder;
class IPropertyHandle;
class UWidget;
struct FWidgetChild;
class SComboButton;
class SSearchBox;

class FWidgetChildTypeCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(TSharedRef<FWidgetBlueprintEditor> InEditor)
	{
		return MakeShareable(new FWidgetChildTypeCustomization(InEditor));
	}

	FWidgetChildTypeCustomization(TSharedRef<FWidgetBlueprintEditor> InEditor)
		: Editor(InEditor)
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	void SetWidgetChild(UUserWidget* OwnerUserWidget, FName WidgetChildName);

	void OnWidgetSelectionChanged(FName SelectedName, ESelectInfo::Type SelectionType);

	TSharedRef<SWidget> GetPopupContent();
	void HandleMenuOpen();

	UWidget* GetCurrentValue() const;
	FText GetCurrentValueText() const;

private:

	TWeakPtr<FWidgetBlueprintEditor> Editor;
	TSharedPtr<SComboButton> WidgetListComboButton;
	TSharedPtr<SListView<TWeakObjectPtr<UWidget>>> WidgetListView;
	TSharedPtr<SSearchBox> SearchBox;

	TWeakPtr<IPropertyHandle> PropertyHandlePtr;

	TArray<TWeakObjectPtr<UWidget>> ReferenceableWidgets;
};
