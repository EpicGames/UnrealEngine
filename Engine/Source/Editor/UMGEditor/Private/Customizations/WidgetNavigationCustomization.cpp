// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/WidgetNavigationCustomization.h"
#include "HAL/IConsoleManager.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"

#include "Blueprint/WidgetNavigation.h"
#include "Input/NavigationMethod.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "SInstancedStructPicker.h"
#include "StructViewerModule.h"
#include "ISinglePropertyView.h"
#include "IStructureDataProvider.h"
#include "IStructureDetailsView.h"
#include "Styling/SlateIconFinder.h"

#include "ClassViewerFilter.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Details/SFunctionSelector.h"
#include "Framework/Application/SlateApplication.h"
#include "Blueprint/WidgetTree.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

#define LOCTEXT_NAMESPACE "FWidgetNavigationCustomization"

// FWidgetNavigationCustomization
////////////////////////////////////////////////////////////////////////////////

void FWidgetNavigationCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FWidgetNavigationCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	TWeakPtr<IPropertyHandle> PropertyHandlePtr(PropertyHandle);

	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Left, LOCTEXT("LeftNavigation", "Left"));
	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Right, LOCTEXT("RightNavigation", "Right"));
	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Up, LOCTEXT("UpNavigation", "Up"));
	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Down, LOCTEXT("DownNavigation", "Down"));
	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Next, LOCTEXT("NextNavigation", "Next"));
	MakeNavRow(PropertyHandlePtr, ChildBuilder, EUINavigation::Previous, LOCTEXT("PreviousNavigation", "Previous"));

	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.Navigation.Experimental"));
	if (CVar && CVar->GetBool())
	{
		MakeNavigationMethodStructPicker(PropertyHandlePtr, ChildBuilder);

		if (DoesWidgetSupportRoutingPolicy(PropertyHandlePtr))
		{
			ChildBuilder.AddCustomRow(LOCTEXT("RoutingPolicy", "Routing Policy"))
				.NameContent()
				[
					SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("RoutingPolicy", "Routing Policy"))
				]
				.ValueContent()
				.MaxDesiredWidth(300)
				[
					SNew(SComboButton)
						.HAlign(HAlign_Center)
						.ButtonContent()
						[
							SNew(STextBlock)
								.Text(this, &FWidgetNavigationCustomization::GetRoutingPolicyText, PropertyHandlePtr)
								.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						.ContentPadding(FMargin(2.0f, 1.0f))
						.MenuContent()
						[
							MakeRoutingPolicyMenu(PropertyHandle)
						]
				];
		}
	}
}

EUINavigationRule FWidgetNavigationCustomization::GetNavigationRule(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	EUINavigationRule Rule = EUINavigationRule::Invalid;
	for ( UObject* OuterObject : OuterObjects )
	{
		if ( UWidget* Widget = Cast<UWidget>(OuterObject) )
		{
			EUINavigationRule CurRule = EUINavigationRule::Escape;
			UWidgetNavigation* WidgetNavigation = Widget->Navigation;
			if ( Widget->Navigation != nullptr )
			{
				CurRule = WidgetNavigation->GetNavigationRule(Nav);
			}

			if ( Rule != EUINavigationRule::Invalid && CurRule != Rule )
			{
				return EUINavigationRule::Invalid;
			}
			Rule = CurRule;
		}
	}

	return Rule;
}

FText FWidgetNavigationCustomization::GetNavigationText(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	EUINavigationRule Rule = GetNavigationRule(PropertyHandle, Nav);

	switch (Rule)
	{
	case EUINavigationRule::Escape:
		return LOCTEXT("NavigationEscape", "Escape");
	case EUINavigationRule::Stop:
		return LOCTEXT("NavigationStop", "Stop");
	case EUINavigationRule::Wrap:
		return LOCTEXT("NavigationWrap", "Wrap");
	case EUINavigationRule::Explicit:
		return LOCTEXT("NavigationExplicit", "Explicit");
	case EUINavigationRule::Invalid:
		return LOCTEXT("NavigationMultipleValues", "Multiple Values");
	case EUINavigationRule::Custom:
		return LOCTEXT("NavigationCustom", "Custom");
	case EUINavigationRule::CustomBoundary:
		return LOCTEXT("NavigationCustomBoundary", "Custom Boundary");
	}

	return FText::GetEmpty();
}


bool FWidgetNavigationCustomization::DoesWidgetSupportRoutingPolicy(TWeakPtr<IPropertyHandle> PropertyHandle) const
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 0)
	{
		return false;
	}

	// Only support routing for widgets that layout children
	for (UObject* OuterObject : OuterObjects)
	{
		UWidget* Widget = Cast<UWidget>(OuterObject);
		if (Widget == nullptr || !Widget->IsA<UPanelWidget>())
		{
			return false;
		}
	}

	return true;
}

TOptional<FName> FWidgetNavigationCustomization::GetUniformNavigationTargetOrFunction(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	bool bFirst = true;
	FName Rule = NAME_None;
	for ( UObject* OuterObject : OuterObjects )
	{
		if ( UWidget* Widget = Cast<UWidget>(OuterObject) )
		{
			FName CurRule = NAME_None;
			UWidgetNavigation* WidgetNavigation = Widget->Navigation;
			if ( Widget->Navigation != nullptr )
			{
				CurRule = WidgetNavigation->GetNavigationData(Nav).WidgetToFocus;
				if ( bFirst )
				{
					Rule = CurRule;
					bFirst = false;
				}
			}

			if ( CurRule != Rule )
			{
				return TOptional<FName>();
			}

			Rule = CurRule;
		}
	}

	return Rule;
}

void FWidgetNavigationCustomization::OnWidgetSelectedForExplicitNavigation(FName ExplictWidgetOrFunction, TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav)
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	const FScopedTransaction Transaction(LOCTEXT("InitializeNavigation", "Edit Widget Navigation"));

	for ( UObject* OuterObject : OuterObjects )
	{
		if ( UWidget* Widget = Cast<UWidget>(OuterObject) )
		{
			const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = Editor.Pin();
			if (Widget == WidgetBlueprintEditor->GetPreview())
			{
				SetNav(Widget, Nav, TOptional<EUINavigationRule>(), ExplictWidgetOrFunction);

				UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
				if (WidgetBlueprint && WidgetBlueprint->GeneratedClass)
				{
					SetNav(WidgetBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>(), Nav, TOptional<EUINavigationRule>(), ExplictWidgetOrFunction);
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
				}
			}
			else
			{
				FWidgetReference WidgetReference = WidgetBlueprintEditor->GetReferenceFromPreview(Widget);

				SetNav(WidgetReference.GetPreview(), Nav, TOptional<EUINavigationRule>(), ExplictWidgetOrFunction);
				SetNav(WidgetReference.GetTemplate(), Nav, TOptional<EUINavigationRule>(), ExplictWidgetOrFunction);
			}
		}
	}
}

EVisibility FWidgetNavigationCustomization::GetExplictWidgetFieldVisibility(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	EUINavigationRule Rule = GetNavigationRule(PropertyHandle, Nav);
	switch (Rule)
	{
		case EUINavigationRule::Explicit:
			return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FWidgetNavigationCustomization::GetCustomWidgetFieldVisibility(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	EUINavigationRule Rule = GetNavigationRule(PropertyHandle, Nav);
	switch (Rule)
	{
	case EUINavigationRule::Custom:
	case EUINavigationRule::CustomBoundary:
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void FWidgetNavigationCustomization::MakeNavRow(TWeakPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, EUINavigation Nav, FText NavName)
{
	static UFunction* CustomWidgetNavSignature = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/UMG")), TEXT("CustomWidgetNavigationDelegate__DelegateSignature"));

	ChildBuilder.AddCustomRow(NavName)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(NavName)
		]
		.ValueContent()
		.MaxDesiredWidth(300)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.HAlign(HAlign_Center)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FWidgetNavigationCustomization::GetNavigationText, PropertyHandle, Nav)
				]
				.ContentPadding(FMargin(2.0f, 1.0f))
				.MenuContent()
				[
					MakeNavMenu(PropertyHandle, Nav)
				]
			]

			// Explicit Navigation Widget
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SComboButton)
				.Visibility(this, &FWidgetNavigationCustomization::GetExplictWidgetFieldVisibility, PropertyHandle, Nav)
				.OnGetMenuContent(this, &FWidgetNavigationCustomization::OnGenerateWidgetList, PropertyHandle, Nav)
				.ContentPadding(1.0f)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FWidgetNavigationCustomization::GetExplictWidget, PropertyHandle, Nav)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]

			// Custom Navigation Widget
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SFunctionSelector, Editor.Pin().ToSharedRef(), CustomWidgetNavSignature)
				.OnSelectedFunction(this, &FWidgetNavigationCustomization::HandleSelectedCustomNavigationFunction, PropertyHandle, Nav)
				.OnResetFunction(this, &FWidgetNavigationCustomization::HandleResetCustomNavigationFunction, PropertyHandle, Nav)
				.CurrentFunction(this, &FWidgetNavigationCustomization::GetUniformNavigationTargetOrFunction, PropertyHandle, Nav)
				.Visibility(this, &FWidgetNavigationCustomization::GetCustomWidgetFieldVisibility, PropertyHandle, Nav)
			]
		];
}

void FWidgetNavigationCustomization::HandleSelectedCustomNavigationFunction(FName SelectedFunction, TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav)
{
	OnWidgetSelectedForExplicitNavigation(SelectedFunction, PropertyHandle, Nav);
}

void FWidgetNavigationCustomization::HandleResetCustomNavigationFunction(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav)
{
	OnWidgetSelectedForExplicitNavigation(NAME_None, PropertyHandle, Nav);
}

TSharedRef<SWidget> FWidgetNavigationCustomization::OnGenerateWidgetList(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav)
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	TArray<UWidget*> Widgets;
	
	UWidgetTree* WidgetTree = Editor.Pin()->GetWidgetBlueprintObj()->WidgetTree;
	WidgetTree->GetAllWidgets(Widgets);

	Widgets.Sort([](const UWidget& LHS, const UWidget& RHS) {
		return LHS.GetName() > RHS.GetName();
	});

	{
		MenuBuilder.BeginSection("Actions");
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ResetFunction", "Reset"),
				LOCTEXT("ResetFunctionTooltip", "Reset this navigation option and clear it out."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cross"),
				FUIAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleResetCustomNavigationFunction, PropertyHandle, Nav))
			);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("Widgets", LOCTEXT("Widgets", "Widgets"));
	{
		for (UWidget* Widget : Widgets)
		{
			if (!Widget->IsGeneratedName())
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(Widget->GetDisplayLabel()),
					FText::GetEmpty(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::OnWidgetSelectedForExplicitNavigation, Widget->GetFName(), PropertyHandle, Nav))
				);
			}
		}
	}
	MenuBuilder.EndSection();

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	return
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.MaxHeight(DisplayMetrics.PrimaryDisplayHeight * 0.5)
		[
			MenuBuilder.MakeWidget()
		];
}

TSharedRef<class SWidget> FWidgetNavigationCustomization::MakeNavMenu(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav)
{
	// create build configurations menu
	FMenuBuilder MenuBuilder(true, NULL);
	{
		FUIAction EscapeAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::Escape));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleEscape", "Escape"), LOCTEXT("NavigationRuleEscapeHint", "Navigation is allowed to escape the bounds of this widget."), FSlateIcon(), EscapeAction);

		FUIAction StopAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::Stop));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleStop", "Stop"), LOCTEXT("NavigationRuleStopHint", "Navigation stops at the bounds of this widget."), FSlateIcon(), StopAction);

		FUIAction WrapAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::Wrap));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleWrap", "Wrap"), LOCTEXT("NavigationRuleWrapHint", "Navigation will wrap to the opposite bound of this object."), FSlateIcon(), WrapAction);

		FUIAction ExplicitAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::Explicit));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleExplicit", "Explicit"), LOCTEXT("NavigationRuleExplicitHint", "Navigation will go to a specified widget."), FSlateIcon(), ExplicitAction);

		FUIAction CustomAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::Custom));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleCustom", "Custom"), LOCTEXT("NavigationRuleCustomHint", "Custom function can determine what widget is navigated to. (Applied when the itself or any children are navigated from)"), FSlateIcon(), CustomAction);

		FUIAction CustomBoundaryAction(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleNavMenuEntryClicked, PropertyHandle, Nav, EUINavigationRule::CustomBoundary));
		MenuBuilder.AddMenuEntry(LOCTEXT("NavigationRuleCustomBoundary", "CustomBoundary"), LOCTEXT("NavigationRuleCustomBoundaryHint", "Custom function can determine what widget is navigated to. (Applied when the boundary is hit)"), FSlateIcon(), CustomBoundaryAction);
	}

	return MenuBuilder.MakeWidget();
}

// Callback for clicking a menu entry for a navigations rule.
void FWidgetNavigationCustomization::HandleNavMenuEntryClicked(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav, EUINavigationRule Rule)
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	const FScopedTransaction Transaction(LOCTEXT("InitializeNavigation", "Edit Widget Navigation"));

	for (UObject* OuterObject : OuterObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(OuterObject))
		{
			const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = Editor.Pin();
			if (Widget == WidgetBlueprintEditor->GetPreview())
			{
				SetNav(Widget, Nav, Rule, FName(NAME_None));

				UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
				if (WidgetBlueprint && WidgetBlueprint->GeneratedClass)
				{
					SetNav(WidgetBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>(), Nav, Rule, FName(NAME_None));
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
				}
			}
			else
			{
				FWidgetReference WidgetReference = WidgetBlueprintEditor->GetReferenceFromPreview(Widget);

				SetNav(WidgetReference.GetPreview(), Nav, Rule, FName(NAME_None));
				SetNav(WidgetReference.GetTemplate(), Nav, Rule, FName(NAME_None));
			}
		}
	}
}

void FWidgetNavigationCustomization::SetNav(UWidget* Widget, EUINavigation Nav, TOptional<EUINavigationRule> Rule, TOptional<FName> WidgetToFocus)
{
	if (!Widget)
	{
		return;
	}

	Widget->Modify();


	UWidgetNavigation* WidgetNavigation = Widget->Navigation;
	if (!Widget->Navigation)
	{
		WidgetNavigation = NewObject<UWidgetNavigation>(Widget);
		WidgetNavigation->SetFlags(RF_Transactional);
		if (Widget->IsTemplate())
		{
			EObjectFlags TemplateFlags = Widget->GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;
			WidgetNavigation->SetFlags(TemplateFlags);
		}
	}


	FWidgetNavigationData* DirectionNavigation = nullptr;

	switch ( Nav )
	{
	case EUINavigation::Left:
		DirectionNavigation = &WidgetNavigation->Left;
		break;
	case EUINavigation::Right:
		DirectionNavigation = &WidgetNavigation->Right;
		break;
	case EUINavigation::Up:
		DirectionNavigation = &WidgetNavigation->Up;
		break;
	case EUINavigation::Down:
		DirectionNavigation = &WidgetNavigation->Down;
		break;
	case EUINavigation::Next:
		DirectionNavigation = &WidgetNavigation->Next;
		break;
	case EUINavigation::Previous:
		DirectionNavigation = &WidgetNavigation->Previous;
		break;
	default:
		// Should not be possible.
		check(false);
		return;
	}

	if ( Rule.IsSet() )
	{
		DirectionNavigation->Rule = Rule.GetValue();
	}

	if ( WidgetToFocus.IsSet() )
	{
		DirectionNavigation->WidgetToFocus = WidgetToFocus.GetValue();
	}

	if ( WidgetNavigation->IsDefaultNavigation() )
	{
		// If the navigation rules are all set to the defaults, remove the navigation
		// information from the widget.
		Widget->Navigation = nullptr;
	}
	else
	{
		Widget->Navigation = WidgetNavigation;
	}
}

EWidgetNavigationRoutingPolicy FWidgetNavigationCustomization::GetRoutingPolicy(TWeakPtr<IPropertyHandle> PropertyHandle) const
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	EWidgetNavigationRoutingPolicy Policy = EWidgetNavigationRoutingPolicy::Default;
	for ( UObject* OuterObject : OuterObjects )
	{
		if ( UWidget* Widget = Cast<UWidget>(OuterObject) )
		{
			EWidgetNavigationRoutingPolicy CurrentPolicy = EWidgetNavigationRoutingPolicy::Default;
			UWidgetNavigation* WidgetNavigation = Widget->Navigation;
			if ( Widget->Navigation != nullptr )
			{
				CurrentPolicy = WidgetNavigation->RoutingPolicy;
			}

			if ( Policy != EWidgetNavigationRoutingPolicy::Default && CurrentPolicy != Policy )
			{
				return EWidgetNavigationRoutingPolicy::Default;
			}
			Policy = CurrentPolicy;
		}
	}

	return Policy;
}

FText FWidgetNavigationCustomization::GetRoutingPolicyText(TWeakPtr<IPropertyHandle> PropertyHandle) const
{
	EWidgetNavigationRoutingPolicy Rule = GetRoutingPolicy(PropertyHandle);
	if (UEnum* EnumPtr = StaticEnum<EWidgetNavigationRoutingPolicy>())
	{
		return EnumPtr->GetDisplayNameTextByValue(StaticCast<int32>(Rule));
	}
	return FText::GetEmpty();
}

FText FWidgetNavigationCustomization::GetExplictWidget(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const
{
	TOptional<FName> OptionalName = GetUniformNavigationTargetOrFunction(PropertyHandle, Nav);
	if (!OptionalName.IsSet())
	{
		return LOCTEXT("NavigationMultipleValues", "Multiple Values");
	}

	return FText::FromName(OptionalName.GetValue());
}

TSharedRef<class SWidget> FWidgetNavigationCustomization::MakeRoutingPolicyMenu(TWeakPtr<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, NULL);

	if (UEnum* EnumPtr = StaticEnum<EWidgetNavigationRoutingPolicy>())
	{
		for (int32 Index = 0; Index < EnumPtr->NumEnums(); ++Index)
		{			
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), Index))
			{
				const EWidgetNavigationRoutingPolicy Value = StaticCast<EWidgetNavigationRoutingPolicy>(EnumPtr->GetValueByIndex(Index));

				const FText DisplayName = EnumPtr->GetDisplayNameTextByIndex(Index);
				const FText Tooltip = EnumPtr->GetToolTipTextByIndex(Index);

				FUIAction Action(FExecuteAction::CreateSP(this, &FWidgetNavigationCustomization::HandleRoutingPolicyMenuEntryClicked, PropertyHandle, Value));
				MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(), Action);
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void FWidgetNavigationCustomization::HandleRoutingPolicyMenuEntryClicked(TWeakPtr<IPropertyHandle> PropertyHandle, EWidgetNavigationRoutingPolicy Policy)
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	const FScopedTransaction Transaction(LOCTEXT("InitializeNavigation", "Edit Widget Navigation"));

	for (UObject* OuterObject : OuterObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(OuterObject))
		{
			const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = Editor.Pin();
			if (Widget == WidgetBlueprintEditor->GetPreview())
			{
				SetRoutingPolicy(Widget, Policy);

				UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
				if (WidgetBlueprint && WidgetBlueprint->GeneratedClass)
				{
					SetRoutingPolicy(WidgetBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>(), Policy);
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
				}
			}
			else
			{
				FWidgetReference WidgetReference = WidgetBlueprintEditor->GetReferenceFromPreview(Widget);

				SetRoutingPolicy(WidgetReference.GetPreview(), Policy);
				SetRoutingPolicy(WidgetReference.GetTemplate(), Policy);
			}
		}
	}
}

void FWidgetNavigationCustomization::SetRoutingPolicy(UWidget* Widget, EWidgetNavigationRoutingPolicy Policy)
{
	if (!Widget)
	{
		return;
	}

	Widget->Modify();

	UWidgetNavigation* WidgetNavigation = Widget->Navigation;
	if (!Widget->Navigation)
	{
		WidgetNavigation = NewObject<UWidgetNavigation>(Widget);
		WidgetNavigation->SetFlags(RF_Transactional);
		if (Widget->IsTemplate())
		{
			EObjectFlags TemplateFlags = Widget->GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;
			WidgetNavigation->SetFlags(TemplateFlags);
		}
	}

	WidgetNavigation->RoutingPolicy = Policy;

	if ( WidgetNavigation->IsDefaultNavigation() )
	{
		// If the navigation rules are all set to the defaults, remove the navigation
		// information from the widget.
		Widget->Navigation = nullptr;
	}
	else
	{
		Widget->Navigation = WidgetNavigation;
	}
}

void FWidgetNavigationCustomization::OnNavigationMethodPrePropertyChange(TWeakPtr<IPropertyHandle> PropertyHandle)
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePtr = PropertyHandle.Pin();
	PropertyHandlePtr->GetOuterObjects(OuterObjects);

	const UScriptStruct* Struct = nullptr;
	for (UObject* OuterObject : OuterObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(OuterObject))
		{
			Widget->Modify();
		}
	}
}

void FWidgetNavigationCustomization::MakeNavigationMethodStructPicker(TWeakPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder)
{
	TSharedPtr<IPropertyHandle> PropertyHandlePinned = PropertyHandle.Pin();
	TSharedPtr<IPropertyHandle> NavigationHandle = PropertyHandlePinned->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWidgetNavigation, NavigationMethod));

	if (NavigationHandle)
	{
		NavigationHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FWidgetNavigationCustomization::OnNavigationMethodPrePropertyChange, PropertyHandle));
		ChildBuilder.AddProperty(NavigationHandle.ToSharedRef());
		return;
	}

	// Widget Navigation is instanced. So child NavigationMethod property is not available. Create a mock drop-down that will reload with valid property handle on struct type selection

	TSharedRef<FInstancedStructFilter> StructFilter = MakeShared<FInstancedStructFilter>();
	StructFilter->BaseStruct = FNavigationMethod::StaticStruct();
	StructFilter->bAllowUserDefinedStructs = false; // Only allow user defined structs when BaseStruct is not set.
	StructFilter->bAllowBaseStruct = false;

	FStructViewerInitializationOptions Options;
	Options.bShowNoneOption = true;
	Options.StructFilter = StructFilter;
	Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
	Options.DisplayMode = EStructViewerDisplayMode::ListView;
	Options.bAllowViewOptions = true;

	const FOnStructPicked OnPicked(FOnStructPicked::CreateSPLambda(this, [PropertyHandle, this](const UScriptStruct* SelectedStruct) {
		HandleNavigationMethodStructPicked(PropertyHandle, SelectedStruct);
	}));
	
	ChildBuilder.AddCustomRow(LOCTEXT("NavigationMethod", "Navigation Method"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("NavigationMethod", "Navigation Method"))
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.ContentPadding(0)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SImage)
						.Image(FSlateIconFinder::FindIconBrushForClass(FNavigationMethod::StaticStruct(), "ClassIcon.Object"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("NavigationMethodNoneText", "None"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			.MenuContent()
			[
				SNew(SBox)
				.WidthOverride(280)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.MaxHeight(500)
					[
						FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, OnPicked)
					]
				]
			]
		];
}

void FWidgetNavigationCustomization::HandleNavigationMethodStructPicked(TWeakPtr<IPropertyHandle> PropertyHandle, const UScriptStruct* NavigationMethodStruct)
{
	TArray<UObject*> OuterObjects;
	TSharedPtr<IPropertyHandle> PropertyHandlePinned = PropertyHandle.Pin();
	PropertyHandlePinned->GetOuterObjects(OuterObjects);

	const FScopedTransaction Transaction(LOCTEXT("InitializeNavigation", "Edit Widget Navigation"));

	for (UObject* OuterObject : OuterObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(OuterObject))
		{
			const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = Editor.Pin();
			if (Widget == WidgetBlueprintEditor->GetPreview())
			{
				SetNavigationMethodStruct(Widget, NavigationMethodStruct);

				UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
				if (WidgetBlueprint && WidgetBlueprint->GeneratedClass)
				{
					SetNavigationMethodStruct(WidgetBlueprint->GeneratedClass->GetDefaultObject<UUserWidget>(), NavigationMethodStruct);
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
				}
			}
			else
			{
				FWidgetReference WidgetReference = WidgetBlueprintEditor->GetReferenceFromPreview(Widget);

				SetNavigationMethodStruct(WidgetReference.GetPreview(), NavigationMethodStruct);
				SetNavigationMethodStruct(WidgetReference.GetTemplate(), NavigationMethodStruct);
			}
		}
	}
}

void FWidgetNavigationCustomization::SetNavigationMethodStruct(UWidget* Widget, const UScriptStruct* NavigationMethodStruct)
{
	if (!Widget)
	{
		return;
	}

	Widget->Modify();

	UWidgetNavigation* WidgetNavigation = Widget->Navigation;
	if (!Widget->Navigation)
	{
		WidgetNavigation = NewObject<UWidgetNavigation>(Widget);
		WidgetNavigation->SetFlags(RF_Transactional);
		if (Widget->IsTemplate())
		{
			EObjectFlags TemplateFlags = Widget->GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;
			WidgetNavigation->SetFlags(TemplateFlags);
		}
	}

	if (NavigationMethodStruct == nullptr)
	{
		WidgetNavigation->NavigationMethod.Reset();
	}
	else
	{
		WidgetNavigation->NavigationMethod.InitializeAsScriptStruct(NavigationMethodStruct);
	}

	if ( WidgetNavigation->IsDefaultNavigation() )
	{
		// If the navigation rules are all set to the defaults, remove the navigation
		// information from the widget.
		Widget->Navigation = nullptr;
	}
	else
	{
		Widget->Navigation = WidgetNavigation;
	}

	// Force refresh the properties panel to get a handle on the NavigationMethod property
	PropertyUtilities->RequestForceRefresh();
}



#undef LOCTEXT_NAMESPACE
