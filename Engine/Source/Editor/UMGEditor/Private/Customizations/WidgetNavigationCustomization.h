// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "WidgetBlueprintEditor.h"
#include "IPropertyTypeCustomization.h"
#include "StructUtils/InstancedStruct.h"
#include "PropertyHandle.h"

class IDetailChildrenBuilder;
class IPropertyUtilities;

class FWidgetNavigationCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(TSharedRef<FWidgetBlueprintEditor> InEditor)
	{
		return MakeShareable(new FWidgetNavigationCustomization(InEditor));
	}

	FWidgetNavigationCustomization(TSharedRef<FWidgetBlueprintEditor> InEditor)
		: Editor(InEditor)
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
private:
	
	EUINavigationRule GetNavigationRule(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;
	FText GetNavigationText(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;

	void MakeNavRow(TWeakPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, EUINavigation Nav, FText NavName);

	TSharedRef<class SWidget> MakeNavMenu(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav);

	void HandleNavMenuEntryClicked(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav, EUINavigationRule Rule);

	TSharedRef<SWidget> OnGenerateWidgetList(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav);

	void HandleSelectedCustomNavigationFunction(FName SelectedFunction, TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav);
	void HandleResetCustomNavigationFunction(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav);

	FText GetExplictWidget(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;
	TOptional<FName> GetUniformNavigationTargetOrFunction(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;
	void OnWidgetSelectedForExplicitNavigation(FName ExplictWidgetOrFunction, TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav);
	EVisibility GetCustomWidgetFieldVisibility(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;
	EVisibility GetExplictWidgetFieldVisibility(TWeakPtr<IPropertyHandle> PropertyHandle, EUINavigation Nav) const;

	void SetNav(UWidget* Widget, EUINavigation Nav, TOptional<EUINavigationRule> Rule, TOptional<FName> WidgetToFocus);

	bool DoesWidgetSupportRoutingPolicy(TWeakPtr<IPropertyHandle> PropertyHandle) const;
	EWidgetNavigationRoutingPolicy GetRoutingPolicy(TWeakPtr<IPropertyHandle> PropertyHandle) const;
	FText GetRoutingPolicyText(TWeakPtr<IPropertyHandle> PropertyHandle) const;
	TSharedRef<class SWidget> MakeRoutingPolicyMenu(TWeakPtr<IPropertyHandle> PropertyHandle);
	void HandleRoutingPolicyMenuEntryClicked(TWeakPtr<IPropertyHandle> PropertyHandle, EWidgetNavigationRoutingPolicy Policy);
	void SetRoutingPolicy(UWidget* Widget, EWidgetNavigationRoutingPolicy Policy);

	void OnNavigationMethodPrePropertyChange(TWeakPtr<IPropertyHandle> PropertyHandle);
	void MakeNavigationMethodStructPicker(TWeakPtr<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder);
	void HandleNavigationMethodStructPicked(TWeakPtr<IPropertyHandle> PropertyHandle, const UScriptStruct* InStruct);
	void SetNavigationMethodStruct(UWidget* Widget, const UScriptStruct* NavigationMethodStruct);

private:
	TWeakPtr<FWidgetBlueprintEditor> Editor;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
