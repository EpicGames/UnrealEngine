// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/WidgetThumbnailCustomization.h"

#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ImageUtils.h"
#include "ImageWrapperHelper.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "WidgetBlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "FWidgetThumbnailCustomization"

TSharedRef<IDetailCustomization> FWidgetThumbnailCustomization::MakeInstance()
{
	return MakeShared<FWidgetThumbnailCustomization>();
}

void FWidgetThumbnailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	WidgetBlueprint = Cast<UWidgetBlueprint>(ObjectsBeingCustomized[0].Get());
	if (!WidgetBlueprint.IsValid())
	{
		return;
	}

	IDetailCategoryBuilder& ThumbnailCategory = DetailBuilder.EditCategory("ThumbnailSettings", FText::GetEmpty(), ECategoryPriority::Default);
	TSharedRef< IPropertyHandle > ThumbnailImage = DetailBuilder.GetProperty("ThumbnailImage");
	TSharedRef< IPropertyHandle > ThumbnailSizeMode = DetailBuilder.GetProperty("ThumbnailSizeMode");
	TSharedRef< IPropertyHandle > ThumbnailCustomSize = DetailBuilder.GetProperty("ThumbnailCustomSize");
	ThumbnailCategory.AddProperty(ThumbnailSizeMode).IsEnabled(TAttribute<bool>(this, &FWidgetThumbnailCustomization::IsThumbnailAutomatic));
	ThumbnailCategory.AddProperty(ThumbnailCustomSize).IsEnabled(TAttribute<bool>(this, &FWidgetThumbnailCustomization::IsThumbnailAutomaticAndCustom));
	ThumbnailCategory.AddProperty(ThumbnailImage);

	ThumbnailCategory.AddCustomRow(LOCTEXT("Browse_Thumbnail", "Custom Thumbnail"))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FWidgetThumbnailCustomization::LoadThumbnailFromFile)
				.ToolTipText(LOCTEXT("Button_Import_Tooltip", "Browse computer for an image to set as thumbnail."))
				.ContentPadding(2.f)
				.Text(LOCTEXT("ImportImage", "Import Image"))
			]
		];
}
FReply FWidgetThumbnailCustomization::LoadThumbnailFromFile()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		TArray<FString> OpenFilenames;
		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();
		bool bOpened = DesktopPlatform->OpenFileDialog(
			(ParentWindow.IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr,
			LOCTEXT("LoadSnapshotDialogTitle", "Load an Image to set as thumbnail for this widget blueprint.").ToString(),
			FPaths::GameAgnosticSavedDir(),
			TEXT(""),
			ImageWrapperHelper::GetImageFilesFilterString(false).GetData(),
			EFileDialogFlags::None,
			OpenFilenames
		);

		if (OpenFilenames.Num() > 0)
		{
			// load the new thumbnail
			UTexture2D* ThumbnailTexture = FImageUtils::ImportFileAsTexture2D(*OpenFilenames[0]);
			FWidgetBlueprintEditorUtils::SetTextureAsAssetThumbnail(WidgetBlueprint.Get(), ThumbnailTexture);
		}
	}

	return FReply::Handled();
}

void FWidgetThumbnailCustomization::ClearThumbnail()
{
	WidgetBlueprint->ThumbnailImage = nullptr;
}

bool FWidgetThumbnailCustomization::IsThumbnailAutomatic() const
{
	return WidgetBlueprint->ThumbnailImage == nullptr;
}

bool FWidgetThumbnailCustomization::IsThumbnailAutomaticAndCustom() const
{
	return WidgetBlueprint->ThumbnailSizeMode == EThumbnailPreviewSizeMode::Custom && IsThumbnailAutomatic();
}
#undef LOCTEXT_NAMESPACE
