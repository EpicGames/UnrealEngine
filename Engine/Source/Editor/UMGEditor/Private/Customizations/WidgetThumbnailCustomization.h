// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "WidgetBlueprint.h"
#include "Input/Reply.h"

/** UI customization for WidgetBlueprint thumbnail */
class FWidgetThumbnailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	FReply LoadThumbnailFromFile();
	void ClearThumbnail();
	bool IsThumbnailAutomatic() const;
	bool IsThumbnailAutomaticAndCustom() const;

	TWeakObjectPtr<UWidgetBlueprint> WidgetBlueprint;
};
