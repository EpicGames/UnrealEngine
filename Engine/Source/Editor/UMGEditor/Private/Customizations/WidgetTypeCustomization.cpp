// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/WidgetTypeCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"

#include "Blueprint/WidgetNavigation.h"
#include "Blueprint/WidgetTree.h"

#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprint.h"

#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "UMG"

// FWidgetTypeCustomization
////////////////////////////////////////////////////////////////////////////////

void FWidgetTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandlePtr = PropertyHandle;

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(FDetailWidgetRow::DefaultValueMaxWidth * 2)
	[
		SAssignNew(WidgetListComboButton, SComboButton)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.OnGetMenuContent(this, &FWidgetTypeCustomization::GetPopupContent)
		.ContentPadding(2.0f)
		.IsEnabled(!PropertyHandle->IsEditConst())
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FWidgetTypeCustomization::GetCurrentValueText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

TSharedRef<SWidget> FWidgetTypeCustomization::GetPopupContent()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
	{
		SAssignNew(WidgetListView, SListView<TWeakObjectPtr<UWidget>>)
			.ListItemsSource(&ReferenceableWidgets)
			.OnSelectionChanged(this, &FWidgetTypeCustomization::OnSelectionChanged)
			.OnGenerateRow(this, &FWidgetTypeCustomization::OnGenerateListItem)
			.SelectionMode(ESelectionMode::Single);

		// Ensure no filter is applied at the time the menu opens
		OnFilterTextChanged(FText::GetEmpty());

		WidgetListView->SetSelection(GetCurrentValue());

		TSharedRef<SVerticalBox> MenuContent =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &FWidgetTypeCustomization::OnFilterTextChanged)
			]

			+ SVerticalBox::Slot()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredHeight(300.0f)
				[
					WidgetListView.ToSharedRef()
				]
			];

		MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);

		WidgetListComboButton->SetMenuContentWidgetToFocus(SearchBox);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FWidgetTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

void FWidgetTypeCustomization::OnSelectionChanged(TWeakObjectPtr<UWidget> InItem, ESelectInfo::Type SelectionInfo)
{
	if (SelectionInfo != ESelectInfo::Direct)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
		if (PropertyHandle.IsValid())
		{
			TArray<FString> Refrences;
			for (int32 Index = 0; Index < PropertyHandle->GetNumPerObjectValues(); Index++)
			{
				Refrences.Add(InItem.Get()->GetPathName());
			}
			PropertyHandle->SetPerObjectValues(Refrences);

			WidgetListComboButton->SetIsOpen(false);
		}
	}
}

UWidget* FWidgetTypeCustomization::GetCurrentValue() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
	if (PropertyHandle.IsValid())
	{
		UObject* Object;
		switch (PropertyHandle->GetValue(Object))
		{
		case FPropertyAccess::Success:
			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				return Widget;
			}
			break;
		}
	}

	return nullptr;
}

FText FWidgetTypeCustomization::GetCurrentValueText() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
	if (PropertyHandle.IsValid())
	{
		UObject* Object;
		switch (PropertyHandle->GetValue(Object))
		{
		case FPropertyAccess::MultipleValues:
			return LOCTEXT("MultipleValues", "Multiple Values");
		case FPropertyAccess::Success:
			if (UWidget* Widget = Cast<UWidget>(Object))
			{
				return Widget->GetLabelText();
			}
			break;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<ITableRow> FWidgetTypeCustomization::OnGenerateListItem(TWeakObjectPtr<UWidget> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (UWidget* Widget = InItem.Get())
	{
		return
			SNew(STableRow<TWeakObjectPtr<UWidget>>, OwnerTable)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(Widget->GetLabelText())
			];
	}

	return SNew(STableRow<TWeakObjectPtr<UWidget>>, OwnerTable);
}

void FWidgetTypeCustomization::OnFilterTextChanged(const FText& InFilterText)
{
	FString CurrentFilterText = InFilterText.ToString();

	ReferenceableWidgets.Reset();

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
	if (PropertyHandle.IsValid())
	{
		FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(PropertyHandle->GetProperty());
		UClass* FilterWidgetClass = ObjectProperty->PropertyClass;

		UWidgetBlueprint* WidgetBlueprint = Editor.Pin()->GetWidgetBlueprintObj();
		if (WidgetBlueprint)
		{
			TArray<UWidget*> AllWidgets;
			WidgetBlueprint->WidgetTree->GetAllWidgets(AllWidgets);

			AllWidgets.Sort([](const UWidget& LHS, const UWidget& RHS) {
				return LHS.GetLabelText().CompareTo(RHS.GetLabelText()) < 0;
			});

			for (UWidget* Widget : AllWidgets)
			{
				if (Widget->bIsVariable == false && Widget->GetDisplayLabel().IsEmpty())
				{
					continue;
				}

				if (!Widget->IsA(FilterWidgetClass))
				{
					continue;
				}

				if (CurrentFilterText.IsEmpty() ||
					Widget->GetClass()->GetName().Contains(CurrentFilterText) ||
					Widget->GetDisplayLabel().Contains(CurrentFilterText))
				{
					ReferenceableWidgets.Add(Widget);
				}
			}
		}
	}

	WidgetListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
