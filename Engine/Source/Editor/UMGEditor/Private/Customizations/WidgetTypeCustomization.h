// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "WidgetBlueprintEditor.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"

class IDetailChildrenBuilder;
class UWidget;

class FWidgetTypeCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(TSharedRef<FWidgetBlueprintEditor> InEditor)
	{
		return MakeShareable(new FWidgetTypeCustomization(InEditor));
	}

	FWidgetTypeCustomization(TSharedRef<FWidgetBlueprintEditor> InEditor)
		: Editor(InEditor)
	{
	}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	TSharedRef<SWidget> GetPopupContent();
	void HandleMenuOpen();

	TSharedRef<ITableRow> OnGenerateListItem(TWeakObjectPtr<UWidget> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void OnSelectionChanged(TWeakObjectPtr<UWidget> InItem, ESelectInfo::Type SelectionInfo);

	UWidget* GetCurrentValue() const;
	FText GetCurrentValueText() const;

	void OnFilterTextChanged(const FText& InFilterText);

private:

	TWeakPtr<FWidgetBlueprintEditor> Editor;
	TSharedPtr<SComboButton> WidgetListComboButton;
	TSharedPtr<SListView<TWeakObjectPtr<UWidget>>> WidgetListView;
	TSharedPtr<SSearchBox> SearchBox;

	TWeakPtr<IPropertyHandle> PropertyHandlePtr;

	TArray<TWeakObjectPtr<UWidget>> ReferenceableWidgets;
};
