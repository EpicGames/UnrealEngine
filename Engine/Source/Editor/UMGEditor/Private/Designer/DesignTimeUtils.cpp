// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/DesignTimeUtils.h"

#include "Framework/Application/SlateApplication.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Geometry.h"
#include "Layout/WidgetPath.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/Optional.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Widgets/SWindow.h"

class SWidget;

#define LOCTEXT_NAMESPACE "UMGEditor"

bool FDesignTimeUtils::GetArrangedWidget(TSharedRef<SWidget> Widget, FArrangedWidget& ArrangedWidget)
{
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
	if ( !WidgetWindow.IsValid() )
	{
		return false;
	}

	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FWidgetPath WidgetPath;
	if ( FSlateApplication::Get().GeneratePathToWidgetUnchecked(Widget, WidgetPath) )
	{
		ArrangedWidget = WidgetPath.FindArrangedWidget(Widget).Get(FArrangedWidget::GetNullWidget());
		return true;
	}

	return false;
}

bool FDesignTimeUtils::GetArrangedWidgetRelativeToWindow(TSharedRef<SWidget> Widget, FArrangedWidget& ArrangedWidget)
{
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
	if ( !WidgetWindow.IsValid() )
	{
		return false;
	}

	while( WidgetWindow->GetParentWidget().IsValid() )
	{
		TSharedRef<SWidget> CurrentWidget = WidgetWindow->GetParentWidget().ToSharedRef();
		TSharedPtr<SWindow> ParentWidgetWindow = FSlateApplication::Get().FindWidgetWindow(CurrentWidget);
		if( !ParentWidgetWindow.IsValid() )
		{
			break;
		}
		WidgetWindow = ParentWidgetWindow;
	}
	
	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FWidgetPath WidgetPath;
	if ( FSlateApplication::Get().GeneratePathToWidgetUnchecked(Widget, WidgetPath) )
	{
		ArrangedWidget = WidgetPath.FindArrangedWidget(Widget).Get(FArrangedWidget::GetNullWidget());
		ArrangedWidget.Geometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
		//ArrangedWidget.Geometry.AppendTransform(Inverse(CurrentWindowRef->GetLocalToScreenTransform()));
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
