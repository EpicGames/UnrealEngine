// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "Templates/SharedPointer.h"

class FArrangedWidget;
class SWidget;

/**
 * A set of utility functions used at design time for the widget blueprint editor.
 */
class FDesignTimeUtils
{
public:
	static bool GetArrangedWidget(TSharedRef<SWidget> Widget, FArrangedWidget& ArrangedWidget);

	static bool GetArrangedWidgetRelativeToWindow(TSharedRef<SWidget> Widget, FArrangedWidget& ArrangedWidget);
};
