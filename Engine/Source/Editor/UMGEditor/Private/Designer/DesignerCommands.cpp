// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/DesignerCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "DesignerCommands"

void FDesignerCommands::RegisterCommands()
{
	UI_COMMAND( LayoutTransform, "Layout Transform Mode", "Adjust widget layout transform", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::W) );
	UI_COMMAND( RenderTransform, "Render Transform Mode", "Adjust widget render transform", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::E) );

	UI_COMMAND( LocationGridSnap, "Grid Snap", "Enables or disables snapping to the grid when dragging objects around", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( RotationGridSnap, "Rotation Snap", "Enables or disables snapping objects to a rotation grid", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleOutlines, "Show Outlines", "Enables or disables showing the dashed outlines", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::G) );
	UI_COMMAND( ToggleRespectLocks, "Respect Locks", "Enables or disables respecting locks placed on widgets.  Normally locked widgets prevent being selected in the designer.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::L) );
	UI_COMMAND( ToggleLocalizationPreview, "Toggle Localization Preview", "Enables or disables the localization preview for the current preview language (see Editor Settings -> Region & Language).", EUserInterfaceActionType::ToggleButton, FInputChord() );
}

#undef LOCTEXT_NAMESPACE
