// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * 
 */
class FDesignerCommands : public TCommands<FDesignerCommands>
{
public:
	FDesignerCommands()
		: TCommands<FDesignerCommands>
		(
			TEXT("WidgetDesigner"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "DesignerCommands", "UMG Designer Commands"), // Localized context name for displaying
			NAME_None, // Parent
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{
	}
	
	/** Layout Mode */
	TSharedPtr< FUICommandInfo > LayoutTransform;

	/** Render Mode */
	TSharedPtr< FUICommandInfo > RenderTransform;

	/** Enables or disables snapping to the grid when dragging objects around */
	TSharedPtr< FUICommandInfo > LocationGridSnap;

	/** Enables or disables snapping to a grid when rotating objects */
	TSharedPtr< FUICommandInfo > RotationGridSnap;

	/** Toggle Showing Outlines */
	TSharedPtr< FUICommandInfo > ToggleOutlines;

	/** Toggle if we care about locking */
	TSharedPtr< FUICommandInfo > ToggleRespectLocks;

	/** Toggle the localization preview on/off */
	TSharedPtr< FUICommandInfo > ToggleLocalizationPreview;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};


