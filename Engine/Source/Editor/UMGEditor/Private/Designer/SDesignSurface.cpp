// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/SDesignSurface.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_EDITOR
	#include "Settings/EditorStyleSettings.h"
	#include "Settings/LevelEditorViewportSettings.h"
#endif // WITH_EDITOR


#define LOCTEXT_NAMESPACE "UMG"

struct FZoomLevelEntry
{
public:
	FZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
		: DisplayText(FText::Format(LOCTEXT("Zoom", "Zoom {0}"), InDisplayText))
		, ZoomAmount(InZoomAmount)
		, LOD(InLOD)
	{
	}

public:
	FText DisplayText;
	float ZoomAmount;
	EGraphRenderingLOD::Type LOD;
};

struct FFixedZoomLevelsContainerDesignSurface : public FZoomLevelsContainer
{
	FFixedZoomLevelsContainerDesignSurface()
	{
		ZoomLevels.Add(FZoomLevelEntry(0.150f, FText::FromString("-10"), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.175f, FText::FromString("-9"), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.200f, FText::FromString("-8"), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.225f, FText::FromString("-7"), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.250f, FText::FromString("-6"), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.375f, FText::FromString("-5"), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.500f, FText::FromString("-4"), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.675f, FText::FromString("-3"), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.750f, FText::FromString("-2"), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.875f, FText::FromString("-1"), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.000f, FText::FromString("1:1"), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.250f, FText::FromString("+1"), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.500f, FText::FromString("+2"), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.750f, FText::FromString("+3"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.000f, FText::FromString("+4"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.250f, FText::FromString("+5"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.500f, FText::FromString("+6"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.750f, FText::FromString("+7"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(3.000f, FText::FromString("+8"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(3.250f, FText::FromString("+9"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(3.500f, FText::FromString("+10"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(4.000f, FText::FromString("+11"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(5.000f, FText::FromString("+12"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(6.000f, FText::FromString("+13"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(7.000f, FText::FromString("+14"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(8.000f, FText::FromString("+15"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(9.000f, FText::FromString("+16"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(10.000f, FText::FromString("+17"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(11.000f, FText::FromString("+18"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(12.000f, FText::FromString("+19"), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(13.000f, FText::FromString("+20"), EGraphRenderingLOD::FullyZoomedIn));
	}

	float GetZoomAmount(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].ZoomAmount;
	}

	int32 GetNearestZoomLevel(float InZoomAmount) const override
	{
		for ( int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex )
		{
			if ( InZoomAmount <= GetZoomAmount(ZoomLevelIndex) )
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}

	FText GetZoomText(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].DisplayText;
	}

	int32 GetNumZoomLevels() const override
	{
		return ZoomLevels.Num();
	}

	int32 GetDefaultZoomLevel() const override
	{
		return 10;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].LOD;
	}

	TArray<FZoomLevelEntry> ZoomLevels;
};

/////////////////////////////////////////////////////
// SDesignSurface

void SDesignSurface::Construct(const FArguments& InArgs)
{
	if ( !ZoomLevels )
	{
		ZoomLevels = MakeUnique<FFixedZoomLevelsContainerDesignSurface>();
	}
	ZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	PreviousZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	PostChangedZoom();
	AllowContinousZoomInterpolation = InArgs._AllowContinousZoomInterpolation;
	bIsPanning = false;
	bIsZooming = false;

	ViewOffset = FVector2D::ZeroVector;
	bDrawGridLines = true;

	ZoomLevelFade = FCurveSequence(0.0f, 1.0f);
	ZoomLevelFade.Play( this->AsShared() );

	ZoomLevelGraphFade = FCurveSequence(0.0f, 0.5f);
	ZoomLevelGraphFade.Play( this->AsShared() );

	bDeferredZoomToExtents = false;

	bAllowContinousZoomInterpolation = false;
	bTeleportInsteadOfScrollingWhenZoomingToFit = false;

	bRequireControlToOverZoom = false;

	ZoomTargetTopLeft = FVector2D::ZeroVector;
	ZoomTargetBottomRight = FVector2D::ZeroVector;

	ZoomToFitPadding = FVector2D(100, 100);
	TotalGestureMagnify = 0.0f;

	TotalMouseDelta = 0.0f;
	ZoomStartOffset = FVector2D::ZeroVector;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

EActiveTimerReturnType SDesignSurface::HandleZoomToFit( double InCurrentTime, float InDeltaTime )
{
	const FVector2D DesiredViewCenter = ( ZoomTargetTopLeft + ZoomTargetBottomRight ) * 0.5f;
	const bool bDoneScrolling = ScrollToLocation(GetCachedGeometry(), DesiredViewCenter, bTeleportInsteadOfScrollingWhenZoomingToFit ? 1000.0f : InDeltaTime);
	const bool bDoneZooming = ZoomToLocation(GetCachedGeometry().GetLocalSize(), ZoomTargetBottomRight - ZoomTargetTopLeft, bDoneScrolling);

	if (bDoneZooming && bDoneScrolling)
	{
		// One final push to make sure we're centered in the end
		ViewOffset = DesiredViewCenter - ( 0.5f * GetCachedGeometry().GetLocalSize() / GetZoomAmount() );

		ZoomTargetTopLeft = FVector2D::ZeroVector;
		ZoomTargetBottomRight = FVector2D::ZeroVector;

		return EActiveTimerReturnType::Stop;
	}
	
	return EActiveTimerReturnType::Continue;
}

void SDesignSurface::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Zoom to extents
	FSlateRect Bounds = ComputeAreaBounds();
	if ( bDeferredZoomToExtents )
	{
		bDeferredZoomToExtents = false;
		ZoomTargetTopLeft = FVector2D(Bounds.Left, Bounds.Top);
		ZoomTargetBottomRight = FVector2D(Bounds.Right, Bounds.Bottom);

		if (!ActiveTimerHandle.IsValid())
		{
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SDesignSurface::HandleZoomToFit));
		}
	}
}

FCursorReply SDesignSurface::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if ( bIsPanning )
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return SCompoundWidget::OnCursorQuery(MyGeometry, CursorEvent);
}

int32 SDesignSurface::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Store the current Scene Index, as child classes may want to use a different scene to preview widgets (Ex: SDesignerView)
	int32 SceneIndex = FSlateApplication::Get().GetRenderer()->GetCurrentSceneIndex();

	OnPaintBackground(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	int32 Result = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Restore previous Scene Index to avoid permanently changing the scene
	FSlateApplication::Get().GetRenderer()->SetCurrentSceneIndex(SceneIndex);

	return Result;
}

void SDesignSurface::OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const FSlateBrush* BackgroundImage = FAppStyle::GetBrush(TEXT("Graph.Panel.SolidBackground"));
	PaintBackgroundAsLines(BackgroundImage, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
}

FReply SDesignSurface::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);

	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton || MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
	{
		bIsPanning = false;

		ViewOffsetStart = ViewOffset;
		MouseDownPositionAbsolute = MouseEvent.GetLastScreenSpacePosition();
	}

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton || FSlateApplication::Get().IsUsingTrackpad())
	{
		TotalMouseDelta = 0.0f;
		ZoomStartOffset = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
	}

	return FReply::Unhandled();
}

FReply SDesignSurface::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);

	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton || MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
	{
		bIsPanning = false;
		bIsZooming = false;
	}

	return FReply::Unhandled();
}

FReply SDesignSurface::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
	const FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

	if ( HasMouseCapture() )
	{
		const FVector2D CursorDelta = MouseEvent.GetCursorDelta();

		const bool bShouldZoom = bIsRightMouseButtonDown && (bIsLeftMouseButtonDown || bIsMiddleMouseButtonDown || ModifierKeysState.IsAltDown() || FSlateApplication::Get().IsUsingTrackpad());
		if ( bShouldZoom )
		{
			const double MouseZoomScaling = 0.04f;
			FReply ReplyState = FReply::Handled();

			TotalMouseDelta += CursorDelta.X + CursorDelta.Y;

			const int32 ZoomLevelDelta = FMath::RoundToInt32(TotalMouseDelta * MouseZoomScaling);

			// Get rid of mouse movement that's been 'used up' by zooming
			if (ZoomLevelDelta != 0)
			{
				TotalMouseDelta -= (ZoomLevelDelta / MouseZoomScaling);
				bIsZooming = true;
			}

			// Perform zoom centered on the cached start offset
			ChangeZoomLevel(ZoomLevelDelta, ZoomStartOffset, MouseEvent.IsControlDown());

			bIsPanning = false;

			return ReplyState;
		}
		else if ( bIsRightMouseButtonDown || bIsMiddleMouseButtonDown )
		{
			FReply ReplyState = FReply::Handled();

			bIsPanning = true;
			bIsZooming = false;
			ViewOffset = ViewOffsetStart + ( (MouseDownPositionAbsolute - MouseEvent.GetScreenSpacePosition()) / MyGeometry.Scale) / GetZoomAmount();

			return ReplyState;
		}
	}

	return FReply::Unhandled();
}

FReply SDesignSurface::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
	const FVector2D WidgetSpaceCursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const int32 ZoomLevelDelta = FMath::FloorToInt(MouseEvent.GetWheelDelta());
	ChangeZoomLevel(ZoomLevelDelta, WidgetSpaceCursorPos, !bRequireControlToOverZoom || MouseEvent.IsControlDown());
	MouseDownPositionAbsolute = MouseEvent.GetScreenSpacePosition();

	return FReply::Handled();
}

FReply SDesignSurface::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent)
{
	const EGestureEvent GestureType = GestureEvent.GetGestureType();
	const FVector2D GestureDelta = GestureEvent.GetGestureDelta();
	if ( GestureType == EGestureEvent::Magnify )
	{
		TotalGestureMagnify += static_cast<float>(GestureDelta.X);
		if ( FMath::Abs(TotalGestureMagnify) > 0.07f )
		{
			// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
			const FVector2D WidgetSpaceCursorPos = MyGeometry.AbsoluteToLocal(GestureEvent.GetScreenSpacePosition());
			const int32 ZoomLevelDelta = TotalGestureMagnify > 0.0f ? 1 : -1;
			ChangeZoomLevel(ZoomLevelDelta, WidgetSpaceCursorPos, !bRequireControlToOverZoom || GestureEvent.IsControlDown());
			MouseDownPositionAbsolute = GestureEvent.GetScreenSpacePosition();
			TotalGestureMagnify = 0.0f;
		}
		return FReply::Handled();
	}
	else if ( GestureType == EGestureEvent::Scroll )
	{
		const EScrollGestureDirection DirectionSetting = GetDefault<ULevelEditorViewportSettings>()->ScrollGestureDirectionForOrthoViewports;
		const bool bUseDirectionInvertedFromDevice = DirectionSetting == EScrollGestureDirection::Natural || (DirectionSetting == EScrollGestureDirection::UseSystemSetting && GestureEvent.IsDirectionInvertedFromDevice());

		this->bIsPanning = true;
		ViewOffset -= (bUseDirectionInvertedFromDevice == GestureEvent.IsDirectionInvertedFromDevice() ? GestureDelta : -GestureDelta) / GetZoomAmount();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SDesignSurface::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	TotalGestureMagnify = 0.0f;
	return FReply::Unhandled();
}

inline float FancyMod(float Value, float Size)
{
	return ( ( Value >= 0 ) ? 0.0f : Size ) + FMath::Fmod(Value, Size);
}

float SDesignSurface::GetZoomAmount() const
{
	if ( AllowContinousZoomInterpolation.Get() )
	{
		return FMath::Lerp(ZoomLevels->GetZoomAmount(PreviousZoomLevel), ZoomLevels->GetZoomAmount(ZoomLevel), ZoomLevelGraphFade.GetLerp());
	}
	else
	{
		return ZoomLevels->GetZoomAmount(ZoomLevel);
	}
}

void SDesignSurface::ChangeZoomLevel(int32 ZoomLevelDelta, const FVector2D& WidgetSpaceZoomOrigin, bool bOverrideZoomLimiting)
{
	// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
	const FVector2D PointToMaintainGraphSpace = PanelCoordToGraphCoord(WidgetSpaceZoomOrigin);

	const int32 DefaultZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	const int32 NumZoomLevels = ZoomLevels->GetNumZoomLevels();

	const bool bAllowFullZoomRange =
		// To zoom in past 1:1 the user must press control
		( ZoomLevel == DefaultZoomLevel && ZoomLevelDelta > 0 && bOverrideZoomLimiting ) ||
		// If they are already zoomed in past 1:1, user may zoom freely
		( ZoomLevel > DefaultZoomLevel );

	const int32 OldZoomLevel = ZoomLevel;

	if ( bAllowFullZoomRange )
	{
		ZoomLevel = FMath::Clamp(ZoomLevel + ZoomLevelDelta, 0, NumZoomLevels - 1);
	}
	else
	{
		// Without control, we do not allow zooming in past 1:1.
		ZoomLevel = FMath::Clamp(ZoomLevel + ZoomLevelDelta, 0, DefaultZoomLevel);
	}

	if ( OldZoomLevel != ZoomLevel )
	{
		PostChangedZoom();

		// Note: This happens even when maxed out at a stop; so the user sees the animation and knows that they're at max zoom in/out
		ZoomLevelFade.Play( this->AsShared() );

		// Re-center the screen so that it feels like zooming around the cursor.
		{
			const FVector2D NewViewOffset = PointToMaintainGraphSpace - WidgetSpaceZoomOrigin / GetZoomAmount();

			// If we're panning while zooming we need to update the viewoffset start.
			ViewOffsetStart += (NewViewOffset - ViewOffset);
			// Update view offset to where ever we scrolled towards.
			ViewOffset = NewViewOffset;

			TotalMouseDelta = 0.0f;
		}
	}
}

FSlateRect SDesignSurface::ComputeSensibleBounds() const
{
	// Pad it out in every direction, to roughly account for nodes being of non-zero extent
	const float Padding = 100.0f;

	FSlateRect Bounds = ComputeAreaBounds();
	Bounds.Left -= Padding;
	Bounds.Top -= Padding;
	Bounds.Right -= Padding;
	Bounds.Bottom -= Padding;

	return Bounds;
}

void SDesignSurface::PostChangedZoom()
{
}

bool SDesignSurface::ScrollToLocation(const FGeometry& MyGeometry, FVector2D DesiredCenterPosition, const float InDeltaTime)
{
	const FVector2D HalfOFScreenInGraphSpace = 0.5f * MyGeometry.GetLocalSize() / GetZoomAmount();
	FVector2D CurrentPosition = ViewOffset + HalfOFScreenInGraphSpace;

	FVector2D NewPosition = FMath::Vector2DInterpTo(CurrentPosition, DesiredCenterPosition, InDeltaTime, 10.f);
	ViewOffset = NewPosition - HalfOFScreenInGraphSpace;

	// If within 1 pixel of target, stop interpolating
	return ( ( NewPosition - DesiredCenterPosition ).SizeSquared() < 1.f );
}

bool SDesignSurface::ZoomToLocation(const FVector2D& CurrentSizeWithoutZoom, const FVector2D& InDesiredSize, bool bDoneScrolling)
{
	if ( bAllowContinousZoomInterpolation && ZoomLevelGraphFade.IsPlaying() )
	{
		return false;
	}

	const int32 DefaultZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	const int32 NumZoomLevels = ZoomLevels->GetNumZoomLevels();
	int32 DesiredZoom = DefaultZoomLevel;

	// Find lowest zoom level that will display all nodes
	for ( int32 Zoom = 0; Zoom < DefaultZoomLevel; ++Zoom )
	{
		const FVector2D SizeWithZoom = (CurrentSizeWithoutZoom - ZoomToFitPadding) / ZoomLevels->GetZoomAmount(Zoom);
		const FVector2D LeftOverSize = SizeWithZoom - InDesiredSize;

		if ( ( InDesiredSize.X > SizeWithZoom.X ) || ( InDesiredSize.Y > SizeWithZoom.Y ) )
		{
			// Use the previous zoom level, this one is too tight
			DesiredZoom = FMath::Max<int32>(0, Zoom - 1);
			break;
		}
	}

	if ( DesiredZoom != ZoomLevel )
	{
		if ( bAllowContinousZoomInterpolation )
		{
			// Animate to it
			PreviousZoomLevel = ZoomLevel;
			ZoomLevel = FMath::Clamp(DesiredZoom, 0, NumZoomLevels - 1);
			ZoomLevelGraphFade.Play(this->AsShared());
			return false;
		}
		else
		{
			// Do it instantly, either first or last
			if ( DesiredZoom < ZoomLevel )
			{
				// Zooming out; do it instantly
				ZoomLevel = PreviousZoomLevel = DesiredZoom;
				ZoomLevelFade.Play(this->AsShared());
			}
			else
			{
				// Zooming in; do it last
				if ( bDoneScrolling )
				{
					ZoomLevel = PreviousZoomLevel = DesiredZoom;
					ZoomLevelFade.Play(this->AsShared());
				}
			}
		}

		PostChangedZoom();
	}

	return true;
}

void SDesignSurface::ZoomToFit(bool bInstantZoom)
{
	bTeleportInsteadOfScrollingWhenZoomingToFit = bInstantZoom;
	bDeferredZoomToExtents = true;
}

FText SDesignSurface::GetZoomText() const
{
	return ZoomLevels->GetZoomText(ZoomLevel);
}

FSlateColor SDesignSurface::GetZoomTextColorAndOpacity() const
{
	return FLinearColor(1, 1, 1, 1.25f - ZoomLevelFade.GetLerp());
}

FSlateRect SDesignSurface::ComputeAreaBounds() const
{
	return FSlateRect(0, 0, 0, 0);
}

FVector2D SDesignSurface::GetViewOffset() const
{
	return ViewOffset;
}

FVector2D SDesignSurface::GraphCoordToPanelCoord(const FVector2D& GraphSpaceCoordinate) const
{
	return ( GraphSpaceCoordinate - GetViewOffset() ) * GetZoomAmount();
}

FVector2D SDesignSurface::PanelCoordToGraphCoord(const FVector2D& PanelSpaceCoordinate) const
{
	return PanelSpaceCoordinate / GetZoomAmount() + GetViewOffset();
}

int32 SDesignSurface::GetGraphRulePeriod() const
{
	return (int32)FAppStyle::GetFloat("Graph.Panel.GridRulePeriod");
}

float SDesignSurface::GetGridScaleAmount() const
{
	return 1;
}

void SDesignSurface::PaintBackgroundAsLines(const FSlateBrush* BackgroundImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const
{
	const bool bAntialias = GetDefault<UEditorStyleSettings>()->bAntiAliasGrid;

	const int32 RulePeriod = GetGraphRulePeriod();
	check(RulePeriod > 0);

	const FLinearColor RegularColor(FAppStyle::GetColor("Graph.Panel.GridLineColor"));
	const FLinearColor RuleColor(FAppStyle::GetColor("Graph.Panel.GridRuleColor"));
	const FLinearColor CenterColor(FAppStyle::GetColor("Graph.Panel.GridCenterColor"));
	const float GraphSmallestGridSize = 8.0f;
	const float RawZoomFactor = GetZoomAmount();
	const float NominalGridSize = GetSnapGridSize() * GetGridScaleAmount();

	float ZoomFactor = RawZoomFactor;
	float Inflation = 1.0f;
	while ( ZoomFactor*Inflation*NominalGridSize <= GraphSmallestGridSize )
	{
		Inflation *= 2.0f;
	}

	const float GridCellSize = NominalGridSize * ZoomFactor * Inflation;

	FVector2f LocalGridOrigin = AllottedGeometry.AbsoluteToLocal(GridOrigin);

	float ImageOffsetX = LocalGridOrigin.X - ((GridCellSize*RulePeriod) * FMath::Max(FMath::CeilToInt(LocalGridOrigin.X / (GridCellSize*RulePeriod)), 0));
	float ImageOffsetY = LocalGridOrigin.Y - ((GridCellSize*RulePeriod) * FMath::Max(FMath::CeilToInt(LocalGridOrigin.Y / (GridCellSize*RulePeriod)), 0));

	// Fill the background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawLayerId,
		AllottedGeometry.ToPaintGeometry(),
		BackgroundImage
		);

	if (bDrawGridLines)
	{
		TArray<FVector2D> LinePoints;
		new (LinePoints)FVector2D(0.0f, 0.0f);
		new (LinePoints)FVector2D(0.0f, 0.0f);

		// Horizontal bars
		for (int32 GridIndex = 0; ImageOffsetY < AllottedGeometry.GetLocalSize().Y; ImageOffsetY += GridCellSize, ++GridIndex)
		{
			if (ImageOffsetY >= 0.0f)
			{
				const bool bIsRuleLine = (GridIndex % RulePeriod) == 0;
				const int32 Layer = bIsRuleLine ? (DrawLayerId + 1) : DrawLayerId;

				const FLinearColor* Color = bIsRuleLine ? &RuleColor : &RegularColor;
				if (FMath::IsNearlyEqual(LocalGridOrigin.Y, ImageOffsetY, 1.0f))
				{
					Color = &CenterColor;
				}

				LinePoints[0] = FVector2D(0.0f, ImageOffsetY);
				LinePoints[1] = FVector2D(AllottedGeometry.GetLocalSize().X, ImageOffsetY);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					Layer,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					*Color,
					bAntialias);
			}
		}

		// Vertical bars
		for (int32 GridIndex = 0; ImageOffsetX < AllottedGeometry.GetLocalSize().X; ImageOffsetX += GridCellSize, ++GridIndex)
		{
			if (ImageOffsetX >= 0.0f)
			{
				const bool bIsRuleLine = (GridIndex % RulePeriod) == 0;
				const int32 Layer = bIsRuleLine ? (DrawLayerId + 1) : DrawLayerId;

				const FLinearColor* Color = bIsRuleLine ? &RuleColor : &RegularColor;
				if (FMath::IsNearlyEqual(LocalGridOrigin.X, ImageOffsetX, 1.0f))
				{
					Color = &CenterColor;
				}

				LinePoints[0] = FVector2D(ImageOffsetX, 0.0f);
				LinePoints[1] = FVector2D(ImageOffsetX, AllottedGeometry.GetLocalSize().Y);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					Layer,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					*Color,
					bAntialias);
			}
		}
	}

	DrawLayerId += 2;
}

#undef LOCTEXT_NAMESPACE
