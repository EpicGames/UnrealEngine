// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/SDesignerToolBar.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Culture.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "Settings/WidgetDesignerSettings.h"

#include "Designer/DesignerCommands.h"
#include "SViewportToolBarComboMenu.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "SequencerCommands.h"

#define LOCTEXT_NAMESPACE "UMG"

void SDesignerToolBar::Construct( const FArguments& InArgs )
{
	CommandList = InArgs._CommandList;

	ChildSlot
	[
		MakeToolBar(InArgs._Extenders)
	];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

TSharedRef< SWidget > SDesignerToolBar::MakeToolBar(const TSharedPtr< FExtender > InExtenders)
{
	FToolBarBuilder ToolbarBuilder( CommandList, FMultiBoxCustomization::None, InExtenders );

	// Use a custom style	
	FName ToolBarStyle = "EditorViewportToolBar";
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	ToolbarBuilder.BeginSection("Localization");
	{
		FUICommandInfo* ToggleLocalizationPreviewCommand = FDesignerCommands::Get().ToggleLocalizationPreview.Get();

		ToolbarBuilder.AddWidget(SNew(SViewportToolBarComboMenu)
			.Cursor(EMouseCursor::Default)
			.IsChecked(this, &SDesignerToolBar::IsLocalizationPreviewChecked)
			.OnCheckStateChanged(this, &SDesignerToolBar::HandleToggleLocalizationPreview)
			.Label(this, &SDesignerToolBar::GetLocalizationPreviewLabel)
			.OnGetMenuContent(this, &SDesignerToolBar::FillLocalizationPreviewMenu)
			.ToggleButtonToolTip(ToggleLocalizationPreviewCommand->GetDescription())
			.MenuButtonToolTip(LOCTEXT("ToggleLocalizationPreview_MenuToolTip", "Choose the localization preview language"))
			.Icon(ToggleLocalizationPreviewCommand->GetIcon())
			.ParentToolBar(SharedThis(this))
			, "ToggleLocalizationPreview");
	}
	ToolbarBuilder.EndSection();

	// Transform controls cannot be focusable as it fights with the press space to change transform mode feature
	ToolbarBuilder.SetIsFocusable( false );

	ToolbarBuilder.BeginSection("View");
	ToolbarBuilder.BeginBlockGroup();
	ToolbarBuilder.AddToolBarButton(FDesignerCommands::Get().ToggleOutlines, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), "ToggleOutlines");
	ToolbarBuilder.AddToolBarButton(FDesignerCommands::Get().ToggleRespectLocks, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), "ToggleRespectLocks");
	ToolbarBuilder.AddToolBarButton(FSequencerCommands::Get().ToggleLimitViewportSelection, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), "ToggleLimitViewportSelection");
	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("Transform");
	ToolbarBuilder.BeginBlockGroup();
	{
		ToolbarBuilder.AddToolBarButton( FDesignerCommands::Get().LayoutTransform, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), "LayoutTransform" );
		ToolbarBuilder.AddToolBarButton( FDesignerCommands::Get().RenderTransform, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), "RenderTransform" );
	}
	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("LocationGridSnap");
	{
		// Grab the existing UICommand 
		FUICommandInfo* Command = FDesignerCommands::Get().LocationGridSnap.Get();

		static FName PositionSnapName = FName(TEXT("PositionSnap"));

		// Setup a GridSnapSetting with the UICommand
		ToolbarBuilder.AddWidget(SNew(SViewportToolBarComboMenu)
			.Cursor(EMouseCursor::Default)
			.IsChecked(this, &SDesignerToolBar::IsLocationGridSnapChecked)
			.OnCheckStateChanged(this, &SDesignerToolBar::HandleToggleLocationGridSnap)
			.Label(this, &SDesignerToolBar::GetLocationGridLabel)
			.OnGetMenuContent(this, &SDesignerToolBar::FillLocationGridSnapMenu)
			.ToggleButtonToolTip(Command->GetDescription())
			.MenuButtonToolTip(LOCTEXT("LocationGridSnap_ToolTip", "Set the Position Grid Snap value"))
			.Icon(Command->GetIcon())
			.ParentToolBar(SharedThis(this))
			, PositionSnapName);
	}
	ToolbarBuilder.EndSection();
	
	return ToolbarBuilder.MakeWidget();
}

ECheckBoxState SDesignerToolBar::IsLocationGridSnapChecked() const
{
	return GetDefault<UWidgetDesignerSettings>()->GridSnapEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDesignerToolBar::HandleToggleLocationGridSnap(ECheckBoxState InState)
{
	UWidgetDesignerSettings* ViewportSettings = GetMutableDefault<UWidgetDesignerSettings>();
	ViewportSettings->GridSnapEnabled = !ViewportSettings->GridSnapEnabled;
}

FText SDesignerToolBar::GetLocationGridLabel() const
{
	const UWidgetDesignerSettings* ViewportSettings = GetDefault<UWidgetDesignerSettings>();
	return FText::AsNumber(ViewportSettings->GridSnapSize);
}

TSharedRef<SWidget> SDesignerToolBar::FillLocationGridSnapMenu()
{
	TArray<float> GridSizes;
	GridSizes.Add(1);
	GridSizes.Add(2);
	GridSizes.Add(3);
	GridSizes.Add(4);
	GridSizes.Add(5);
	GridSizes.Add(10);
	GridSizes.Add(15);
	GridSizes.Add(25);

	UE::UnrealEd::FLocationGridCheckboxListExecuteActionDelegate ExecuteDelegate =
		UE::UnrealEd::FLocationGridCheckboxListExecuteActionDelegate::CreateLambda(
			[GridSizes](int CurrGridSizeIndex)
			{
				UWidgetDesignerSettings* ViewportSettings = GetMutableDefault<UWidgetDesignerSettings>();
				const int32 CurrGridSize = GridSizes[CurrGridSizeIndex];
				ViewportSettings->GridSnapSize = CurrGridSize;
			}
		);

	UE::UnrealEd::FLocationGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
		UE::UnrealEd::FLocationGridCheckboxListIsCheckedDelegate::CreateLambda(
			[GridSizes](int CurrGridSizeIndex)
			{
				const UWidgetDesignerSettings* ViewportSettings = GetDefault<UWidgetDesignerSettings>();
				const int32 CurrGridSize = GridSizes[CurrGridSizeIndex];
				return (ViewportSettings->GridSnapSize == CurrGridSize);
			}
		);

	UE::UnrealEd::FLocationGridValuesArrayDelegate GridSizesDelegate =
		UE::UnrealEd::FLocationGridValuesArrayDelegate::CreateLambda(
			[GridSizes]()
			{
				return GridSizes;
			}
		);

	UE::UnrealEd::FLocationGridSnapMenuOptions MenuOptions;
	MenuOptions.MenuName = "UMG_LocationGridCheckboxList";
	MenuOptions.ExecuteDelegate = ExecuteDelegate;
	MenuOptions.IsCheckedDelegate = IsCheckedDelegate;
	MenuOptions.GridValuesArrayDelegate = GridSizesDelegate;
	MenuOptions.CommandList = CommandList;
	return UE::UnrealEd::CreateLocationGridSnapMenu(MenuOptions);
}

void SDesignerToolBar::SetGridSize(int32 InGridSize)
{
	UWidgetDesignerSettings* ViewportSettings = GetMutableDefault<UWidgetDesignerSettings>();
	ViewportSettings->GridSnapSize = InGridSize;
}

bool SDesignerToolBar::IsGridSizeChecked(int32 InGridSnapSize)
{
	const UWidgetDesignerSettings* ViewportSettings = GetDefault<UWidgetDesignerSettings>();
	return ( ViewportSettings->GridSnapSize == InGridSnapSize );
}

ECheckBoxState SDesignerToolBar::IsLocalizationPreviewChecked() const
{
	return FTextLocalizationManager::Get().IsGameLocalizationPreviewEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDesignerToolBar::HandleToggleLocalizationPreview(ECheckBoxState InState)
{
	if (InState == ECheckBoxState::Checked)
	{
		FTextLocalizationManager::Get().EnableGameLocalizationPreview();
	}
	else
	{
		FTextLocalizationManager::Get().DisableGameLocalizationPreview();
	}
}

FText SDesignerToolBar::GetLocalizationPreviewLabel() const
{
	const FString PreviewGameLanguage = FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage();
	return PreviewGameLanguage.IsEmpty() ? LOCTEXT("LocalizationPreviewLanguage_None", "None") : FText::AsCultureInvariant(PreviewGameLanguage);
}

TSharedRef<SWidget> SDesignerToolBar::FillLocalizationPreviewMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder LocationGridMenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	TArray<FCultureRef> GameCultures;
	{
		const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
		GameCultures = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, false);
	}

	LocationGridMenuBuilder.BeginSection("LocalizationPreviewLanguage", LOCTEXT("LocalizationPreviewLanguage", "Preview Language"));
	LocationGridMenuBuilder.AddMenuEntry(
		LOCTEXT("LocalizationPreviewLanguage_None", "None"),
		LOCTEXT("LocalizationPreviewLanguage_None_ToolTip", "Clear the active localization preview language"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&SDesignerToolBar::SetLocalizationPreviewLanguage, FString()),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&SDesignerToolBar::IsLocalizationPreviewLanguageChecked, FString())
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
	for (const FCultureRef& GameCulture : GameCultures)
	{
		LocationGridMenuBuilder.AddMenuEntry(
			FText::AsCultureInvariant(GameCulture->GetDisplayName()),
			FText::Format(LOCTEXT("LocalizationPreviewLanguage_ToolTip", "Set the active localization preview language to '{0}'"), FText::AsCultureInvariant(GameCulture->GetName())),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&SDesignerToolBar::SetLocalizationPreviewLanguage, GameCulture->GetName()),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&SDesignerToolBar::IsLocalizationPreviewLanguageChecked, GameCulture->GetName())
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	LocationGridMenuBuilder.EndSection();

	LocationGridMenuBuilder.BeginSection("LocalizationSettings", LOCTEXT("LocalizationSettings", "Settings"));
	LocationGridMenuBuilder.AddMenuEntry(
		LOCTEXT("LocalizationSettings_RegionAndLanguage", "Region & Language"),
		LOCTEXT("LocalizationSettings_RegionAndLanguage_ToolTip", "Open the 'Region & Language' settings for the editor"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&SDesignerToolBar::OpenRegionAndLanguageSettings)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
	LocationGridMenuBuilder.EndSection();

	return LocationGridMenuBuilder.MakeWidget();
}

void SDesignerToolBar::SetLocalizationPreviewLanguage(FString InCulture)
{
	FTextLocalizationManager::Get().ConfigureGameLocalizationPreviewLanguage(InCulture);
	FTextLocalizationManager::Get().EnableGameLocalizationPreview();
}

bool SDesignerToolBar::IsLocalizationPreviewLanguageChecked(FString InCulture)
{
	const FString PreviewGameLanguage = FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage();
	return PreviewGameLanguage == InCulture;
}

void SDesignerToolBar::OpenRegionAndLanguageSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Internationalization");
}

#undef LOCTEXT_NAMESPACE
