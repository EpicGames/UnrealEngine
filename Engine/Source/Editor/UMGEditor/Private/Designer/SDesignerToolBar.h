// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "SViewportToolBar.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FExtender;
class FUICommandList;
class SWidget;

enum class ECheckBoxState : uint8;

/**
 * 
 */
class SDesignerToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS( SDesignerToolBar ){}
		SLATE_ARGUMENT( TSharedPtr<FUICommandList>, CommandList )
		SLATE_ARGUMENT( TSharedPtr<FExtender>, Extenders )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	/**
	 * Static: Creates a widget for the main tool bar
	 *
	 * @return	New widget
	 */
	TSharedRef< SWidget > MakeToolBar( const TSharedPtr< FExtender > InExtenders );	

private:
	// Begin Grid Snapping
	ECheckBoxState IsLocationGridSnapChecked() const;
	void HandleToggleLocationGridSnap(ECheckBoxState InState);
	FText GetLocationGridLabel() const;
	TSharedRef<SWidget> FillLocationGridSnapMenu();
	static void SetGridSize(int32 InGridSize);
	static bool IsGridSizeChecked(int32 InGridSnapSize);
	// End Grid Snapping

	// Begin Localization Preview
	ECheckBoxState IsLocalizationPreviewChecked() const;
	void HandleToggleLocalizationPreview(ECheckBoxState InState);
	FText GetLocalizationPreviewLabel() const;
	TSharedRef<SWidget> FillLocalizationPreviewMenu();
	static void SetLocalizationPreviewLanguage(FString InCulture);
	static bool IsLocalizationPreviewLanguageChecked(FString InCulture);
	static void OpenRegionAndLanguageSettings();
	// End Localization Preview

	/** Command list */
	TSharedPtr<FUICommandList> CommandList;
};
