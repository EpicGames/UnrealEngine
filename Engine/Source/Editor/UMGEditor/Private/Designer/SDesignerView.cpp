// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/SDesignerView.h"
#include "Rendering/DrawElements.h"
#include "Components/PanelWidget.h"
#include "Misc/ConfigCacheIni.h"
#include "WidgetBlueprint.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#include "Animation/WidgetAnimation.h"

#include "Components/CanvasPanelSlot.h"
#include "Blueprint/WidgetTree.h"
#include "Settings/WidgetDesignerSettings.h"

#include "ISettingsModule.h"

#include "Designer/DesignTimeUtils.h"

#include "Extensions/CanvasSlotExtension.h"
#include "Extensions/GridSlotExtension.h"
#include "Extensions/HorizontalSlotExtension.h"
#include "Extensions/StackBoxSlotExtension.h"
#include "Extensions/UniformGridSlotExtension.h"
#include "Extensions/VerticalSlotExtension.h"
#include "Designer/SPaintSurface.h"

#include "Kismet2/BlueprintEditorUtils.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragDrop/WidgetTemplateDragDropOp.h"
#include "DragDrop/SelectedWidgetDragDropOp.h"

#include "Templates/WidgetTemplateBlueprintClass.h"
#include "Templates/WidgetTemplateImageClass.h"

#include "Designer/SZoomPan.h"
#include "Designer/SRuler.h"
#include "Designer/SDisappearingBar.h"
#include "Designer/SDesignerToolBar.h"
#include "Designer/DesignerCommands.h"
#include "Designer/STransformHandle.h"
#include "Engine/UserInterfaceSettings.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Input/SNumericEntryBox.h"

#include "Engine/Texture2D.h"
#include "Editor.h"
#include "WidgetBlueprintEditorUtils.h"

#include "ObjectEditorUtils.h"
#include "PreviewScene.h"
#include "ScopedTransaction.h"
#include "Components/NamedSlot.h"

#include "Types/ReflectionMetadata.h"

#include "Math/TransformCalculus2D.h"
#include "Input/HittestGrid.h"
#include "Internationalization/Culture.h"

#include "Fonts/FontMeasure.h"
#include "WidgetEditingProjectSettings.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/DPICustomScalingRule.h"
#include "UMGEditorModule.h"
#include "ToolMenus.h"
#include "Styling/ToolBarStyle.h"
#include "UMGEditorProjectSettings.h"
#include "Types/InvisibleToWidgetReflectorMetaData.h"
#include "SequencerCommands.h"

#define LOCTEXT_NAMESPACE "UMG"

const float HoveredAnimationTime = 0.150f;


class SResizeDesignerHandle : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SResizeDesignerHandle) {
		_Visibility = EVisibility::Visible;
		_Cursor = EMouseCursor::ResizeSouthEast;
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SDesignerView> InDesigner)
	{
		Designer = InDesigner;
		bResizing = false;

		ChildSlot
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("UMGEditor.ResizeAreaHandle"))
		];
	}

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bResizing = true;
			AbsoluteOffset = MouseEvent.GetScreenSpacePosition() - FVector2D(MyGeometry.AbsolutePosition);
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			TSharedPtr<SDesignerView> DesignerView = Designer.Pin();
			bResizing = false;
			DesignerView->EndResizingArea();
			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (bResizing)
		{
			TSharedPtr<SDesignerView> DesignerView = Designer.Pin();
			if (!DesignerView)
			{
				return FReply::Unhandled();
            }

			DesignerView->BeginResizingArea();

			const float ZoomAmount = DesignerView->GetZoomAmount();

			FVector2D AreaSize = (MouseEvent.GetScreenSpacePosition() - AbsoluteOffset) - DesignerView->GetWidgetOriginAbsolute();
			AreaSize /= ZoomAmount;
			AreaSize /= MyGeometry.Scale;

			if (const UWidgetEditingProjectSettings* Settings = DesignerView->GetRelevantSettings())
			{
				for (const FDebugResolution& Resolution : Settings->DebugResolutions)
				{
					if (((AreaSize - FVector2D(Resolution.Width, Resolution.Height)) * ZoomAmount).Size() < 10.0f)
					{
						AreaSize = FVector2D(Resolution.Width, Resolution.Height);
						break;
					}
				}
			}

			DesignerView->SetPreviewAreaSize((int32)AreaSize.X, (int32)AreaSize.Y);

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}
	// End SWidget interface

private:
	bool bResizing;
	TWeakPtr<SDesignerView> Designer;
	FVector2D AbsoluteOffset;
};



struct FWidgetHitResult
{
public:
	FWidgetReference Widget;
	FArrangedWidget WidgetArranged;

	UNamedSlot* NamedSlot;
	FArrangedWidget NamedSlotArranged;

public:
	FWidgetHitResult()
		: WidgetArranged(SNullWidget::NullWidget, FGeometry())
		, NamedSlotArranged(SNullWidget::NullWidget, FGeometry())
	{
	}
};

//////////////////////////////////////////////////////////////////////////

UWidget* SDesignerView::GetWidgetInDesignScopeFromSlateWidget(TSharedRef<SWidget>& InWidget)
{
	TSharedPtr<FReflectionMetaData> ReflectionMetadata = InWidget->GetMetaData<FReflectionMetaData>();
	if ( ReflectionMetadata.IsValid() )
	{
		if ( UObject* SourceWidget = ReflectionMetadata->SourceObject.Get() )
		{
			// The first UUserWidget outer of the source widget should be equal to the PreviewWidget for
			// it to be part of the scope of the design area we're dealing with.
			if ( SourceWidget->GetTypedOuter<UUserWidget>() == PreviewWidget )
			{
				return Cast<UWidget>(SourceWidget);
			}
		}
	}

	return nullptr;
}


/////////////////////////////////////////////////////
// SDesignerView
 
const FString SDesignerView::ConfigSectionName = "UMGEditor.Designer";
const FString SDesignerView::DefaultPreviewOverrideName = "";

void SDesignerView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	ScopedTransaction = nullptr;

	PreviewWidget = nullptr;
	BlueprintEditor = InBlueprintEditor;

	TransformMode = ETransformMode::Layout;

	bShowResolutionOutlines = false;

	HeightReadFromSettings = 0;
	WidthReadFromSettings = 0;
	SetStartupResolution();

	CachedPreviewDesiredSize = FVector2D(0, 0);

	ResolutionTextFade = FCurveSequence(0.0f, 1.0f);
	ResolutionTextFade.Play(this->AsShared());

	HoveredWidgetOutlineFade = FCurveSequence(0.0f, 0.15f);

	SelectedWidgetContextMenuLocation = FVector2D(0, 0);

	bMovingExistingWidget = false;

	RegisterExtensions();

	GEditor->OnBlueprintReinstanced().AddRaw(this, &SDesignerView::OnPreviewNeedsRecreation);

	BindCommands();

	SDesignSurface::Construct(SDesignSurface::FArguments()
		.AllowContinousZoomInterpolation(false)
		.Content()
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0f)
			.FillRow(1, 1.0f)

			// Corner
			+ SGridPanel::Slot(0, 0)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				.BorderBackgroundColor(FLinearColor(FColor(48, 48, 48)))
			]

			// Top Ruler
			+ SGridPanel::Slot(1, 0)
			[
				SAssignNew(TopRuler, SRuler)
				.Orientation(Orient_Horizontal)
				.Visibility(this, &SDesignerView::GetRulerVisibility)
			]

			// Side Ruler
			+ SGridPanel::Slot(0, 1)
			[
				SAssignNew(SideRuler, SRuler)
				.Orientation(Orient_Vertical)
				.Visibility(this, &SDesignerView::GetRulerVisibility)
			]

			// Designer content area
			+ SGridPanel::Slot(1, 1)
			[
				SAssignNew(PreviewHitTestRoot, SOverlay)
				.Visibility(EVisibility::Visible)
				.Clipping(EWidgetClipping::ClipToBoundsAlways)

				// The bottom layer of the overlay where the actual preview widget appears.
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SZoomPan)
					.Visibility(EVisibility::HitTestInvisible)
					.ZoomAmount(this, &SDesignerView::GetZoomAmount)
					.ViewOffset(this, &SDesignerView::GetViewOffset)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBorder)
							.Padding(FMargin(0))
							.BorderImage(this, &SDesignerView::GetPreviewBackground)
							[
								SAssignNew(PreviewAreaConstraint, SBox)
								.WidthOverride(this, &SDesignerView::GetPreviewAreaWidth)
								.HeightOverride(this, &SDesignerView::GetPreviewAreaHeight)
								[
									SAssignNew(PreviewSurface, SDPIScaler)
									.DPIScale(this, &SDesignerView::GetPreviewDPIScale)
									[
										SAssignNew(PreviewSizeConstraint, SBox)
										.WidthOverride(this, &SDesignerView::GetPreviewSizeWidth)
										.HeightOverride(this, &SDesignerView::GetPreviewSizeHeight)
									]
								]
							]
						]
					]
				]

				// A layer in the overlay where we draw effects, like the highlight effects.
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(EffectsLayer, SPaintSurface)
					.OnPaintHandler(this, &SDesignerView::HandleEffectsPainting)
					.AddMetaData<FInvisibleToWidgetReflectorMetaData>(FInvisibleToWidgetReflectorMetaData())
				]

				// 
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(DesignerWidgetCanvas, SCanvas)
					.Visibility(EVisibility::SelfHitTestInvisible)

					+ SCanvas::Slot()
					.Size(FVector2D(20, 20))
					.Position(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(this, &SDesignerView::GetAreaResizeHandlePosition)))
					[
						SNew(SResizeDesignerHandle, SharedThis(this))
						.Visibility(this, &SDesignerView::GetAreaResizeHandleVisibility)
					]
				]

				// A layer in the overlay where we put all the user intractable widgets, like the reorder widgets.
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ExtensionWidgetCanvas, SCanvas)
					.Visibility(this, &SDesignerView::GetExtensionCanvasVisibility)
				]

				// Designer overlay UI, toolbar, status messages, zoom level...etc
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateOverlayUI()
				]
			]
		]
	);

	InBlueprintEditor->OnSelectedWidgetsChanged.AddSP(this, &SDesignerView::OnEditorSelectionChanged);
	InBlueprintEditor->OnHoveredWidgetSet.AddSP(this, &SDesignerView::OnHoveredWidgetSet);
	InBlueprintEditor->OnHoveredWidgetCleared.AddSP(this, &SDesignerView::OnHoveredWidgetCleared);
	InBlueprintEditor->OnWidgetPreviewUpdated.AddSP(this, &SDesignerView::OnPreviewNeedsRecreation);
	InBlueprintEditor->OnSelectedAnimationChanged.AddSP(this, &SDesignerView::OnSelectedAnimationChanged);

	if (const TSharedPtr<ISequencer>& Sequencer = InBlueprintEditor->GetSequencer())
	{
		Sequencer->OnViewportSelectionLimitedChanged().AddSP(this, &SDesignerView::OnSelectionLimitedChanged);
	}

	DesignerHittestGrid = MakeShared<FHittestGrid>();

	ZoomToFit(/*bInstantZoom*/ true);

	FCoreDelegates::OnSafeFrameChangedEvent.AddSP(this, &SDesignerView::SwapSafeZoneTypes);
	//RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SDesignerView::EnsureTick));
}

EVisibility SDesignerView::GetExtensionCanvasVisibility() const
{
	// If any selected widgets are hidden, then don't show widget extensions.
	// If we want to support extensions on mixed-visibility in the future,
	// every existing widget extension will probably need to be updated, as
	// most do not check widget visibility before performing their function.
	for (const FWidgetReference& Widget : GetSelectedWidgets())
	{
		UWidget* Preview = Widget.GetPreview();
		if (!Preview || !Preview->IsVisibleInDesigner())
		{
			return EVisibility::Hidden;
		}
	}
	return EVisibility::SelfHitTestInvisible;
}

EActiveTimerReturnType SDesignerView::EnsureTick(double InCurrentTime, float InDeltaTime)
{
	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SDesignerView::CreateOverlayUI()
{
	const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar");

	return SNew(SOverlay)

	// Outline and text for important state.
	+ SOverlay::Slot()
	.Padding(0.0f)
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		.Visibility(this, &SDesignerView::GetDesignerOutlineVisibility)

		// Top-right corner text indicating PIE is active
		+ SOverlay::Slot()
		.Padding(0.0f)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SNew(SImage)
			.ColorAndOpacity(this, &SDesignerView::GetDesignerOutlineColor)
			.Image(FAppStyle::GetBrush(TEXT("UMGEditor.DesignerMessageBorder")))
		]

		// Top-right corner text indicating PIE is active
		+ SOverlay::Slot()
		.Padding(20.0f)
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Graph.SimulatingText")
			.ColorAndOpacity(this, &SDesignerView::GetDesignerOutlineColor)
			.Text(this, &SDesignerView::GetDesignerOutlineText)
		]
	]

	// Top bar with buttons for changing the designer
	+ SOverlay::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(6.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("Graph.ZoomText"))
				.Text(this, &SDesignerView::GetZoomText)
				.ColorAndOpacity(this, &SDesignerView::GetZoomTextColorAndOpacity)
				.Visibility(EVisibility::SelfHitTestInvisible)
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(40.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
			.Font(FCoreStyle::GetDefaultFontStyle(TEXT("BoldCondensed"), 14))
			.Text(this, &SDesignerView::GetCursorPositionText)
			.ColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.25f))
			.Visibility(this, &SDesignerView::GetCursorPositionTextVisibility)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(40.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
			.Font(FCoreStyle::GetDefaultFontStyle(TEXT("BoldCondensed"), 14))
			.Text(this, &SDesignerView::GetSelectedWidgetDimensionsText)
			.ColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.25f))
			.Visibility(this, &SDesignerView::GetSelectedWidgetDimensionsVisibility)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Top)
		[
			SNew(SSpacer)
			.Size(FVector2D(1.0f, 1.0f))
		]

		+ SHorizontalBox::Slot()
		.Padding(0.0f, 1.0f)
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SDesignerToolBar)
			.CommandList(CommandList)
		]
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 1.0f)
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.ToolTipText(LOCTEXT("ZoomToFit_ToolTip", "Zoom To Fit"))
			.OnClicked(this, &SDesignerView::HandleZoomToFitClicked)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("UMGEditor.ZoomToFit"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.ToolTipText(LOCTEXT("SwapAspectRatio_ToolTip", "Switch between Landscape and Portrait"))
			.OnClicked(this, &SDesignerView::HandleSwapAspectRatioClicked)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.IsEnabled(this, &SDesignerView::GetAspectRatioSwitchEnabled)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SDesignerView::GetAspectRatioSwitchImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.ToolTipText(LOCTEXT("Mirror_ToolTip", "Flip the current safe zones"))
			.OnClicked(this, &SDesignerView::HandleFlipSafeZonesClicked)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.IsEnabled(this, &SDesignerView::GetFlipDeviceEnabled)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("UMGEditor.Mirror"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Preview Screen Size
		+ SHorizontalBox::Slot()
		.Padding(2.0f,0.0f)
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SComboButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.OnGetMenuContent(this, &SDesignerView::GetResolutionsMenu)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ScreenSize", "Screen Size"))
				.ToolTipText(LOCTEXT("ScreenSize_ToolTip", "Change the size of the designer canvas to the screen resolution."))
				.TextStyle(&ToolBarStyle.LabelStyle)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Screen Fill Size Rule
		+ SHorizontalBox::Slot()
		.Padding(2.0f, 0.0f)
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SComboButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.OnGetMenuContent(this, &SDesignerView::GetScreenSizingFillMenu)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SDesignerView::GetScreenSizingFillText)
				.ToolTipText(LOCTEXT("ScreenFill_ToolTip", "Adjust how the widget content fills the selected Screen Size."))
				.TextStyle(&ToolBarStyle.LabelStyle)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(this, &SDesignerView::GetCustomResolutionEntryVisibility)
			.Text(LOCTEXT("Width", "Width"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.Delta(1)
			.MinSliderValue(1)
			.MinValue(0)
			.MaxSliderValue(TOptional<int32>(10000))
			.Value(this, &SDesignerView::GetCustomResolutionWidth)
			.OnValueChanged(this, &SDesignerView::OnCustomResolutionWidthChanged)
			.Visibility(this, &SDesignerView::GetCustomResolutionEntryVisibility)
			.MinDesiredValueWidth(50.0f)
			.ToolTipText(LOCTEXT("CustomSize_WidthTooltip", "1+\tSets the width of the widget in the designer.\n0\tThe width will match the desired width of the widget."))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(this, &SDesignerView::GetCustomResolutionEntryVisibility)
			.Text(LOCTEXT("Height", "Height"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.Delta(1)
			.MinSliderValue(1)
			.MaxSliderValue(TOptional<int32>(10000))
			.MinValue(0)
			.Value(this, &SDesignerView::GetCustomResolutionHeight)
			.OnValueChanged(this, &SDesignerView::OnCustomResolutionHeightChanged)
			.Visibility(this, &SDesignerView::GetCustomResolutionEntryVisibility)
			.MinDesiredValueWidth(50)
			.ToolTipText(LOCTEXT("CustomSize_HeightTooltip", "1+\tSets the height of the widget in the designer.\n0\tThe height will match the desired height of the widget."))
		]
	]

	// Info Bar, displays heads up information about some actions.
	+ SOverlay::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Bottom)
	[
		SNew(SDisappearingBar)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FLinearColor(0.10f, 0.10f, 0.10f, 0.75f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 5.0f))
			.Visibility(this, &SDesignerView::GetInfoBarVisibility)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDesignerView::GetInfoBarText)
			]
		]
	]

	// Bottom bar to show current resolution & AR
	+ SOverlay::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Bottom)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(6.0f, 0.0f, 0.0f, 2.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Visibility(this, &SDesignerView::GetResolutionTextVisibility)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDesignerView::GetCurrentScaleFactorText)
				.ColorAndOpacity(this, &SDesignerView::GetResolutionTextColorAndOpacity)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Visibility(this, &SDesignerView::GetResolutionTextVisibility)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDesignerView::GetCurrentSafeZoneText)
				.ColorAndOpacity(this, &SDesignerView::GetResolutionTextColorAndOpacity)
			]
			+SVerticalBox::Slot()
			[			
				SNew(STextBlock)
				.Visibility(this, &SDesignerView::GetResolutionTextVisibility)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDesignerView::GetCurrentResolutionText)
				.ColorAndOpacity(this, &SDesignerView::GetResolutionTextColorAndOpacity)
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		.Padding(0.0f, 0.0f, 6.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Bottom)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDesignerView::GetCurrentDPIScaleText)
				.ColorAndOpacity(this, &SDesignerView::GetCurrentDPIScaleColor)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Bottom)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ContentPadding(FMargin(3, 1))
				.OnClicked(this, &SDesignerView::HandleDPISettingsClicked)
				.ToolTipText(LOCTEXT("DPISettingsTooltip", "Configure the UI Scale Curve to control how the UI is scaled on different resolutions."))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("UMGEditor.DPISettings"))
				]
			]
		]
	];
}

SDesignerView::~SDesignerView()
{
	for (const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions)
	{
		Ext->Uninitialize();
	}
	DesignerExtensions.Reset();

	UWidgetBlueprint* Blueprint = GetBlueprint();
	if ( Blueprint )
	{
		Blueprint->OnChanged().RemoveAll(this);
		Blueprint->OnCompiled().RemoveAll(this);
	}

	if ( BlueprintEditor.IsValid() )
	{
		auto PinnedEditor = BlueprintEditor.Pin();
		PinnedEditor->OnSelectedWidgetsChanged.RemoveAll(this);
		PinnedEditor->OnHoveredWidgetSet.RemoveAll(this);
		PinnedEditor->OnHoveredWidgetCleared.RemoveAll(this);
		PinnedEditor->OnWidgetPreviewUpdated.RemoveAll(this);
		PinnedEditor->OnSelectedAnimationChanged.RemoveAll(this);

		if (const TSharedPtr<ISequencer> Sequencer = PinnedEditor->GetSequencer())
		{
			Sequencer->OnViewportSelectionLimitedChanged().RemoveAll(this);
		}
	}

	if ( GEditor )
	{
		GEditor->OnBlueprintReinstanced().RemoveAll(this);
	}
}

void SDesignerView::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);

	const FDesignerCommands& Commands = FDesignerCommands::Get();

	CommandList->MapAction(
		Commands.LayoutTransform,
		FExecuteAction::CreateSP(this, &SDesignerView::SetTransformMode, ETransformMode::Layout),
		FCanExecuteAction::CreateSP(this, &SDesignerView::CanSetTransformMode, ETransformMode::Layout),
		FIsActionChecked::CreateSP(this, &SDesignerView::IsTransformModeActive, ETransformMode::Layout)
	);

	CommandList->MapAction(
		Commands.RenderTransform,
		FExecuteAction::CreateSP(this, &SDesignerView::SetTransformMode, ETransformMode::Render),
		FCanExecuteAction::CreateSP(this, &SDesignerView::CanSetTransformMode, ETransformMode::Render),
		FIsActionChecked::CreateSP(this, &SDesignerView::IsTransformModeActive, ETransformMode::Render)
	);

	CommandList->MapAction(
		Commands.ToggleOutlines,
		FExecuteAction::CreateSP(this, &SDesignerView::ToggleShowingOutlines),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDesignerView::IsShowingOutlines)
	);

	CommandList->MapAction(
		Commands.ToggleRespectLocks,
		FExecuteAction::CreateSP(this, &SDesignerView::ToggleRespectingLocks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SDesignerView::IsRespectingLocks)
	);

	CommandList->MapAction(
		FSequencerCommands::Get().ToggleLimitViewportSelection,
		FExecuteAction::CreateSP(this, &SDesignerView::ToggleViewportSelectionLimited),
		FCanExecuteAction::CreateSP(this, &SDesignerView::CanToggleViewportSelectionLimited),
		FIsActionChecked::CreateSP(this, &SDesignerView::IsViewportSelectionLimited)
	);
}

void SDesignerView::AddReferencedObjects(FReferenceCollector& Collector)
{
	if ( PreviewWidget )
	{
		Collector.AddReferencedObject(PreviewWidget);
	}

	for (auto& DropPreview : DropPreviews)
	{
		if (DropPreview.Widget)
		{
			Collector.AddReferencedObject(DropPreview.Widget);
		}
		if (DropPreview.Parent)
		{
			Collector.AddReferencedObject(DropPreview.Parent);
		}
	}
}

void SDesignerView::SetTransformMode(ETransformMode::Type InTransformMode)
{
	if ( !InTransaction() )
	{
		TransformMode = InTransformMode;
	}
}

bool SDesignerView::CanSetTransformMode(ETransformMode::Type InTransformMode) const
{
	return true;
}

bool SDesignerView::IsTransformModeActive(ETransformMode::Type InTransformMode) const
{
	return TransformMode == InTransformMode;
}

void SDesignerView::ToggleShowingOutlines()
{
	TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();

	Editor->SetShowDashedOutlines(!Editor->GetShowDashedOutlines());
	Editor->InvalidatePreview();
}

bool SDesignerView::IsShowingOutlines() const
{
	return BlueprintEditor.Pin()->GetShowDashedOutlines();
}

void SDesignerView::ToggleRespectingLocks()
{
	TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();

	Editor->SetIsRespectingLocks(!Editor->GetIsRespectingLocks());
}

bool SDesignerView::IsRespectingLocks() const
{
	return BlueprintEditor.Pin()->GetIsRespectingLocks();
}

void SDesignerView::SetStartupResolution()
{
	// Whether the user selected a common resolution.
	if (!GConfig->GetBool(*ConfigSectionName, TEXT("bCommonResolutionSelected"), bCommonResolutionSelected, GEditorPerProjectIni))
	{
		GConfig->SetBool(*ConfigSectionName, TEXT("bCommonResolutionSelected"), false, GEditorPerProjectIni);
		bCommonResolutionSelected = false;
	}
	// Use user-set resolution
	const UWidgetDesignerSettings* DesignerSettings = GetDefault<const UWidgetDesignerSettings>();
	const FUintVector2 DefaultPreviewResolution = DesignerSettings->DefaultPreviewResolution;
	// Width
	if (!GConfig->GetInt(*ConfigSectionName, TEXT("PreviewWidth"), PreviewWidth, GEditorPerProjectIni) || !bCommonResolutionSelected)
	{
		GConfig->SetInt(*ConfigSectionName, TEXT("PreviewWidth"), DefaultPreviewResolution.X, GEditorPerProjectIni);
		PreviewWidth = DefaultPreviewResolution.X;
	}
	// Initially assign WidthReadFromSettings to PreviewWidth
	WidthReadFromSettings = PreviewWidth;
	// Height
	PreviewOverrideName = DefaultPreviewOverrideName;
	if (!GConfig->GetInt(*ConfigSectionName, TEXT("PreviewHeight"), PreviewHeight, GEditorPerProjectIni) || !bCommonResolutionSelected)
	{
		GConfig->SetInt(*ConfigSectionName, TEXT("PreviewHeight"), DefaultPreviewResolution.Y, GEditorPerProjectIni);
		PreviewHeight = DefaultPreviewResolution.Y;
	}
	// Initially assign HeightReadFromSettings to PreviewHeight
	HeightReadFromSettings = PreviewHeight;
	// Aspect Ratio
	if (!GConfig->GetString(*ConfigSectionName, TEXT("PreviewAspectRatio"), PreviewAspectRatio, GEditorPerProjectIni) || !bCommonResolutionSelected)
	{
		const int32 GCD = FMath::GreatestCommonDivisor(PreviewWidth, PreviewHeight);
		PreviewAspectRatio = FString::Printf(TEXT("%d:%d"), PreviewWidth / GCD, PreviewHeight / GCD);
		GConfig->SetString(*ConfigSectionName, TEXT("PreviewAspectRatio"), *PreviewAspectRatio, GEditorPerProjectIni);
	}
	// Portrait Mode
	if (!GConfig->GetBool(*ConfigSectionName, TEXT("bIsInPortraitMode"), bPreviewIsPortrait, GEditorPerProjectIni))
	{
		GConfig->SetBool(*ConfigSectionName, TEXT("bIsInPortraitMode"), false, GEditorPerProjectIni);
		bPreviewIsPortrait = false;
	}
	// Profile Type
	if (!GConfig->GetString(*ConfigSectionName, TEXT("ProfileName"), PreviewOverrideName, GEditorPerProjectIni))
	{
		GConfig->SetString(*ConfigSectionName, TEXT("ProfileName"), *DefaultPreviewOverrideName, GEditorPerProjectIni);
		PreviewOverrideName = DefaultPreviewOverrideName;
	}
	// Scale factor
	if (!GConfig->GetFloat(*ConfigSectionName, TEXT("ScaleFactor"), ScaleFactor, GEditorPerProjectIni))
	{
		GConfig->SetFloat(*ConfigSectionName, TEXT("ScaleFactor"), 1.0f, GEditorPerProjectIni);
		ScaleFactor = 1.0f;
	}
	if (!GConfig->GetBool(*ConfigSectionName, TEXT("bCanPreviewSwapAspectRatio"), bCanPreviewSwapAspectRatio, GEditorPerProjectIni))
	{
		GConfig->SetBool(*ConfigSectionName, TEXT("bCanPreviewSwapAspectRatio"), false, GEditorPerProjectIni);
		bCanPreviewSwapAspectRatio = false;
	}

	if (!PreviewOverrideName.IsEmpty())
	{
		ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
		DesignerSafeZoneOverride = PlaySettings->CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
	}
	else
	{
		FSlateApplication::Get().ResetCustomSafeZone();
		FSlateApplication::Get().GetSafeZoneSize(DesignerSafeZoneOverride, FVector2D(PreviewWidth, PreviewHeight));
	}
	FMargin SafeZoneRatio = DesignerSafeZoneOverride;
	SafeZoneRatio.Left /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Right /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Bottom /= (PreviewHeight / 2.0f);
	SafeZoneRatio.Top /= (PreviewHeight / 2.0f);
	FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, true);

}

float SDesignerView::GetPreviewScale() const
{
	return GetZoomAmount() * GetPreviewDPIScale();
}

const TSet<FWidgetReference>& SDesignerView::GetSelectedWidgets() const
{
	return BlueprintEditor.Pin()->GetSelectedWidgets();
}

FWidgetReference SDesignerView::GetSelectedWidget() const
{
	const TSet<FWidgetReference>& SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();

	// Only return a selected widget when we have only a single item selected.
	if ( SelectedWidgets.Num() == 1 )
	{
		TSet<FWidgetReference>::TConstIterator SetIt(SelectedWidgets);
		return *SetIt;
	}

	return FWidgetReference();
}

ETransformMode::Type SDesignerView::GetTransformMode() const
{
	return TransformMode;
}

FOptionalSize SDesignerView::GetPreviewAreaWidth() const
{
	TTuple<FVector2D, FVector2D> AreaAndSize = FWidgetBlueprintEditorUtils::GetWidgetPreviewAreaAndSize(GetDefaultWidget(), CachedPreviewDesiredSize, FVector2D(PreviewWidth, PreviewHeight), GetDefaultWidget()->DesignSizeMode, TOptional<FVector2D>());
	FVector2D Area = AreaAndSize.Get<0>();

	return static_cast<float>(Area.X);
}

FOptionalSize SDesignerView::GetPreviewAreaHeight() const
{
	TTuple<FVector2D, FVector2D> AreaAndSize = FWidgetBlueprintEditorUtils::GetWidgetPreviewAreaAndSize(GetDefaultWidget(), CachedPreviewDesiredSize, FVector2D(PreviewWidth, PreviewHeight), GetDefaultWidget()->DesignSizeMode, TOptional<FVector2D>());
	FVector2D Area = AreaAndSize.Get<0>();;

	return static_cast<float>(Area.Y);
}

FOptionalSize SDesignerView::GetPreviewSizeWidth() const
{
	TTuple<FVector2D, FVector2D> AreaAndSize = FWidgetBlueprintEditorUtils::GetWidgetPreviewAreaAndSize(GetDefaultWidget(), CachedPreviewDesiredSize, FVector2D(PreviewWidth, PreviewHeight), GetDefaultWidget()->DesignSizeMode, TOptional<FVector2D>());
	FVector2D Size = AreaAndSize.Get<1>();

	return static_cast<float>(Size.X);
}

FOptionalSize SDesignerView::GetPreviewSizeHeight() const
{
	TTuple<FVector2D, FVector2D> AreaAndSize = FWidgetBlueprintEditorUtils::GetWidgetPreviewAreaAndSize(GetDefaultWidget(), CachedPreviewDesiredSize, FVector2D(PreviewWidth, PreviewHeight), GetDefaultWidget()->DesignSizeMode, TOptional<FVector2D>());
	FVector2D Size = AreaAndSize.Get<1>();

	return static_cast<float>(Size.Y);
}

void SDesignerView::BeginResizingArea()
{
	bDrawGridLines = false;
	bShowResolutionOutlines = true;
}

void SDesignerView::EndResizingArea()
{
	bDrawGridLines = true;
	bShowResolutionOutlines = false;
}

const UWidgetEditingProjectSettings* SDesignerView::GetRelevantSettings() const
{
	if (UWidgetBlueprint* WidgetBlueprint = GetBlueprint())
	{
		return WidgetBlueprint->GetRelevantSettings();
	}
	// Default to the UMG Editor project settings
	return GetDefault<UUMGEditorProjectSettings>();
}

void SDesignerView::SetPreviewAreaSize(int32 Width, int32 Height)
{
	if (UUserWidget* DefaultWidget = GetDefaultWidget())
	{
		Width = FMath::Max(Width, 1);
		Height = FMath::Max(Height, 1);

		switch (DefaultWidget->DesignSizeMode)
		{
			case EDesignPreviewSizeMode::Custom:
			case EDesignPreviewSizeMode::CustomOnScreen:
			{
				DefaultWidget->DesignTimeSize = FVector2D(Width, Height);
				break;
			}
			default:
			{
				int32 GCD = FMath::GreatestCommonDivisor(Width, Height);

				PreviewWidth = Width;
				PreviewHeight = Height;
				PreviewAspectRatio = FString::Printf(TEXT("%d:%d"), Width / GCD, Height / GCD);

				const bool bSaveChanges = false;
				if (bSaveChanges)
				{
					GConfig->SetInt(*ConfigSectionName, TEXT("PreviewWidth"), Width, GEditorPerProjectIni);
					GConfig->SetInt(*ConfigSectionName, TEXT("PreviewHeight"), Height, GEditorPerProjectIni);
					GConfig->SetString(*ConfigSectionName, TEXT("PreviewAspectRatio"), *PreviewAspectRatio, GEditorPerProjectIni);
					GConfig->SetBool(*ConfigSectionName, TEXT("bIsInPortraitMode"), bPreviewIsPortrait, GEditorPerProjectIni);
					GConfig->SetString(*ConfigSectionName, TEXT("ProfileName"), *PreviewOverrideName, GEditorPerProjectIni);
					GConfig->SetFloat(*ConfigSectionName, TEXT("ScaleFactor"), ScaleFactor, GEditorPerProjectIni);
					GConfig->SetBool(*ConfigSectionName, TEXT("bCanPreviewSwapAspectRatio"), bCanPreviewSwapAspectRatio, GEditorPerProjectIni);
					GConfig->SetBool(*ConfigSectionName, TEXT("bCommonResolutionSelected"), false, GEditorPerProjectIni);
				}
				break;
			}
		}
	}

	BroadcastDesignerChanged();

	ResolutionTextFade.Play(this->AsShared());
}

FVector2D SDesignerView::GetAreaResizeHandlePosition() const
{
	FGeometry PreviewAreaGeometry = PreviewAreaConstraint->GetTickSpaceGeometry();
	FGeometry DesignerOverlayGeometry = DesignerWidgetCanvas->GetTickSpaceGeometry();

	FVector2D AbsoluteResizeHandlePosition = PreviewAreaGeometry.LocalToAbsolute(PreviewAreaGeometry.GetLocalSize() + FVector2D(2, 2));

	return DesignerOverlayGeometry.AbsoluteToLocal(AbsoluteResizeHandlePosition);
}

EVisibility SDesignerView::GetAreaResizeHandleVisibility() const
{
	if (UUserWidget* DefaultWidget = GetDefaultWidget())
	{
		switch (DefaultWidget->DesignSizeMode)
		{
		case EDesignPreviewSizeMode::Desired:
			return EVisibility::Collapsed;
		default:
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

const FSlateBrush* SDesignerView::GetPreviewBackground() const
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		if ( DefaultWidget->PreviewBackground )
		{
			BackgroundImage.SetResourceObject(DefaultWidget->PreviewBackground);
			return &BackgroundImage;
		}
	}

	return nullptr;
}

float SDesignerView::GetPreviewDPIScale() const
{
	return FWidgetBlueprintEditorUtils::GetWidgetPreviewDPIScale(GetDefaultWidget(), FVector2D(PreviewWidth,PreviewHeight));
}

FSlateRect SDesignerView::ComputeAreaBounds() const
{
	return FSlateRect(0, 0, GetPreviewAreaWidth().Get(), GetPreviewAreaHeight().Get());
}

int32 SDesignerView::GetSnapGridSize() const
{
	const UWidgetDesignerSettings* DesignerSettings = GetDefault<UWidgetDesignerSettings>();
	return DesignerSettings->GridSnapSize;
}

int32 SDesignerView::GetGraphRulePeriod() const
{
	return 10;
}

float SDesignerView::GetGridScaleAmount() const
{
	return GetPreviewDPIScale();
}

EVisibility SDesignerView::GetInfoBarVisibility() const
{
	if ( DesignerMessageStack.Num() > 0 )
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FText SDesignerView::GetInfoBarText() const
{
	if ( DesignerMessageStack.Num() > 0 )
	{
		return DesignerMessageStack.Top();
	}

	return FText::GetEmpty();
}

void SDesignerView::PushDesignerMessage(const FText& Message)
{
	DesignerMessageStack.Push(Message);
}

void SDesignerView::PopDesignerMessage()
{
	if ( DesignerMessageStack.Num() > 0)
	{
		DesignerMessageStack.Pop();
	}
}

void SDesignerView::OnEditorSelectionChanged()
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	TSet<FWidgetReference> PendingSelectedWidgets = BPEd->GetSelectedWidgets();

	// Notify all widgets that are no longer selected.
	for ( FWidgetReference& WidgetRef : SelectedWidgetsCache )
	{
		if ( WidgetRef.IsValid() && !PendingSelectedWidgets.Contains(WidgetRef) )
		{
			WidgetRef.GetPreview()->DeselectByDesigner();
		}

		if (UWidget* WidgetTemplate = WidgetRef.GetTemplate())
		{
			// Find all named slot host widgets that are hierarchical ancestors of this widget and call deselect on them as well
			TArray<FWidgetReference> AncestorSlotHostWidgets;
			FWidgetBlueprintEditorUtils::FindAllAncestorNamedSlotHostWidgetsForContent(AncestorSlotHostWidgets, WidgetTemplate, BPEd.ToSharedRef());

			for (FWidgetReference SlotHostWidget : AncestorSlotHostWidgets)
			{
				SlotHostWidget.GetPreview()->DeselectByDesigner();
			}
		}
	}

	// Notify all widgets that are now selected.
	for ( FWidgetReference& WidgetRef : PendingSelectedWidgets )
	{
		if ( WidgetRef.IsValid() && !SelectedWidgetsCache.Contains(WidgetRef) )
		{
			WidgetRef.GetPreview()->SelectByDesigner();

			if (UWidget* WidgetTemplate = WidgetRef.GetTemplate())
			{
				// Find all named slot host widgets that are hierarchical ancestors of this widget and call select on them as well
				TArray<FWidgetReference> AncestorSlotHostWidgets;
				FWidgetBlueprintEditorUtils::FindAllAncestorNamedSlotHostWidgetsForContent(AncestorSlotHostWidgets, WidgetTemplate, BPEd.ToSharedRef());

				for (FWidgetReference SlotHostWidget : AncestorSlotHostWidgets)
				{
					SlotHostWidget.GetPreview()->SelectByDesigner();
				}
			}
		}
	}

	SelectedWidgetsCache = PendingSelectedWidgets;

	CreateExtensionWidgetsForSelection();
}

void SDesignerView::OnHoveredWidgetSet(const FWidgetReference& InHoveredWidget)
{
	HoveredWidgetOutlineFade.Play(this->AsShared());
}

void SDesignerView::OnHoveredWidgetCleared()
{
	HoveredWidgetOutlineFade.JumpToEnd();
}

FGeometry SDesignerView::GetDesignerGeometry() const
{
	return PreviewHitTestRoot->GetTickSpaceGeometry();
}

FVector2D SDesignerView::GetWidgetOriginAbsolute() const
{
	if (PreviewWidget)
	{
		FGeometry Geometry;
		if (GetWidgetGeometry(PreviewWidget, Geometry))
		{
			return FVector2D(Geometry.AbsolutePosition);
		}
	}

	return FVector2D::ZeroVector;
}

void SDesignerView::MarkDesignModifed(bool bRequiresRecompile)
{
	if ( bRequiresRecompile )
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	}
}

bool SDesignerView::GetWidgetParentGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const
{
	if ( UWidget* WidgetPreview = Widget.GetPreview() )
	{
		if ( UPanelWidget* Parent = WidgetPreview->GetParent() )
		{
			return GetWidgetGeometry(Parent, Geometry);
		}
	}

	Geometry = GetDesignerGeometry();
	return true;
}

bool SDesignerView::GetWidgetGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const
{
	if ( const UWidget* WidgetPreview = Widget.GetPreview() )
	{
		return GetWidgetGeometry(WidgetPreview, Geometry);
	}

	return false;
}

bool SDesignerView::GetWidgetGeometry(const UWidget* InPreviewWidget, FGeometry& Geometry) const
{
	if (TSharedPtr<SWidget> CachedPreviewWidget = InPreviewWidget->GetCachedWrappedWidget())
	{
		const FArrangedWidget* ArrangedWidget = CachedWidgetGeometry.Find(CachedPreviewWidget.ToSharedRef());
		if ( ArrangedWidget )
		{
			Geometry = ArrangedWidget->Geometry;
			return true;
		}
	}

	return false;
}

FGeometry SDesignerView::MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const
{
	FGeometry NewGeometry = WidgetGeometry;

	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if ( WidgetWindow.IsValid() )
	{
		TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

		NewGeometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
	}

	return NewGeometry;
}

void SDesignerView::ClearExtensionWidgets()
{
	ExtensionWidgetCanvas->ClearChildren();
}

void SDesignerView::CreateExtensionWidgetsForSelection()
{
	// Remove all the current extension widgets
	ClearExtensionWidgets();

	// Get the selected widgets as an array
	TArray<FWidgetReference> Selected = GetSelectedWidgets().Array();
	
	TArray< TSharedRef<FDesignerSurfaceElement> > ExtensionElements;

	if ( Selected.Num() > 0 )
	{
		const float Offset = 10;

		// Add transform handles
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::TopLeft), EExtensionLayoutLocation::TopLeft, FVector2D(-Offset, -Offset))));
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::TopCenter), EExtensionLayoutLocation::TopCenter, FVector2D(0, -Offset))));
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::TopRight), EExtensionLayoutLocation::TopRight, FVector2D(Offset, -Offset))));

		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::CenterLeft), EExtensionLayoutLocation::CenterLeft, FVector2D(-Offset, 0))));
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::CenterRight), EExtensionLayoutLocation::CenterRight, FVector2D(Offset, 0))));

		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::BottomLeft), EExtensionLayoutLocation::BottomLeft, FVector2D(-Offset, Offset))));
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::BottomCenter), EExtensionLayoutLocation::BottomCenter, FVector2D(0, Offset))));
		ExtensionElements.Add(MakeShareable(new FDesignerSurfaceElement(SNew(STransformHandle, this, ETransformDirection::BottomRight), EExtensionLayoutLocation::BottomRight, FVector2D(Offset, Offset))));

		// Build extension widgets for new selection
		for ( TSharedRef<FDesignerExtension>& Ext : DesignerExtensions )
		{
			if ( Ext->CanExtendSelection(Selected) )
			{
				Ext->ExtendSelection(Selected, ExtensionElements);
			}
		}

		// Add Widgets to designer surface
		for ( TSharedRef<FDesignerSurfaceElement>& ExtElement : ExtensionElements )
		{
			ExtensionWidgetCanvas->AddSlot()
				.Position(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(this, &SDesignerView::GetExtensionPosition, ExtElement)))
				.Size(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(this, &SDesignerView::GetExtensionSize, ExtElement)))
				[
					ExtElement->GetWidget()
				];
		}
	}
}

FVector2D SDesignerView::GetExtensionPosition(TSharedRef<FDesignerSurfaceElement> ExtensionElement) const
{
	FWidgetReference SelectedWidget = GetSelectedWidget();

	if ( SelectedWidget.IsValid() )
	{
		FGeometry SelectedWidgetGeometry;
		FGeometry SelectedWidgetParentGeometry;

		if ( GetWidgetGeometry(SelectedWidget, SelectedWidgetGeometry) && GetWidgetParentGeometry(SelectedWidget, SelectedWidgetParentGeometry) )
		{
			const FVector2D ParentPostion_DesignerSpace = FVector2D(SelectedWidgetParentGeometry.AbsolutePosition - GetDesignerGeometry().AbsolutePosition) / GetDesignerGeometry().Scale;
			const FVector2D ParentSize = SelectedWidgetParentGeometry.Size * GetPreviewScale();

			FVector2D FinalPosition(0, 0);

			if (ExtensionElement->GetLocation() == EExtensionLayoutLocation::RelativeFromParent)
			{
				FinalPosition = GetDesignerGeometry().AbsoluteToLocal(SelectedWidgetParentGeometry.LocalToAbsolute(FVector2D(0, 0)));
				FinalPosition += ExtensionElement->GetOffset();
			}
			else
			{
				FVector2D WidgetPosition = FVector2D::ZeroVector;

				// Get the initial offset based on the location around the selected object.
				switch (ExtensionElement->GetLocation())
				{
				case EExtensionLayoutLocation::TopLeft:
					WidgetPosition = FVector2D(0, 0);
					break;
				case EExtensionLayoutLocation::TopCenter:
					WidgetPosition = FVector2D(SelectedWidgetGeometry.GetLocalSize().X * 0.5f, 0);
					break;
				case EExtensionLayoutLocation::TopRight:
					WidgetPosition = FVector2D(SelectedWidgetGeometry.GetLocalSize().X, 0);
					break;

				case EExtensionLayoutLocation::CenterLeft:
					WidgetPosition = FVector2D(0, SelectedWidgetGeometry.GetLocalSize().Y * 0.5f);
					break;
				case EExtensionLayoutLocation::CenterCenter:
					WidgetPosition = FVector2D(SelectedWidgetGeometry.GetLocalSize().X * 0.5f, SelectedWidgetGeometry.GetLocalSize().Y * 0.5f);
					break;
				case EExtensionLayoutLocation::CenterRight:
					WidgetPosition = FVector2D(SelectedWidgetGeometry.GetLocalSize().X, SelectedWidgetGeometry.GetLocalSize().Y * 0.5f);
					break;

				case EExtensionLayoutLocation::BottomLeft:
					WidgetPosition = FVector2D(0, SelectedWidgetGeometry.GetLocalSize().Y);
					break;
				case EExtensionLayoutLocation::BottomCenter:
					WidgetPosition = FVector2D(SelectedWidgetGeometry.GetLocalSize().X * 0.5f, SelectedWidgetGeometry.GetLocalSize().Y);
					break;
				case EExtensionLayoutLocation::BottomRight:
					WidgetPosition = SelectedWidgetGeometry.GetLocalSize();
					break;
				}

				FVector2D SelectedWidgetScale = FVector2D(SelectedWidgetGeometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());

				FVector2D ApplicationScaledOffset = ExtensionElement->GetOffset() * GetDesignerGeometry().Scale;

				FVector2D LocalOffsetFull = ApplicationScaledOffset / SelectedWidgetScale;
				FVector2D PositionFullOffset = GetDesignerGeometry().AbsoluteToLocal(SelectedWidgetGeometry.LocalToAbsolute(WidgetPosition + LocalOffsetFull));
				FVector2D LocalOffsetHalf = (ApplicationScaledOffset / 2.0f) / SelectedWidgetScale;
				FVector2D PositionHalfOffset = GetDesignerGeometry().AbsoluteToLocal(SelectedWidgetGeometry.LocalToAbsolute(WidgetPosition + LocalOffsetHalf));

				FVector2D PivotCorrection = PositionHalfOffset - (PositionFullOffset + FVector2D(5.0f, 5.0f));

				FinalPosition = PositionFullOffset + PivotCorrection;
			}

			// Add the alignment offset
			FinalPosition += ParentSize * ExtensionElement->GetAlignment();

			return FinalPosition;
		}
	}

	return FVector2D(0, 0);
}

FVector2D SDesignerView::GetExtensionSize(TSharedRef<FDesignerSurfaceElement> ExtensionElement) const
{
	return ExtensionElement->GetWidget()->GetDesiredSize();
}

void SDesignerView::ClearDropPreviews()
{
	UWidgetBlueprint* BP = GetBlueprint();
	for (const auto& DropPreview : DropPreviews)
	{
		if (DropPreview.Parent)
		{
			DropPreview.Parent->RemoveChild(DropPreview.Widget);
		}

		BP->WidgetTree->RemoveWidget(DropPreview.Widget);

		// Since the widget has been removed from the widget tree, move it into the transient package. Otherwise,
		// it will remain outered to the widget tree and end up as a property in the BP class layout as a result.
		if (DropPreview.Widget->GetOutermost() != GetTransientPackage())
		{
			DropPreview.Widget->SetFlags(RF_NoFlags);
			DropPreview.Widget->Rename(nullptr, GetTransientPackage());
		}
	}
	DropPreviews.Empty();
}

UWidgetBlueprint* SDesignerView::GetBlueprint() const
{
	if ( BlueprintEditor.IsValid() )
	{
		UBlueprint* BP = BlueprintEditor.Pin()->GetBlueprintObj();
		return Cast<UWidgetBlueprint>(BP);
	}

	return nullptr;
}

void SDesignerView::Register(TSharedRef<FDesignerExtension> Extension)
{
	if (!DesignerExtensions.Contains(Extension))
	{
		Extension->Initialize(this, GetBlueprint());
		DesignerExtensions.Add(Extension);
	}
}

void SDesignerView::Unregister(TSharedRef<FDesignerExtension> Extension)
{
	if (DesignerExtensions.Contains(Extension))
	{
		DesignerExtensions.RemoveSingle(Extension);
		Extension->Uninitialize();
	}
}

namespace DesignerView
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void RegisterDeprecatedExtensions(SDesignerView* Self, TSharedPtr<FDesignerExtensibilityManager> DesignerExtensibilityManager)
	{
		for (const TSharedRef<FDesignerExtension>& Extension : DesignerExtensibilityManager->GetExternalDesignerExtensions())
		{
			Self->Register(Extension);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void SDesignerView::RegisterExtensions()
{
	Register(MakeShareable(new FVerticalSlotExtension()));
	Register(MakeShareable(new FHorizontalSlotExtension()));
	Register(MakeShareable(new FStackBoxSlotExtension()));
	Register(MakeShareable(new FCanvasSlotExtension()));
	Register(MakeShareable(new FUniformGridSlotExtension()));
	Register(MakeShareable(new FGridSlotExtension()));

	//Register External Extensions
	IUMGEditorModule& UMGEditorInterface = FModuleManager::GetModuleChecked<IUMGEditorModule>("UMGEditor");
	TSharedPtr<FDesignerExtensibilityManager> DesignerExtensibilityManager = UMGEditorInterface.GetDesignerExtensibilityManager();

	DesignerView::RegisterDeprecatedExtensions(this, DesignerExtensibilityManager);
	for (const TSharedRef<IDesignerExtensionFactory>& ExtensionFactory : DesignerExtensibilityManager->GetExternalDesignerExtensionFactories())
	{
		Register(ExtensionFactory->CreateDesignerExtension());
	}
}

void SDesignerView::OnPreviewNeedsRecreation()
{
	// Because widget blueprints can contain other widget blueprints, the safe thing to do is to have all
	// designers jettison their previews on the compilation of any widget blueprint.  We do this to prevent
	// having slate widgets that still may reference data in their owner UWidget that has been garbage collected.
	CachedWidgetGeometry.Reset();

	PreviewWidget = nullptr;
	PreviewSizeConstraint->SetContent(SNullWidget::NullWidget);

	// Notify all designer extensions that the content has changed
	for (const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions)
	{
		Ext->PreviewContentChanged(SNullWidget::NullWidget);
	}
}

SDesignerView::FWidgetHitResult::FWidgetHitResult()
	: Widget()
	, WidgetArranged(SNullWidget::NullWidget, FGeometry())
	, NamedSlot(NAME_None)
{
}

bool SDesignerView::FindWidgetUnderCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSubclassOf<UWidget> FindType, FWidgetHitResult& HitResult)
{
	//@TODO UMG Make it so you can request dropable widgets only, to find the first parentable.

	// Query the hit test grid we create for the design surface, and determine what widgets we hit.
	TArray<FWidgetAndPointer> BubblePath = DesignerHittestGrid->GetBubblePath(MouseEvent.GetScreenSpacePosition(), 0.0f, true, INDEX_NONE);

	HitResult.Widget = FWidgetReference();
	HitResult.NamedSlot = NAME_None;

	UUserWidget* PreviewUserWidget = BlueprintEditor.Pin()->GetPreview();
	if ( PreviewUserWidget )
	{
		UWidget* WidgetUnderCursor = nullptr;

		// We loop through each hit slate widget until we arrive at one that we can access from the root widget.
		for ( int32 ChildIndex = BubblePath.Num() - 1; ChildIndex >= 0; ChildIndex-- )
		{
			FArrangedWidget& Child = BubblePath[ChildIndex];
			WidgetUnderCursor = PreviewUserWidget->GetWidgetHandle(Child.Widget);
			
			if (WidgetUnderCursor == nullptr)
			{
				continue;
			}

			// Ignore the drop preview widgets when doing widget picking
			if (DropPreviews.ContainsByPredicate([WidgetUnderCursor](const FDropPreview& Preview){ return Preview.Widget == WidgetUnderCursor; }))
			{
				WidgetUnderCursor = nullptr;
				continue;
			}

			// Ignore widgets that don't pass our find widget filter.
			if ( WidgetUnderCursor->GetClass()->IsChildOf(FindType) == false )
			{
				WidgetUnderCursor = nullptr;
				continue;
			}
			
			// We successfully found a widget that's accessible from the root.
			if ( WidgetUnderCursor )
			{
				HitResult.Widget = BlueprintEditor.Pin()->GetReferenceFromPreview(WidgetUnderCursor);
				HitResult.WidgetArranged = Child;

				if ( UUserWidget* UserWidgetUnderCursor = Cast<UUserWidget>(WidgetUnderCursor) )
				{
					// Find the named slot we're over, if any
					for ( int32 SubChildIndex = BubblePath.Num() - 1; SubChildIndex > ChildIndex; SubChildIndex-- )
					{
						FArrangedWidget& SubChild = BubblePath[SubChildIndex];
						UNamedSlot* NamedSlot = Cast<UNamedSlot>(UserWidgetUnderCursor->GetWidgetHandle(SubChild.Widget));
						if ( NamedSlot )
						{
							HitResult.NamedSlot = NamedSlot->GetFName();
							break;
						}
					}
				}

				return true;
			}
		}
	}

	return false;
}

void SDesignerView::ResolvePendingSelectedWidgets()
{
	if ( PendingSelectedWidget.IsValid() )
	{
		TSet<FWidgetReference> SelectedTemplates;
		SelectedTemplates.Add(PendingSelectedWidget);
		bool AppendOrToggle = FSlateApplication::Get().GetModifierKeys().IsControlDown() || FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		BlueprintEditor.Pin()->SelectWidgets(SelectedTemplates, AppendOrToggle);

		PendingSelectedWidget = FWidgetReference();
	}
}

FReply SDesignerView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDesignSurface::OnMouseButtonDown(MyGeometry, MouseEvent);

	//TODO UMG Undoable Selection

	bool bFoundWidgetUnderCursor = false;
	{
		// Narrow life scope of FWidgetHitResult so it doesn't keep a hard reference on any widget.
		FWidgetHitResult HitResult;
		bFoundWidgetUnderCursor = FindWidgetUnderCursor(MyGeometry, MouseEvent, UWidget::StaticClass(), HitResult);
		if (bFoundWidgetUnderCursor)
		{
			SelectedWidgetContextMenuLocation = HitResult.WidgetArranged.Geometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			PendingSelectedWidget = HitResult.Widget;
		}
	}

	if (bFoundWidgetUnderCursor)
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			if (IsSelectableInSequencer(PendingSelectedWidget.GetPreview()))
			{
				const TSet<FWidgetReference>& SelectedWidgets = GetSelectedWidgets();

				bool bResolvePendingSelectionImmediately = true;

				if (SelectedWidgets.Num() > 0)
				{
					for (const auto& SelectedWidget : SelectedWidgets)
					{
						auto PendingTemplate = PendingSelectedWidget.GetTemplate();
						auto SelectedTemplate = SelectedWidget.GetTemplate();

						if ( PendingSelectedWidget == SelectedWidget || ( PendingTemplate && SelectedTemplate && PendingTemplate->IsChildOf( SelectedTemplate ) ) )
						{
							bResolvePendingSelectionImmediately = false;
							break;
						}
					}
				}

				// If the newly clicked item is a child of the active selection, add it to the pending set of selected 
				// widgets, if they begin dragging we can just move the parent, but if it's not part of the parent set, 
				// we want to immediately begin dragging it.  Also if the currently selected widget is the root widget, 
				// we won't be moving it so just resolve immediately.
				if ( bResolvePendingSelectionImmediately )
				{
					ResolvePendingSelectedWidgets();
				}

				DraggingStartPositionScreenSpace = MouseEvent.GetScreenSpacePosition();
			}
			else
			{
				PendingSelectedWidget = FWidgetReference();
			}
		}
	}
	else
	{
		// Clear the selection immediately if we didn't click anything.
		if(MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			TSet<FWidgetReference> SelectedTemplates;
			BlueprintEditor.Pin()->SelectWidgets(SelectedTemplates, false);
		}
	}

	// Capture mouse for the drag handle and general mouse actions
	return FReply::Handled().PreventThrottling().SetUserFocus(AsShared(), EFocusCause::Mouse).CaptureMouse(AsShared());
}

FReply SDesignerView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( HasMouseCapture() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		ResolvePendingSelectedWidgets();

		bMovingExistingWidget = false;

		EndTransaction(false);
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		if ( !bIsPanning && !bIsZooming )
		{
			ResolvePendingSelectedWidgets();

			ShowContextMenu(MyGeometry, MouseEvent);
		}
	}

	SDesignSurface::OnMouseButtonUp(MyGeometry, MouseEvent);

	return FReply::Handled().ReleaseMouseCapture();
}

FReply SDesignerView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetCursorDelta().IsZero() )
	{
		return FReply::Unhandled();
	}

	CachedMousePosition = MouseEvent.GetScreenSpacePosition();

	FReply SurfaceHandled = SDesignSurface::OnMouseMove(MyGeometry, MouseEvent);
	if ( SurfaceHandled.IsEventHandled() )
	{
		return SurfaceHandled;
	}

	if ( MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && HasMouseCapture() )
	{
		const TSet<FWidgetReference>& SelectedWidgets = GetSelectedWidgets();

		if (SelectedWidgets.Num() > 0 && !bMovingExistingWidget)
		{
			if ( TransformMode == ETransformMode::Layout )
			{
				bool bIsRootWidgetSelected = false;
				for (const auto& SelectedWidget : SelectedWidgets)
				{
					UWidget* ParentWidget = SelectedWidget.GetTemplate()->GetParent();
					if (!ParentWidget || Cast<UNamedSlot>(ParentWidget))
					{
						bIsRootWidgetSelected = true;
						break;
					}
				}

				if (!bIsRootWidgetSelected)
				{
					bMovingExistingWidget = true;
					//Drag selected widgets
					return FReply::Handled().DetectDrag(AsShared(), EKeys::LeftMouseButton);
				}
			}
			else
			{
				checkSlow(TransformMode == ETransformMode::Render);
				checkSlow(bMovingExistingWidget == false);

				if (SelectedWidgets.Num() == 1)
				{
					BeginTransaction(LOCTEXT("MoveWidgetRT", "Move Widget (Render Transform)"));
				}
				else
				{
					check(SelectedWidgets.Num() > 1);
					BeginTransaction(LOCTEXT("MoveWidgetsRT", "Move Widgets (Render Transform)"));
				}

				for (const auto& SelectedWidget : SelectedWidgets)
				{
					if (UWidget* WidgetPreview = SelectedWidget.GetPreview())
					{
						FGeometry ParentGeometry;
						if (GetWidgetParentGeometry(SelectedWidget, ParentGeometry))
						{
							const FSlateRenderTransform& AbsoluteToLocalTransform = Inverse(ParentGeometry.GetAccumulatedRenderTransform());

							FWidgetTransform WidgetRenderTransform = WidgetPreview->GetRenderTransform();
							WidgetRenderTransform.Translation += TransformVector(AbsoluteToLocalTransform, MouseEvent.GetCursorDelta());

							static const FName RenderTransformName(TEXT("RenderTransform"));

							FObjectEditorUtils::SetPropertyValue<UWidget, FWidgetTransform>(WidgetPreview, RenderTransformName, WidgetRenderTransform);
							FObjectEditorUtils::SetPropertyValue<UWidget, FWidgetTransform>(SelectedWidget.GetTemplate(), RenderTransformName, WidgetRenderTransform);
						}
					}
				}
			}
		}
	}
	

	// Update the hovered widget under the mouse
	if (const TSharedPtr<FWidgetBlueprintEditor> PinnedBlueprintEditor = BlueprintEditor.Pin())
	{
		FWidgetHitResult HitResult;
		if (FindWidgetUnderCursor(MyGeometry, MouseEvent, UWidget::StaticClass(), HitResult)
			&& IsSelectableInSequencer(HitResult.Widget.GetPreview()))
		{
			PinnedBlueprintEditor->SetHoveredWidget(HitResult.Widget);
		}
		else if (PinnedBlueprintEditor->GetHoveredWidget().IsValid())
		{
			PinnedBlueprintEditor->ClearHoveredWidget();
		}
	}

	return FReply::Unhandled();
}

void SDesignerView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDesignSurface::OnMouseEnter(MyGeometry, MouseEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();
}

void SDesignerView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SDesignSurface::OnMouseLeave(MouseEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();
}

FReply SDesignerView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	UWidget* SelectedWidget = GetSelectedWidget().GetPreview();

	//If the selected widget is a canvas panel, we'd like to drop the widget right under the cursor
	//Otherwise, we'll just cascade it off the current selection
	if(SelectedWidget)
	{
		if (SelectedWidget->IsA(UPanelWidget::StaticClass()))
		{
			BlueprintEditor.Pin()->PasteDropLocation = SelectedWidget->GetTickSpaceGeometry().AbsoluteToLocal(CachedMousePosition);
		}
		else
		{
			UCanvasPanelSlot* WidgetSlot = Cast<UCanvasPanelSlot>(SelectedWidget->Slot);
			if (WidgetSlot)
			{
				BlueprintEditor.Pin()->PasteDropLocation = WidgetSlot->GetPosition() + FVector2D(25, 25);
			}
		}
	}
	if ( BlueprintEditor.Pin()->DesignerCommandList->ProcessCommandBindings(InKeyEvent) )
	{
		return FReply::Handled();
	}

	if ( CommandList->ProcessCommandBindings(InKeyEvent) )
	{
		return FReply::Handled();
	}

	const UWidgetDesignerSettings* DesignerSettings = GetDefault<UWidgetDesignerSettings>();

	if ( InKeyEvent.GetKey() == EKeys::Up )
	{
		return NudgeSelectedWidget(FVector2D(0, -DesignerSettings->GridSnapSize));
	}
	else if ( InKeyEvent.GetKey() == EKeys::Down )
	{
		return NudgeSelectedWidget(FVector2D(0, DesignerSettings->GridSnapSize));
	}
	else if ( InKeyEvent.GetKey() == EKeys::Left )
	{
		return NudgeSelectedWidget(FVector2D(-DesignerSettings->GridSnapSize, 0));
	}
	else if ( InKeyEvent.GetKey() == EKeys::Right )
	{
		return NudgeSelectedWidget(FVector2D(DesignerSettings->GridSnapSize, 0));
	}

	return FReply::Unhandled();
}

FReply SDesignerView::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Unhandled();
}

FReply SDesignerView::NudgeSelectedWidget(FVector2D Nudge)
{
	for ( const FWidgetReference& WidgetRef : GetSelectedWidgets() )
	{
		if ( WidgetRef.IsValid() )
		{
			UWidget* CurrentTemplateWidget = WidgetRef.GetTemplate();
			UWidget* CurrentPreviewWidget = WidgetRef.GetPreview();

			if (CurrentTemplateWidget && CurrentPreviewWidget)
			{
				UPanelSlot* TemplateSlot = CurrentTemplateWidget->Slot;
				UPanelSlot* PreviewSlot = CurrentPreviewWidget->Slot;

				if ( TemplateSlot && PreviewSlot )
				{
					FScopedTransaction Transaction(LOCTEXT("Designer_NudgeWidget", "Nudge Widget"));

					const UWidgetDesignerSettings* const WidgetDesignerSettings = GetDefault<UWidgetDesignerSettings>();

					// Attempt to nudge the slot. 
					if (TemplateSlot->NudgeByDesigner(Nudge, WidgetDesignerSettings->GridSnapEnabled ? TOptional<int32>(WidgetDesignerSettings->GridSnapSize) : TOptional<int32>()))
					{
						PreviewSlot->SynchronizeFromTemplate(TemplateSlot);
						
						UWidgetBlueprint* Blueprint = GetBlueprint();
						FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
					}
					// Nudge failed, cancel transaction.
					else
					{
						Transaction.Cancel();
					}
				}
			}
		}
	}

	return FReply::Handled();
}

void SDesignerView::ShowContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FWidgetBlueprintEditorUtils::CreateWidgetContextMenu(MenuBuilder, BlueprintEditor.Pin().ToSharedRef(), SelectedWidgetContextMenuLocation);

	TSharedPtr<SWidget> MenuContent = MenuBuilder.MakeWidget();

	if ( MenuContent.IsValid() )
	{
		FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}
}

void SDesignerView::PopulateWidgetGeometryCache(FArrangedWidget& Root)
{
	const FSlateRect Rect = PreviewHitTestRoot->GetTickSpaceGeometry().GetLayoutBoundingRect();
	const FSlateRect PaintRect = PreviewHitTestRoot->GetPaintSpaceGeometry().GetLayoutBoundingRect();
	DesignerHittestGrid->SetHittestArea(Rect.GetTopLeft(), Rect.GetSize(),  PaintRect.GetTopLeft());
	DesignerHittestGrid->Clear();

	PopulateWidgetGeometryCache_Loop(Root);
}

void SDesignerView::PopulateWidgetGeometryCache_Loop(FArrangedWidget& CurrentWidget)
{
	// If this widget clips to its bounds, then generate a new clipping rect representing the intersection of the bounding
	// rectangle of the widget's geometry, and the current clipping rectangle.
	bool bClipToBounds, bAlwaysClip, bIntersectClipBounds;
	FSlateRect CullingBounds = CurrentWidget.Widget->CalculateCullingAndClippingRules(CurrentWidget.Geometry, FSlateRect(), bClipToBounds, bAlwaysClip, bIntersectClipBounds);

	// NOTE: We're unable to deal with custom clipping states with this method, we'd have to do the true paint
	//   for widgets, which would be much more expensive.

	if (bClipToBounds)
	{
		// The hit test grid records things in desktop space, so we use the tick geometry instead of the paint geometry.
		FSlateClippingZone DesktopClippingZone(CurrentWidget.Geometry);
		DesktopClippingZone.SetShouldIntersectParent(bIntersectClipBounds);
		DesktopClippingZone.SetAlwaysClip(bAlwaysClip);
	}

	bool bIncludeInHitTestGrid = false;

	// Widgets that are children of foreign userWidgets should not be considered selection candidates.
	UWidget* CandidateUWidget = GetWidgetInDesignScopeFromSlateWidget(CurrentWidget.Widget);
	if (CandidateUWidget)
	{
		bool bRespectLocks = IsRespectingLocks();

		if (!CandidateUWidget->IsVisibleInDesigner())
		{
			bIncludeInHitTestGrid = false;
		}
		else if (bRespectLocks && CandidateUWidget->IsLockedInDesigner())
		{
			bIncludeInHitTestGrid = false;
		}
		else
		{
			bIncludeInHitTestGrid = true;
		}
	}

	if (bIncludeInHitTestGrid)
	{
		DesignerHittestGrid->AddWidget(&(CurrentWidget.Widget.Get()), 0, 0, FSlateInvalidationWidgetSortOrder());
	}

	FArrangedChildren ArrangedChildren(EVisibility::All);
	CurrentWidget.Widget->ArrangeChildren(CurrentWidget.Geometry, ArrangedChildren);

	CachedWidgetGeometry.Add(CurrentWidget.Widget, CurrentWidget);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];
		PopulateWidgetGeometryCache_Loop(SomeChild);
	}
}

int32 SDesignerView::HandleEffectsPainting(const FOnPaintHandlerParams& PaintArgs)
{
	DrawSelectionAndHoverOutline(PaintArgs);
	DrawSafeZone(PaintArgs);

	return PaintArgs.Layer + 1;
}

void SDesignerView::DrawSelectionAndHoverOutline(const FOnPaintHandlerParams& PaintArgs)
{
	const TSet<FWidgetReference>& SelectedWidgets = GetSelectedWidgets();

	// Allow the extensions to paint anything they want.
	for ( const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions )
	{
		Ext->Paint(SelectedWidgets, PaintArgs.Geometry, PaintArgs.ClippingRect, PaintArgs.OutDrawElements, PaintArgs.Layer);
	}

	const FLinearColor SelectedTint(0, 1, 0);
	const bool bAntiAlias = false;

	for ( const FWidgetReference& SelectedWidget : SelectedWidgets )
	{
		TSharedPtr<SWidget> SelectedSlateWidget = SelectedWidget.GetPreviewSlate();

		if ( SelectedSlateWidget.IsValid() )
		{
			TSharedRef<SWidget> Widget = SelectedSlateWidget.ToSharedRef();

			FArrangedWidget ArrangedWidget(SNullWidget::NullWidget, FGeometry());
			FDesignTimeUtils::GetArrangedWidgetRelativeToWindow(Widget, ArrangedWidget);

			// Draw selection effect
			const FVector2D OutlinePixelSize = FVector2D(2.0f, 2.0f) / FVector2D(ArrangedWidget.Geometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());
			FPaintGeometry SelectionGeometry = ArrangedWidget.Geometry.ToInflatedPaintGeometry(OutlinePixelSize);

			FSlateClippingZone SelectionZone(SelectionGeometry);

			TArray<FVector2D> Points;
			Points.Add(FVector2D(SelectionZone.TopLeft));
			Points.Add(FVector2D(SelectionZone.TopRight));
			Points.Add(FVector2D(SelectionZone.BottomRight));
			Points.Add(FVector2D(SelectionZone.BottomLeft));
			Points.Add(FVector2D(SelectionZone.TopLeft));

			FSlateDrawElement::MakeLines(
				PaintArgs.OutDrawElements,
				PaintArgs.Layer,
				FPaintGeometry(),
				Points,
				ESlateDrawEffect::None,
				SelectedTint,
				bAntiAlias,
				2.0f);
		}
	}

	const FWidgetReference& HoveredWidget = BlueprintEditor.Pin()->GetHoveredWidget();
	TSharedPtr<SWidget> HoveredSlateWidget = HoveredWidget.GetPreviewSlate();

	// Don't draw the hovered effect if it's also the selected widget
	if ( HoveredSlateWidget.IsValid() && !SelectedWidgets.Contains(HoveredWidget) )
	{
		const FLinearColor HoveredTint(0, 0.5, 1, HoveredWidgetOutlineFade.GetLerp()); // Azure = 0x007FFF

		TSharedRef<SWidget> Widget = HoveredSlateWidget.ToSharedRef();

		FArrangedWidget ArrangedWidget(SNullWidget::NullWidget, FGeometry());
		FDesignTimeUtils::GetArrangedWidgetRelativeToWindow(Widget, ArrangedWidget);

		// Draw hovered effect
		const FVector2D OutlinePixelSize = FVector2D(2.0f, 2.0f) / FVector2D(ArrangedWidget.Geometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());
		FPaintGeometry HoveredGeometry = ArrangedWidget.Geometry.ToInflatedPaintGeometry(OutlinePixelSize);

		FSlateClippingZone HoveredZone(HoveredGeometry);

		TArray<FVector2D> Points;
		Points.Add(FVector2D(HoveredZone.TopLeft));
		Points.Add(FVector2D(HoveredZone.TopRight));
		Points.Add(FVector2D(HoveredZone.BottomRight));
		Points.Add(FVector2D(HoveredZone.BottomLeft));
		Points.Add(FVector2D(HoveredZone.TopLeft));

		FSlateDrawElement::MakeLines(
			PaintArgs.OutDrawElements,
			PaintArgs.Layer,
			FPaintGeometry(),
			Points,
			ESlateDrawEffect::None,
			HoveredTint,
			bAntiAlias,
			2.0f);
	}
}

void SDesignerView::DrawSafeZone(const FOnPaintHandlerParams& PaintArgs)
{
	bool bCanShowSafeZone = false;

	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		switch ( DefaultWidget->DesignSizeMode )
		{
		case EDesignPreviewSizeMode::CustomOnScreen:
		case EDesignPreviewSizeMode::DesiredOnScreen:
		case EDesignPreviewSizeMode::FillScreen:
			bCanShowSafeZone = true;
			break;
		}
	}

	if ( bCanShowSafeZone )
{
		const float UnsafeZoneAlpha = 0.2f;
		const FLinearColor UnsafeZoneColor(1.0f, 0.5f, 0.5f, UnsafeZoneAlpha);
		const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
			
		FGeometry PreviewGeometry = PreviewAreaConstraint->GetTickSpaceGeometry();
		PreviewGeometry.AppendTransform(FSlateLayoutTransform(Inverse(PaintArgs.Args.GetWindowToDesktopTransform())));
		
		const float Width = static_cast<float>(PreviewWidth);
		const float Height = static_cast<float>(PreviewHeight);
		if (PreviewOverrideName.IsEmpty())
		{
			FMargin SafeMargin;
			FSlateApplication::Get().ResetCustomSafeZone();
			FSlateApplication::Get().GetSafeZoneSize(SafeMargin, FVector2f(Width, Height));
			const float HeightOfSides = Height - SafeMargin.GetTotalSpaceAlong<Orient_Vertical>();
			// Top bar
			FSlateDrawElement::MakeBox(
				PaintArgs.OutDrawElements,
				PaintArgs.Layer,
				PreviewGeometry.ToPaintGeometry(FVector2f(Width, SafeMargin.Top), FSlateLayoutTransform()),
				WhiteBrush,
				ESlateDrawEffect::None,
				UnsafeZoneColor
			);

			// Bottom bar
			FSlateDrawElement::MakeBox(
				PaintArgs.OutDrawElements,
				PaintArgs.Layer,
				PreviewGeometry.ToPaintGeometry(FVector2f(Width, SafeMargin.Bottom), FSlateLayoutTransform(FVector2f(0.0f, Height - SafeMargin.Bottom))),
				WhiteBrush,
				ESlateDrawEffect::None,
				UnsafeZoneColor
			);

			// Left bar
			FSlateDrawElement::MakeBox(
				PaintArgs.OutDrawElements,
				PaintArgs.Layer,
				PreviewGeometry.ToPaintGeometry(FVector2f(SafeMargin.Left, HeightOfSides), FSlateLayoutTransform(FVector2f(0.0f, SafeMargin.Top))),
				WhiteBrush,
				ESlateDrawEffect::None,
				UnsafeZoneColor
			);

			// Right bar
			FSlateDrawElement::MakeBox(
				PaintArgs.OutDrawElements,
				PaintArgs.Layer,
				PreviewGeometry.ToPaintGeometry(FVector2f(SafeMargin.Right, HeightOfSides), FSlateLayoutTransform(FVector2f(Width - SafeMargin.Right, SafeMargin.Top))),
				WhiteBrush,
				ESlateDrawEffect::None,
				UnsafeZoneColor
			);

		}
		else
		{
			ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
			if (bSafeZoneFlipped)
			{
				DesignerSafeZoneOverride = PlaySettings->FlipCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
			}
			else
			{
				DesignerSafeZoneOverride = PlaySettings->CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
			}

			for (int ZoneIndex = 0; ZoneIndex < CustomSafeZoneStarts.Num(); ZoneIndex++)
			{
				FVector2D Start = CustomSafeZoneStarts[ZoneIndex];
				FVector2D Dimensions = CustomSafeZoneDimensions[ZoneIndex];
				FSlateDrawElement::MakeBox(
					PaintArgs.OutDrawElements,
					PaintArgs.Layer,
					PreviewGeometry.ToPaintGeometry(Dimensions, FSlateLayoutTransform(Start)),
					WhiteBrush,
					ESlateDrawEffect::None,
					UnsafeZoneColor
				);
			}
		}
	}
}

void SDesignerView::UpdatePreviewWidget(bool bForceUpdate)
{
	UUserWidget* LatestPreviewWidget = BlueprintEditor.Pin()->GetPreview();

	if ( LatestPreviewWidget != PreviewWidget || bForceUpdate )
	{
		PreviewWidget = LatestPreviewWidget;
		if ( PreviewWidget )
		{
			// Notify all designer extensions that a new preview was created
			for (const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions)
			{
				Ext->PreviewContentCreated(PreviewWidget);
			}
		
			TSharedRef<SWidget> NewPreviewSlateWidget = PreviewWidget->TakeWidget();
			NewPreviewSlateWidget->SlatePrepass(PreviewSizeConstraint->GetCachedGeometry().Scale);

			PreviewSlateWidget = NewPreviewSlateWidget;

			PreviewSizeConstraint->SetContent(NewPreviewSlateWidget);

			// Notify all designer extensions that the content has changed
			for (const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions)
			{
				Ext->PreviewContentChanged(NewPreviewSlateWidget);
			}

			// Notify all selected widgets that they are selected, because there are new preview objects
			// state may have been lost so this will recreate it if the widget does something special when
			// selected.
			for ( const FWidgetReference& WidgetRef : GetSelectedWidgets() )
			{
				if ( WidgetRef.IsValid() )
				{
					WidgetRef.GetPreview()->SelectByDesigner();
				}
			}

			BroadcastDesignerChanged();
		}
		else
		{
			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoWidgetPreview", "No Widget Preview"))
				]
			];
		}
	}
}

void SDesignerView::BroadcastDesignerChanged()
{
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	UUserWidget* LatestPreviewWidget = WidgetBlueprintEditor->GetPreview();
	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	
	if ( LatestPreviewWidget )
	{
		FDesignerChangedEventArgs EventArgs;
		if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
		{
			switch ( DefaultWidget->DesignSizeMode )
			{
			case EDesignPreviewSizeMode::CustomOnScreen:
			case EDesignPreviewSizeMode::DesiredOnScreen:
			case EDesignPreviewSizeMode::FillScreen:
				EventArgs.bScreenPreview = true;
				break;
			default:
				EventArgs.bScreenPreview = false;
			}
		}

		EventArgs.Size = FVector2D(PreviewWidth, PreviewHeight);
		EventArgs.DpiScale = GetPreviewDPIScale();

		LatestPreviewWidget->OnDesignerChanged(EventArgs);
		WidgetBlueprintEditor->OnWidgetPreviewReady.Broadcast();
	}
}

void SDesignerView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	UUserWidget* DefaultWidget = GetDefaultWidget();
	if ( DefaultWidget && ( DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::CustomOnScreen || DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::DesiredOnScreen ) )
	{
		PreviewAreaConstraint->SetHAlign(HAlign_Left);
		PreviewAreaConstraint->SetVAlign(VAlign_Top);
	}
	else
	{
		PreviewAreaConstraint->SetHAlign(HAlign_Fill);
		PreviewAreaConstraint->SetVAlign(VAlign_Fill);
	}

	// Tick the parent first to update CachedGeometry
	SDesignSurface::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const bool bForceUpdate = false;
	UpdatePreviewWidget(bForceUpdate);

	// Update the preview flow direction based on the active game preview language (if any)
	{
		TOptional<EFlowDirectionPreference> FlowDirectionOverride;
		if (FTextLocalizationManager::Get().IsGameLocalizationPreviewEnabled())
		{
			if (FCulturePtr PreviewLanguage = FInternationalization::Get().GetCulture(FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage()))
			{
				FlowDirectionOverride = PreviewLanguage->IsRightToLeft() ? EFlowDirectionPreference::RightToLeft : EFlowDirectionPreference::LeftToRight;
			}
		}
		PreviewSizeConstraint->SetFlowDirectionPreference(FlowDirectionOverride.Get(EFlowDirectionPreference::Inherit));
	}

	// Perform an arrange children pass to cache the geometry of all widgets so that we can query it later.
	{
		CachedWidgetGeometry.Reset();
		FArrangedWidget WindowWidgetGeometry(PreviewHitTestRoot.ToSharedRef(), GetDesignerGeometry());
		PopulateWidgetGeometryCache(WindowWidgetGeometry);
	}

	TArray< TFunction<void()> >& QueuedActions = BlueprintEditor.Pin()->GetQueuedDesignerActions();
	for ( TFunction<void()>& Action : QueuedActions )
	{
		Action();
	}

	if ( QueuedActions.Num() > 0 )
	{
		QueuedActions.Reset();

		CachedWidgetGeometry.Reset();
		FArrangedWidget WindowWidgetGeometry(PreviewHitTestRoot.ToSharedRef(), GetDesignerGeometry());
		PopulateWidgetGeometryCache(WindowWidgetGeometry);
	}

	// Tick all designer extensions in case they need to update widgets
	for ( const TSharedRef<FDesignerExtension>& Ext : DesignerExtensions )
	{
		Ext->Tick(GetDesignerGeometry(), InCurrentTime, InDeltaTime);
	}

	// Compute the origin in absolute space.
	FGeometry RootGeometry = CachedWidgetGeometry.FindChecked(PreviewSurface.ToSharedRef()).Geometry;
	FVector2D AbsoluteOrigin = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(FVector2D::ZeroVector);

	GridOrigin = AbsoluteOrigin;

	TopRuler->SetRuling(AbsoluteOrigin, 1.0f / GetPreviewScale());
	SideRuler->SetRuling(AbsoluteOrigin, 1.0f / GetPreviewScale());

	if ( IsHovered() )
	{
		// Get cursor in absolute window space.
		FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
		CursorPos = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(RootGeometry.AbsoluteToLocal(CursorPos));

		TopRuler->SetCursor(CursorPos);
		SideRuler->SetCursor(CursorPos);
	}
	else
	{
		TopRuler->SetCursor(TOptional<FVector2D>());
		SideRuler->SetCursor(TOptional<FVector2D>());
	}

	if ( PreviewWidget )
	{
		if (TSharedPtr<SWidget> CachedWidget = PreviewWidget->GetCachedWrappedWidget())
		{
			CachedPreviewDesiredSize = CachedWidget->GetDesiredSize();
		}
	}
}

int32 SDesignerView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// SDesignerView defines a relationship between a UWorld and a portion of the Slate hierarchy (its superclass and children), therefore it is
	// responsible for pushing its scene onto the stack so that any child widgets that use Material Parameter Collection will resolve to the MPC
	// instance associated with that scene.
	UE::Slate::FSceneRegistrationScope SceneScope;
	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = BlueprintEditor.Pin())
	{
		if (FPreviewScene* Preview = BlueprintEditorPtr->GetPreviewScene())
		{
			if (FSceneInterface* Scene = Preview->GetWorld()->Scene)
			{
				SceneScope = FSlateApplication::Get().GetRenderer()->RegisterCurrentScene(Scene);
			}
		}
	}

	return SDesignSurface::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

void SDesignerView::OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Set a UI scale for materials to use as reference, done on a per-window basis since don't want to change global uniforms per element
	if (SWindow* ParentWindow = OutDrawElements.GetPaintWindow())
	{
		ParentWindow->SetViewportScaleUIOverride(GetZoomAmount());
	}

	SDesignSurface::OnPaintBackground(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

	if (bShowResolutionOutlines)
	{
		if (const UWidgetEditingProjectSettings* Settings = FWidgetBlueprintEditorUtils::GetRelevantSettings(BlueprintEditor))
		{
			for (const FDebugResolution& Resolution : Settings->DebugResolutions)
			{
				DrawResolution(Resolution, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			}
		}
	}
}

void SDesignerView::DrawResolution(const FDebugResolution& Resolution, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const float Scale = GetZoomAmount();
	const FVector2D ZeroSpace = AllottedGeometry.AbsoluteToLocal(GridOrigin);

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

	FVector2D ResolutionSize(Resolution.Width, Resolution.Height);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(ResolutionSize * Scale, FSlateLayoutTransform(ZeroSpace)),
		WhiteBrush,
		ESlateDrawEffect::None,
		Resolution.Color
	);

	FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("UMGEditor.ResizeResolutionFont");

	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	FString ResolutionString;
	if (Resolution.Description.IsEmpty())
	{
		ResolutionString = FString::Printf(TEXT("%d x %d"), Resolution.Width, Resolution.Height);
	}
	else
	{
		ResolutionString = FString::Printf(TEXT("%d x %d - %s"), Resolution.Width, Resolution.Height, *Resolution.Description);
	}

	FVector2D ResolutionStringSize = FontMeasureService->Measure(ResolutionString, FontInfo);
	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToOffsetPaintGeometry(ZeroSpace + ResolutionSize * Scale - (ResolutionStringSize + FVector2D(2, 2))),
		ResolutionString,
		FontInfo,
		ESlateDrawEffect::None,
		FLinearColor::Black);
}

FReply SDesignerView::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	typedef FSelectedWidgetDragDropOp::FDraggingWidgetReference FDragWidget;

	SDesignSurface::OnDragDetected(MyGeometry, MouseEvent);

	const TSet<FWidgetReference>& SelectedWidgets = GetSelectedWidgets();

	if (SelectedWidgets.Num() > 0)
	{
		TArray<FDragWidget> DraggingWidgetCandidates;

		// Clear any pending selected widgets, the user has already decided what widget they want.
		PendingSelectedWidget = FWidgetReference();

		for (const FWidgetReference& SelectedWidget : SelectedWidgets)
		{		
			if (UWidget* WidgetPreview = SelectedWidget.GetPreview())
			{
				if (TSharedPtr<SWidget> CachedWidget = WidgetPreview->GetCachedWrappedWidget())
				{
					// Determine The offset to keep the widget from the mouse while dragging
					FArrangedWidget ArrangedWidget(SNullWidget::NullWidget, FGeometry());
					FDesignTimeUtils::GetArrangedWidget(CachedWidget.ToSharedRef(), ArrangedWidget);
					SelectedWidgetContextMenuLocation = ArrangedWidget.Geometry.AbsoluteToLocal(DraggingStartPositionScreenSpace);

					FDragWidget DraggingWidget;
					DraggingWidget.Widget = SelectedWidget;
					DraggingWidget.DraggedOffset = SelectedWidgetContextMenuLocation / ArrangedWidget.Geometry.GetLocalSize();
					DraggingWidgetCandidates.Add(DraggingWidget);
				}
			}
		}

		TArray<FDragWidget> DraggingWidgets;

		for (const FDragWidget& Candidate : DraggingWidgetCandidates)
		{
			// check the parent chain of each dragged widget and ignore those that are children of other dragged widgets
			bool bIsChild = false;
			for (auto CursorPtr = Candidate.Widget.GetTemplate()->GetParent(); CursorPtr != nullptr; CursorPtr = CursorPtr->GetParent())
			{
				if (DraggingWidgetCandidates.ContainsByPredicate([CursorPtr](const FDragWidget& W){ return W.Widget.GetTemplate() == CursorPtr; }))
				{
					bIsChild = true;
					break;
				}
			}

			if (!bIsChild)
			{
				DraggingWidgets.Add(Candidate);
			}
		}

		ClearExtensionWidgets();

		TSharedRef<FSelectedWidgetDragDropOp> DragOp = FSelectedWidgetDragDropOp::New(BlueprintEditor.Pin(), this, DraggingWidgets);
		TWeakPtr<SDesignerView> WeakDesignerView = SharedThis(this);
		DragOp->OnDragDropEnded.AddLambda([WeakDesignerView, WeakDragOp = DragOp.ToWeakPtr()]()
			{
				if (TSharedPtr<SDesignerView> DesignerViewPtr = WeakDesignerView.Pin())
				{
					if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = DesignerViewPtr->BlueprintEditor.Pin())
					{
						if (DesignerViewPtr->DropPreviews.Num() == 0)
						{
							DesignerViewPtr->bMovingExistingWidget = false;
							if (WeakDragOp.IsValid() && WeakDragOp.Pin()->DraggedWidgets.Num() > 0)
							{
								BlueprintEditorPtr->RefreshPreview();
							}
						}
					}
				}
			});

		return FReply::Handled().BeginDragDrop(DragOp);
	}

	return FReply::Unhandled();
}

void SDesignerView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDesignSurface::OnDragEnter(MyGeometry, DragDropEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();

	//@TODO UMG Drop Feedback
}

void SDesignerView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SDesignSurface::OnDragLeave(DragDropEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();

	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if ( DecoratedDragDropOp.IsValid() )
	{
		DecoratedDragDropOp->SetCursorOverride(TOptional<EMouseCursor::Type>());
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
	
	ClearDropPreviews();
}

FReply SDesignerView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDesignSurface::OnDragOver(MyGeometry, DragDropEvent);

	const bool bIsPreview = true;
	bool bFoundChangingParent = false;
	TSharedPtr<FSelectedWidgetDragDropOp> SelectedDragDropOp = DragDropEvent.GetOperationAs<FSelectedWidgetDragDropOp>();
	if (SelectedDragDropOp.IsValid())
	{
		for (const auto& DraggedWidget : SelectedDragDropOp->DraggedWidgets)
		{
			if (!DraggedWidget.bStayingInParent)
			{
				bFoundChangingParent = true;
				break;
			}
		}

		if (bFoundChangingParent)
		{
			ClearDropPreviews();
			ProcessDropAndAddWidget(MyGeometry, DragDropEvent, bIsPreview);
		}
		else
		{
			MoveWidgets(MyGeometry, DragDropEvent, bIsPreview, nullptr, false);
		}
	}
	else
	{
		ClearDropPreviews();
		ProcessDropAndAddWidget(MyGeometry, DragDropEvent, bIsPreview);
	}

	if ( DropPreviews.Num() > 0 )
	{
		//@TODO UMG Drop Feedback
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDesignerView::DetermineDragDropPreviewWidgets(TArray<UWidget*>& OutWidgets, const FDragDropEvent& DragDropEvent, UWidgetTree* RootWidgetTree)
{
	OutWidgets.Empty();
	UWidgetBlueprint* Blueprint = GetBlueprint();

	if (RootWidgetTree == nullptr)
	{
		return;
	}

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, RootWidgetTree, DragDropOp);

	if (Widget)
	{
		OutWidgets.Add(Widget);
	}

	// Mark the widgets for design-time rendering
	for (UWidget* OutWidget : OutWidgets)
	{
		OutWidget->SetDesignerFlags(BlueprintEditor.Pin()->GetCurrentDesignerFlags());
	}
}

void SDesignerView::SwapSafeZoneTypes()
{
	if (FDisplayMetrics::GetDebugTitleSafeZoneRatio() < 1.f)
	{
		PreviewOverrideName = FString();
	}
}

void SDesignerView::ProcessDropAndAddWidget(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const bool bIsPreview)
{
	TSharedPtr<FDragDropOperation> DragOperation = DragDropEvent.GetOperation();

	// In order to prevent the GetWidgetAtCursor code from picking the widgets we're about to move, we need to mark them
	// as the drop preview widgets before any other code can run.
	TSharedPtr<FSelectedWidgetDragDropOp> SelectedDragDropOp = DragDropEvent.GetOperationAs<FSelectedWidgetDragDropOp>();
	if (SelectedDragDropOp.IsValid())
	{
		DropPreviews.Empty();

		for (const auto& DraggedWidget : SelectedDragDropOp->DraggedWidgets)
		{
			FDropPreview DropPreview;
			DropPreview.Parent = nullptr;
			DropPreview.Widget = DraggedWidget.Preview;
			DropPreview.DragOperation = DragOperation;
			DropPreviews.Add(DropPreview);
		}
	}

	ClearDropPreviews();

	UWidgetBlueprint* BP = GetBlueprint();

	UWidget* Target = nullptr;
	UWidgetTree* TargetTree = nullptr;

	FWidgetHitResult HitResult;
	if (FindWidgetUnderCursor(MyGeometry, DragDropEvent, UPanelWidget::StaticClass(), HitResult))
	{
		Target = bIsPreview ? HitResult.Widget.GetPreview() : HitResult.Widget.GetTemplate();
		TargetTree = (bIsPreview && Target) ? Cast<UWidgetTree>(Target->GetOuter()) : ToRawPtr(BP->WidgetTree);
	}
	else if (BP->WidgetTree->RootWidget == nullptr || !bIsPreview)
	{
		TargetTree = BP->WidgetTree;
	}

	FGeometry WidgetUnderCursorGeometry = HitResult.WidgetArranged.Geometry;

	FScopedTransaction DragAndDropTransaction(LOCTEXT("Designer_DragAddDrop", "Drag and Drop Widget"));
	TArray<UWidget*> DragDropPreviewWidgets;

	if ( SelectedDragDropOp.IsValid() && SelectedDragDropOp->Designer != this)
	{
		// Only accept drag drop from the same editor
		SelectedDragDropOp.Reset();
	}
	else
	{
		DetermineDragDropPreviewWidgets(DragDropPreviewWidgets, DragDropEvent, TargetTree);
	}

	if ( DragDropPreviewWidgets.Num() > 0 )
	{
		BlueprintEditor.Pin()->SetHoveredWidget(HitResult.Widget);

		DragOperation->SetCursorOverride(TOptional<EMouseCursor::Type>());

		FScopedTransaction Transaction(LOCTEXT("Designer_AddWidget", "Add Widget"));

		FText DropOnTargetFailureText = FText::GetEmpty();
		const bool bShouldPreventDropOnTargetExtensions = FWidgetBlueprintEditorUtils::ShouldPreventDropOnTargetExtensions(Target, SelectedDragDropOp, DropOnTargetFailureText);
		if ( bShouldPreventDropOnTargetExtensions )
		{
			DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);
			DragAndDropTransaction.Cancel();
		}
		// If there's no root widget go ahead and add the widget into the root slot.
		else if ( BP->WidgetTree->RootWidget == nullptr )
		{
			if ( !bIsPreview )
			{
				BP->WidgetTree->SetFlags(RF_Transactional);
				BP->WidgetTree->Modify();
			}

			// TODO UMG This method isn't great, maybe the user widget should just be a canvas.

			// Add it to the root if there are no other widgets to add it to.
			BP->WidgetTree->RootWidget = DragDropPreviewWidgets[0];

			for (UWidget* Widget : DragDropPreviewWidgets)
			{
				FDropPreview DropPreview;
				DropPreview.Widget = Widget;
				DropPreview.Parent = nullptr;
				DropPreview.DragOperation = DragOperation;
				DropPreviews.Add(DropPreview);
			}
			
			if (!bIsPreview && BP->WidgetTree->RootWidget)
			{
				// We've added a new widget to the BP, notify the BP so it can create a GUID to track it
				BP->OnVariableAdded(BP->WidgetTree->RootWidget->GetFName());
			}
		}
		// If there's already a root widget we need to try and place our widget into a parent widget that we've picked against
		else if ( Target && Target->IsA(UPanelWidget::StaticClass()) )
		{
			UPanelWidget* Parent = Cast<UPanelWidget>(Target);

			// If this isn't a preview operation we need to modify a few things to properly undo the operation.
			if ( !bIsPreview )
			{
				Parent->SetFlags(RF_Transactional);
				Parent->Modify();

				BP->WidgetTree->SetFlags(RF_Transactional);
				BP->WidgetTree->Modify();
			}

			// Determine local position inside the parent widget and add the widget to the slot.
			FVector2D LocalPosition = WidgetUnderCursorGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());

			for (UWidget* Widget : DragDropPreviewWidgets)
			{
				if (UPanelSlot* Slot = Parent->AddChild(Widget))
				{
					const UWidgetDesignerSettings* const WidgetDesignerSettings = GetDefault<UWidgetDesignerSettings>();
					const TOptional<int32> GridSnapSize = WidgetDesignerSettings->GridSnapEnabled ? TOptional<int32>(WidgetDesignerSettings->GridSnapSize) : TOptional<int32>();
					Slot->DragDropPreviewByDesigner(LocalPosition, GridSnapSize, GridSnapSize);

					FDropPreview DropPreview;
					DropPreview.Widget = Widget;
					DropPreview.Parent = Parent;
					DropPreview.DragOperation = DragOperation;
					DropPreviews.Add(DropPreview);
					
					if (!bIsPreview && Widget)
					{
						// We've added a new widget to the BP, notify the BP so it can create a GUID to track it
						BP->OnVariableAdded(Widget->GetFName());
					}
				}
				else
				{
					// Too many children. Stop processing them.
					if (Widget == DragDropPreviewWidgets[0])
					{
						DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);
					}
					break;

					// TODO UMG ERROR Slot can not be created because maybe the max children has been reached.
					//          Maybe we can traverse the hierarchy and add it to the first parent that will accept it?
				}
			}
		}
		else
		{
			DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);

			// Cancel the transaction even if it's not a preview, since we can't do anything
			DragAndDropTransaction.Cancel();
		}

		if (bIsPreview)
		{
			DragAndDropTransaction.Cancel();
		}

		// Remove widgets tracked by the 'DropPreviews' set. We don't consider them to be transient at this point because they have been inserted into the widget tree hierarchy.
		for (const FDropPreview& DropPreview : DropPreviews)
		{
			DragDropPreviewWidgets.RemoveSwap(DropPreview.Widget);
		}

		// Move the remaining widgets into the transient package. Otherwise, they will remain outered to the WidgetTree and end up as properties in the BP class layout as a result.
		for (UWidget* Widget : DragDropPreviewWidgets)
		{
			if (Widget->GetOutermost() != GetTransientPackage())
			{
				Widget->SetFlags(RF_NoFlags);
				Widget->Rename(nullptr, GetTransientPackage());
			}
		}

		// If we had preview widgets, we know that we can not be performing a selected widget drag/drop operation. Bail.
		return;
	}

	// Attempt to deal with moving widgets from a drag operation.
	MoveWidgets(MyGeometry, DragDropEvent, bIsPreview, Target, true);
}

void SDesignerView::MoveWidgets(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const bool bIsPreview, UWidget* Target, const bool bAnyWidgetChangingParent)
{
	TSharedPtr<FSelectedWidgetDragDropOp> SelectedDragDropOp = DragDropEvent.GetOperationAs<FSelectedWidgetDragDropOp>();
	TSharedPtr<FDragDropOperation> DragOperation = DragDropEvent.GetOperation();
	FScopedTransaction DragAndDropTransaction(LOCTEXT("Designer_DragAddDrop", "Drag and Drop Widget"));

	if (SelectedDragDropOp.IsValid() && SelectedDragDropOp->DraggedWidgets.Num() > 0)
	{
		SelectedDragDropOp->SetCursorOverride(TOptional<EMouseCursor::Type>());

		FText TransactionText;
		if (SelectedDragDropOp->DraggedWidgets.Num() == 1)
		{
			TransactionText = LOCTEXT("Designer_MoveWidget", "Move Widget");
		}
		else
		{
			TransactionText = LOCTEXT("Designer_MoveWidgets", "Move Widgets");
		}

		FScopedTransaction Transaction(TransactionText);
		bool bWidgetMoved = false;

		for (auto& DraggedWidget : SelectedDragDropOp->DraggedWidgets)
		{
			// If they've pressed alt, and we were staying in the parent, disable that
			// and adjust the designer message to no longer warn.
			if (DragDropEvent.IsAltDown() && DraggedWidget.bStayingInParent)
			{
				DraggedWidget.bStayingInParent = false;
				if (SelectedDragDropOp->bShowingMessage)
				{
					SelectedDragDropOp->bShowingMessage = false;
					PopDesignerMessage();
				}
			}

			FWidgetHitResult HitResult;
			FGeometry WidgetUnderCursorGeometry;

			bool bFoundWidgetUnderCursor = FindWidgetUnderCursor(MyGeometry, DragDropEvent, UPanelWidget::StaticClass(), HitResult);

			// If we're staying in the parent we started in, replace the parent found under the cursor with
			// the original one, also update the arranged widget data so that our layout calculations are accurate.
			UWidgetBlueprint* BP = GetBlueprint();

			if (DraggedWidget.bStayingInParent)
			{
				// If we are not changing parents, keep the widget in the hierarchy but clean the DropPreviews list.
				if (!bAnyWidgetChangingParent)
				{
					DropPreviews.Empty();
				}
				WidgetUnderCursorGeometry = GetDesignerGeometry();
				if (GetWidgetGeometry(DraggedWidget.ParentWidget, WidgetUnderCursorGeometry))
				{
					Target = bIsPreview ? DraggedWidget.ParentWidget.GetPreview() : DraggedWidget.ParentWidget.GetTemplate();
				}
			}
			else if (bFoundWidgetUnderCursor)
			{
				WidgetUnderCursorGeometry = HitResult.WidgetArranged.Geometry;
			}

			// If we changed the value of bStayingInParent to false since the last check, remove this widget from the hierarchy and later determine a new parent for it.
			if (!DraggedWidget.bStayingInParent && !bAnyWidgetChangingParent)
			{
				ClearDropPreviews();
				Target = bIsPreview ? HitResult.Widget.GetPreview() : HitResult.Widget.GetTemplate();
			}

			FWidgetReference TargetReference = bIsPreview ? BlueprintEditor.Pin()->GetReferenceFromPreview(Target) : BlueprintEditor.Pin()->GetReferenceFromTemplate(Target);
			BlueprintEditor.Pin()->SetHoveredWidget(TargetReference);

			FText DropOnTargetFailureText = FText::GetEmpty();
			const bool bShouldPreventDropOnTargetExtensions = FWidgetBlueprintEditorUtils::ShouldPreventDropOnTargetExtensions(Target, SelectedDragDropOp, DropOnTargetFailureText);

			// If the widget being hovered over is a panel, attempt to place it into that panel.
			if (Target && Target->IsA(UPanelWidget::StaticClass()) && !bShouldPreventDropOnTargetExtensions)
			{
				bWidgetMoved = true;

				UWidget* Widget = bIsPreview ? DraggedWidget.Preview : DraggedWidget.Template;
				UWidget* ParentWidget = bIsPreview ? DraggedWidget.ParentWidget.GetPreview() : DraggedWidget.ParentWidget.GetTemplate();
				if (ensure(Widget))
				{
					UPanelWidget* NewParent = Cast<UPanelWidget>(Target);

					const bool bIsChangingParent = ParentWidget != Target;
					if (bIsChangingParent)
					{
						check(ParentWidget != nullptr);
						UBlueprint* OriginalBP = nullptr;

						// If this isn't a preview operation we need to modify a few things to properly undo the operation.
						if (!bIsPreview)
						{
							NewParent->SetFlags(RF_Transactional);
							NewParent->Modify();

							BP->WidgetTree->SetFlags(RF_Transactional);
							BP->WidgetTree->Modify();

							// If the Widget is changing parents, there's a chance it might be moving to a different WidgetTree as well.
							UWidgetTree* OriginalWidgetTree = Cast<UWidgetTree>(Widget->GetOuter());

							if (UWidgetTree::TryMoveWidgetToNewTree(Widget, BP->WidgetTree))
							{
								// The Widget likely originated from a different blueprint, so get what blueprint it was originally a part of.
								OriginalBP = OriginalWidgetTree ? OriginalWidgetTree->GetTypedOuter<UBlueprint>() : nullptr;
							}

							Widget->SetFlags(RF_Transactional);
							Widget->Modify();

							ParentWidget->SetFlags(RF_Transactional);
							ParentWidget->Modify();

						}

						// The Widget originated from a different blueprint, so mark it as modified.
						if (OriginalBP && OriginalBP != BP)
						{
							FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(OriginalBP);
						}
					}
					else if (!bIsPreview)
					{
						Widget->SetFlags(RF_Transactional);
						Widget->Modify();

						ParentWidget->SetFlags(RF_Transactional);
						ParentWidget->Modify();
					}

					FVector2D ScreenSpacePosition = DragDropEvent.GetScreenSpacePosition();

					const UWidgetDesignerSettings* DesignerSettings = GetDefault<UWidgetDesignerSettings>();
					bool bGridSnapX, bGridSnapY;
					bGridSnapX = bGridSnapY = DesignerSettings->GridSnapEnabled;

					// As long as shift is pressed and we're staying in the same parent,
					// allow the user to lock the movement to a specific axis.
					const bool bLockToAxis =
						FSlateApplication::Get().GetModifierKeys().IsShiftDown() &&
						DraggedWidget.bStayingInParent;

					if (bLockToAxis)
					{
						// Choose the largest axis of movement as the primary axis to lock to.
						FVector2D DragDelta = ScreenSpacePosition - DraggingStartPositionScreenSpace;
						if (FMath::Abs(DragDelta.X) > FMath::Abs(DragDelta.Y))
						{
							// Lock to X Axis
							ScreenSpacePosition.Y = DraggingStartPositionScreenSpace.Y;
							bGridSnapY = false;
						}
						else
						{
							// Lock To Y Axis
							ScreenSpacePosition.X = DraggingStartPositionScreenSpace.X;
							bGridSnapX = false;
						}
					}
					FVector2D LocalPosition = WidgetUnderCursorGeometry.AbsoluteToLocal(ScreenSpacePosition);
					UPanelSlot* Slot = nullptr;

					// Determine if we need to create a new slot or fetch an existing one.
					// Fetching is much faster, so we want to avoid creating whenever possible.
					if (bAnyWidgetChangingParent || bIsChangingParent)
					{

						if (bIsChangingParent)
						{
							Slot = NewParent->AddChild(Widget);
						}
						else if (UPanelWidget* ParentWidgetAsPanel = Cast<UPanelWidget>(ParentWidget))
						{
							Slot = ParentWidgetAsPanel->InsertChildAt(ParentWidgetAsPanel->GetChildIndex(Widget), Widget);
						}
					}
					else
					{
						if (UPanelWidget* ParentWidgetAsPanel = Cast<UPanelWidget>(ParentWidget))
						{
							Slot = Widget->Slot;

							// If we expected to find a slot but it's null, we have to create it.
							if (Slot == nullptr)
							{
								Slot = ParentWidgetAsPanel->AddChild(Widget);
							}
						}
					}

					if (Slot != nullptr)
					{
						FWidgetBlueprintEditorUtils::ImportPropertiesFromText(Slot, DraggedWidget.ExportedSlotProperties);

						bool HasChangedLayout = false;
						// HACK UMG: In order to correctly drop items into the canvas that have a non-zero anchor,
						// we need to know the layout information after slate has performed a pre-pass.  So we have
						// to rebase the layout and reinterpret the new position based on anchor point layout data.
						// This should be pulled out into an extension of some kind so that this can be fixed for
						// other widgets as well that may need to do work like this.
						if (UCanvasPanelSlot* CanvasSlot = Cast<UCanvasPanelSlot>(Slot))
						{
							if (bIsPreview)
							{
								if (TSharedPtr<SWidget> CachedWidget = Widget->GetCachedWrappedWidget())
								{
									CanvasSlot->SaveBaseLayout();

									FArrangedWidget ArrangedWidget(SNullWidget::NullWidget, FGeometry());
																		
									FDesignTimeUtils::GetArrangedWidget(CachedWidget.ToSharedRef(), ArrangedWidget);
									FVector2D Offset = DraggedWidget.DraggedOffset * ArrangedWidget.Geometry.GetLocalSize();
									
									FVector2D NewPosition;
									if (bAnyWidgetChangingParent || bIsChangingParent)
									{
										NewPosition = LocalPosition - Offset;
									}
									else
									{
										NewPosition = LocalPosition - DragDropEvent.GetCursorDelta() - Offset;

									}
									// Perform grid snapping on X and Y if we need to.
									if (bGridSnapX)
									{
										NewPosition.X = ((int32)NewPosition.X) - (((int32)NewPosition.X) % DesignerSettings->GridSnapSize);
									}
									if (bGridSnapY)
									{
										NewPosition.Y = ((int32)NewPosition.Y) - (((int32)NewPosition.Y) % DesignerSettings->GridSnapSize);
									}
									CanvasSlot->SetDesiredPosition(NewPosition);

									CanvasSlot->RebaseLayout();
									HasChangedLayout = true;
								}
							}
						}
						else
						{
							const TOptional<int32> XGridSnapSize = bGridSnapX ? TOptional<int32>(DesignerSettings->GridSnapSize) : TOptional<int32>();
							const TOptional<int32> YGridSnapSize = bGridSnapY ? TOptional<int32>(DesignerSettings->GridSnapSize) : TOptional<int32>();
							HasChangedLayout = Slot->DragDropPreviewByDesigner(LocalPosition, XGridSnapSize, YGridSnapSize);
						}

						// Re-export slot properties.
						if (HasChangedLayout)
						{
							FWidgetBlueprintEditorUtils::ExportPropertiesToText(Slot, DraggedWidget.ExportedSlotProperties);
						}

						FDropPreview DropPreview;
						DropPreview.Widget = Widget;
						DropPreview.Parent = NewParent;
						DropPreviews.Add(DropPreview);
					}
					else
					{
						SelectedDragDropOp->SetCursorOverride(EMouseCursor::SlashedCircle);

						// TODO UMG ERROR Slot can not be created because maybe the max children has been reached.
						//          Maybe we can traverse the hierarchy and add it to the first parent that will accept it?
					}
				}
			}
			else
			{
				SelectedDragDropOp->SetCursorOverride(EMouseCursor::SlashedCircle);
			}
		}

		if (bIsPreview || !bWidgetMoved)
		{
			DragAndDropTransaction.Cancel();
		}
	}

	// Either we're not dragging anything, or no widgets were valid...
	if (DropPreviews.Num() == 0)
	{
		DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);
	}
}

FReply SDesignerView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDesignSurface::OnDrop(MyGeometry, DragDropEvent);

	bMovingExistingWidget = false;
	const bool bIsPreview = false;
	bool bFoundChangingParent = false;
	TSharedPtr<FSelectedWidgetDragDropOp> SelectedDragDropOp = DragDropEvent.GetOperationAs<FSelectedWidgetDragDropOp>();

	if (SelectedDragDropOp.IsValid())
	{
		for (const auto& DraggedWidget : SelectedDragDropOp->DraggedWidgets)
		{
			if (!DraggedWidget.bStayingInParent)
			{
				bFoundChangingParent = true;
				break;
			}
		}
		if (bFoundChangingParent)
		{
			ClearDropPreviews();
			ProcessDropAndAddWidget(MyGeometry, DragDropEvent, bIsPreview);
		}
		else
		{
			MoveWidgets(MyGeometry, DragDropEvent, bIsPreview, nullptr, false);
		}
	}
	else
	{
		bFoundChangingParent = true;
		ClearDropPreviews();
		ProcessDropAndAddWidget(MyGeometry, DragDropEvent, bIsPreview);
	}

	if (DropPreviews.Num() > 0)
	{
		UWidgetBlueprint* BP = GetBlueprint();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);

		TSet<FWidgetReference> SelectedTemplates;
		for (const auto& DropPreview : DropPreviews)
		{
			SelectedTemplates.Add(BlueprintEditor.Pin()->GetReferenceFromTemplate(DropPreview.Widget));
		}

		BlueprintEditor.Pin()->SelectWidgets(SelectedTemplates, false);
		// Regenerate extension widgets now that we've finished moving or placing the widget.
		CreateExtensionWidgetsForSelection();

		DropPreviews.Empty();
		return FReply::Handled().SetUserFocus(SharedThis(this));
	}
	else if (SelectedDragDropOp.IsValid())
	{
		// If we were dragging any widgets, even if we didn't move them, we need to refresh the preview
		// because they are collapsed in the preview when the drag begins
		if (SelectedDragDropOp->DraggedWidgets.Num() > 0)
		{
			BlueprintEditor.Pin().Get()->RefreshPreview();
		}
	}
	return FReply::Unhandled();
}

FText SDesignerView::GetResolutionText(int32 Width, int32 Height, const FString& AspectRatio) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Width"), FText::AsNumber(Width, &FNumberFormattingOptions::DefaultNoGrouping()));
	Args.Add(TEXT("Height"), FText::AsNumber(Height, &FNumberFormattingOptions::DefaultNoGrouping()));
	Args.Add(TEXT("AspectRatio"), FText::FromString(AspectRatio));

	return FText::Format(LOCTEXT("CommonResolutionFormat", "{Width} x {Height} ({AspectRatio})"), Args);
}

FText SDesignerView::GetCurrentResolutionText() const
{
	return GetResolutionText(PreviewWidth, PreviewHeight, PreviewAspectRatio);
}

FText SDesignerView::GetCurrentDPIScaleText() const
{
	FNumberFormattingOptions Options = FNumberFormattingOptions::DefaultNoGrouping();
	Options.MinimumIntegralDigits = 1;
	Options.MaximumFractionalDigits = 2;
	Options.MinimumFractionalDigits = 1;

	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings->UIScaleRule == EUIScalingRule::Custom)
	{
		UClass* CustomScalingRuleClassInstance = UISettings->CustomScalingRuleClass.TryLoadClass<UDPICustomScalingRule>();

		if (CustomScalingRuleClassInstance == nullptr)
		{
			return LOCTEXT("NoCustomRuleWarning", "Warning: Using Custom DPI Rule with no rules class set. Set a class in User Interface Project Settings. ");
		}
	}

	FText DPIString = FText::AsNumber(GetPreviewDPIScale(), &Options);
	return FText::Format(LOCTEXT("CurrentDPIScaleFormat", "DPI Scale {0}"), DPIString);
}

FSlateColor SDesignerView::GetCurrentDPIScaleColor() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings->UIScaleRule == EUIScalingRule::Custom)
	{
		UClass* CustomScalingRuleClassInstance = UISettings->CustomScalingRuleClass.TryLoadClass<UDPICustomScalingRule>();

		if (CustomScalingRuleClassInstance == nullptr)
		{
			return FSlateColor(FLinearColor::Yellow);
		}
	}

	return FSlateColor(FLinearColor(1, 1, 1, 0.25f));
}

FText SDesignerView::GetCurrentScaleFactorText() const
{
	FNumberFormattingOptions Options = FNumberFormattingOptions::DefaultNoGrouping();
	Options.MinimumIntegralDigits = 1;
	Options.MaximumFractionalDigits = 2;
	Options.MinimumFractionalDigits = 1;

	FText DPIString = FText::AsNumber(ScaleFactor, &Options);
	return FText::Format(LOCTEXT("CurrentContentScale", "Device Content Scale {0}"), DPIString);
}

FText SDesignerView::GetCurrentSafeZoneText() const
{
	if (PreviewOverrideName.IsEmpty())
	{
		float SafeZone = FDisplayMetrics::GetDebugTitleSafeZoneRatio();
		if (FMath::IsNearlyEqual(SafeZone, 1.0f))
		{
			return LOCTEXT("NoSafeZoneSet", "No Device Safe Zone Set");
		}
		return FText::Format(LOCTEXT("UniformSafeZone", "Uniform Safe Zone: {0}"), FText::AsNumber(SafeZone));
	}
	return FText::FromString(PreviewOverrideName);
}

FSlateColor SDesignerView::GetResolutionTextColorAndOpacity() const
{
	return FLinearColor(1, 1, 1, 1.25f - ResolutionTextFade.GetLerp());
}

EVisibility SDesignerView::GetResolutionTextVisibility() const
{
	// If we're using a custom design time size, don't bother showing the resolution
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		const bool bScreenlessSizing =
			DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Custom ||
			DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Desired;

		if ( bScreenlessSizing )
		{
			return EVisibility::Collapsed;
		}
	}

	return EVisibility::SelfHitTestInvisible;
}

EVisibility SDesignerView::GetDesignerOutlineVisibility() const
{
	if ( GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr )
	{
		return EVisibility::HitTestInvisible;
	}

	TSharedPtr<ISequencer> Sequencer = BlueprintEditor.Pin()->GetSequencer();
	if ( Sequencer.IsValid() && Sequencer->GetAutoChangeMode() != EAutoChangeMode::None )
	{
		return EVisibility::HitTestInvisible;
	}

	if ( Sequencer.IsValid() )
	{
		UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(Sequencer->GetFocusedMovieSceneSequence());
		if ( WidgetAnimation != UWidgetAnimation::GetNullAnimation() )
		{
			return EVisibility::HitTestInvisible;
		}
	}

	return EVisibility::Hidden;
}

FSlateColor SDesignerView::GetDesignerOutlineColor() const
{
	if ( GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr )
	{
		FLinearColor SimulatingIndicatorColor = FLinearColor(0.863f, 0.407f, 0.0f);
		return SimulatingIndicatorColor;
	}

	TSharedPtr<ISequencer> Sequencer = BlueprintEditor.Pin()->GetSequencer();
	if ( Sequencer.IsValid() && Sequencer->GetAutoChangeMode() != EAutoChangeMode::None )
	{
		FLinearColor AnimRecordingIndicatorColor = FLinearColor::FromSRGBColor(FColor(251, 37, 0));
		return AnimRecordingIndicatorColor;
	}

	if ( Sequencer.IsValid() )
	{
		UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(Sequencer->GetFocusedMovieSceneSequence());
		if ( WidgetAnimation != UWidgetAnimation::GetNullAnimation() )
		{
			FLinearColor AnimSelectedIndicatorColor = FLinearColor::FromSRGBColor(FColor(0, 67, 240));
			return AnimSelectedIndicatorColor;
		}
	}

	return FLinearColor::Transparent;
}

FText SDesignerView::GetDesignerOutlineText() const
{
	if ( GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr )
	{
		return LOCTEXT("SIMULATING", "SIMULATING");
	}

	TSharedPtr<ISequencer> Sequencer = BlueprintEditor.Pin()->GetSequencer();
	if ( Sequencer.IsValid() && Sequencer->GetAutoChangeMode() != EAutoChangeMode::None )
	{
		return LOCTEXT("RECORDING", "RECORDING");
	}

	if ( Sequencer.IsValid() )
	{
		UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(Sequencer->GetFocusedMovieSceneSequence());
		if ( WidgetAnimation != UWidgetAnimation::GetNullAnimation() )
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Name"), FText::FromString(WidgetAnimation->GetDisplayLabel()));
			return FText::Format(LOCTEXT("SELECTED", "SELECTED: {Name}"), Args);
		}
	}

	return FText::GetEmpty();
}

FText SDesignerView::GetCursorPositionText() const
{
	if (const FArrangedWidget* CachedPreviewSurface = CachedWidgetGeometry.Find(PreviewSurface.ToSharedRef()))
	{
		const FGeometry& RootGeometry = CachedPreviewSurface->Geometry;
		const FVector2D CursorPos = RootGeometry.AbsoluteToLocal(FSlateApplication::Get().GetCursorPos()) / GetPreviewDPIScale();

		return FText::Format(LOCTEXT("CursorPositionFormat", "{0} x {1}"), FText::AsNumber(FMath::RoundToInt(CursorPos.X)), FText::AsNumber(FMath::RoundToInt(CursorPos.Y)));
	}
	return FText();
}

EVisibility SDesignerView::GetCursorPositionTextVisibility() const
{
	return IsHovered() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

FText SDesignerView::GetSelectedWidgetDimensionsText() const
{
	const FWidgetReference SelectedWidget = GetSelectedWidget();
	if ( SelectedWidget.IsValid() )
	{
		const UWidget* WidgetPreview = SelectedWidget.GetPreview();
		const FVector2D& Size = WidgetPreview->GetCachedGeometry().GetLocalSize();

		FNumberFormattingOptions FmtOptions;
		FmtOptions.SetMaximumFractionalDigits(2);
		const FText ScaleFactorText = FText::Format(
			LOCTEXT("SelectionDimensionsScaleFormat", "(Render Scale: {0} x {1})"), 
			FText::AsNumber(WidgetPreview->GetRenderTransform().Scale.X, &FmtOptions),
			FText::AsNumber(WidgetPreview->GetRenderTransform().Scale.Y, &FmtOptions));

		bool bShowScaleFactor = !FMath::IsNearlyEqual(WidgetPreview->GetRenderTransform().Scale.X, 1.f) || !FMath::IsNearlyEqual(WidgetPreview->GetRenderTransform().Scale.Y, 1.f);
		return FText::Format(
			LOCTEXT("SelectionDimensionsFormat", "Selection: {0} x {1} {2}"), 
			FText::AsNumber(Size.X, &FmtOptions), 
			FText::AsNumber(Size.Y, &FmtOptions), 
			bShowScaleFactor ? ScaleFactorText : FText());
	}
	return FText();
}

EVisibility SDesignerView::GetSelectedWidgetDimensionsVisibility() const
{
	const TSet<FWidgetReference>& SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();
	return SelectedWidgets.Num() == 1 ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

FReply SDesignerView::HandleDPISettingsClicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Engine", "UI");

	return FReply::Handled();
}

void SDesignerView::HandleOnCommonResolutionSelected(const FPlayScreenResolution InResolution)
{
	bSafeZoneFlipped = false;
	bCanPreviewSwapAspectRatio = InResolution.bCanSwapAspectRatio;
	WidthReadFromSettings = InResolution.Width;
	HeightReadFromSettings = InResolution.Height;
	// Most resolutions (tablets, phones, TVs, etc.) can be stored in either portrait or landscape mode, and may need to be flipped
	if (bCanPreviewSwapAspectRatio && (bPreviewIsPortrait != (InResolution.Width < InResolution.Height)))
	{
		PreviewWidth = InResolution.LogicalHeight;
		PreviewHeight = InResolution.LogicalWidth;
	}
	else
	{
		PreviewWidth = InResolution.LogicalWidth;
		PreviewHeight = InResolution.LogicalHeight;
	}
	bPreviewIsPortrait = PreviewWidth < PreviewHeight;
	PreviewAspectRatio = InResolution.AspectRatio;

	PreviewOverrideName = InResolution.ProfileName;

	ScaleFactor = InResolution.ScaleFactor;

	GConfig->SetInt(*ConfigSectionName, TEXT("PreviewWidth"), PreviewWidth, GEditorPerProjectIni);
	GConfig->SetInt(*ConfigSectionName, TEXT("PreviewHeight"), PreviewHeight, GEditorPerProjectIni);
	GConfig->SetString(*ConfigSectionName, TEXT("PreviewAspectRatio"), *PreviewAspectRatio, GEditorPerProjectIni);
	GConfig->SetBool(*ConfigSectionName, TEXT("bIsInPortraitMode"), bPreviewIsPortrait, GEditorPerProjectIni);
	GConfig->SetString(*ConfigSectionName, TEXT("ProfileName"), *PreviewOverrideName, GEditorPerProjectIni);
	GConfig->SetFloat(*ConfigSectionName, TEXT("ScaleFactor"), ScaleFactor, GEditorPerProjectIni);
	GConfig->SetBool(*ConfigSectionName, TEXT("bCanPreviewSwapAspectRatio"), bCanPreviewSwapAspectRatio, GEditorPerProjectIni);
	GConfig->SetBool(*ConfigSectionName, TEXT("bCommonResolutionSelected"), true, GEditorPerProjectIni);
	if (!PreviewOverrideName.IsEmpty())
	{
		ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
		DesignerSafeZoneOverride = PlayInSettings->CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
	}
	else
	{
		FSlateApplication::Get().ResetCustomSafeZone();
		FSlateApplication::Get().GetSafeZoneSize(DesignerSafeZoneOverride, FVector2D(PreviewWidth, PreviewHeight));
	}
	FMargin SafeZoneRatio = DesignerSafeZoneOverride;
	SafeZoneRatio.Left /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Right /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Bottom /= (PreviewHeight / 2.0f);
	SafeZoneRatio.Top /= (PreviewHeight / 2.0f);
	FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, true);

	if (UUserWidget* DefaultWidget = GetDefaultWidget())
	{
		// If we using custom or desired design time sizes and the user picks a screen size, they must
		// want to also change the visualization to be custom on screen or desired on screen, doesn't
		// make sense to change it otherwise as it would have no effect.
		if (DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Custom)
		{
			DefaultWidget->DesignSizeMode = EDesignPreviewSizeMode::CustomOnScreen;
		}
		else if (DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Desired)
		{
			DefaultWidget->DesignSizeMode = EDesignPreviewSizeMode::DesiredOnScreen;
		}

		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}

	// Recreate the preview widget so PreConstruct fires with the new preview dimensions.
	// UpdatePreviewWidget (on next Tick) will call BroadcastDesignerChanged once the
	// new preview and its child widget trees are fully initialized.
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin())
	{
		Editor->RefreshPreview();
	}

	ResolutionTextFade.Play(this->AsShared());
}

bool SDesignerView::HandleIsCommonResolutionSelected(const FPlayScreenResolution InResolution) const
{
	// If we're using a custom design time size, none of the other resolutions should appear selected, even if they match.
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		if ( DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Custom || DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Desired )
		{
			return false;
		}
	}

	int32 TestHeight = InResolution.Height;
	int32 TestWidth = InResolution.Width;

	// Swap the width and height to test if the preview is currently flipped
	if ((InResolution.bCanSwapAspectRatio) && (PreviewWidth > PreviewHeight))
	{
		TestHeight = InResolution.Width;
		TestWidth = InResolution.Height;
	}

	// Compare to the size in the settings
	const bool bSizeMatches = (((TestWidth == WidthReadFromSettings) && (TestHeight == HeightReadFromSettings))
		|| (((bCanPreviewSwapAspectRatio && (PreviewWidth > PreviewHeight)) || InResolution.bCanSwapAspectRatio) && (TestHeight == WidthReadFromSettings) && (TestWidth == HeightReadFromSettings))); // flipped to landscape

	if (!PreviewOverrideName.IsEmpty() || !InResolution.ProfileName.IsEmpty())
	{
		// Would have the same r.MobileContentScaleFactor and original size
		return InResolution.ProfileName.Equals(PreviewOverrideName) && bSizeMatches;
	}

	return bSizeMatches;
}

FUIAction SDesignerView::GetResolutionMenuAction( const FPlayScreenResolution& ScreenResolution )
{
	FExecuteAction OnResolutionSelected = FExecuteAction::CreateRaw( this, &SDesignerView::HandleOnCommonResolutionSelected, ScreenResolution );
	FIsActionChecked OnIsResolutionSelected = FIsActionChecked::CreateRaw( this, &SDesignerView::HandleIsCommonResolutionSelected, ScreenResolution );
	return FUIAction( OnResolutionSelected, FCanExecuteAction(), OnIsResolutionSelected );
}

TOptional<int32> SDesignerView::GetCustomResolutionWidth() const
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		return FMath::TruncToInt32(DefaultWidget->DesignTimeSize.X);
	}

	return 1;
}

TOptional<int32> SDesignerView::GetCustomResolutionHeight() const
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		return  FMath::TruncToInt32(DefaultWidget->DesignTimeSize.Y);
	}

	return 1;
}

void SDesignerView::OnCustomResolutionWidthChanged(int32 InValue)
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		DefaultWidget->DesignTimeSize.X = InValue;
		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}
}

void SDesignerView::OnCustomResolutionHeightChanged(int32 InValue)
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		DefaultWidget->DesignTimeSize.Y = InValue;
		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}
}

EVisibility SDesignerView::GetCustomResolutionEntryVisibility() const
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		const bool bCustomSizing =
			DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::Custom ||
			DefaultWidget->DesignSizeMode == EDesignPreviewSizeMode::CustomOnScreen;

		return bCustomSizing ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

UUserWidget* SDesignerView::GetDefaultWidget() const
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	if (UUserWidget* Default = BPEd->GetWidgetBlueprintObj()->GeneratedClass->GetDefaultObject<UUserWidget>())
	{
		return Default;
	}

	return nullptr;
}

TSharedRef<SWidget> SDesignerView::GetResolutionsMenu()
{
	UCommonResolutionMenuContext* CommonResolutionMenuContext = NewObject<UCommonResolutionMenuContext>();
	CommonResolutionMenuContext->GetUIActionFromLevelPlaySettings = UCommonResolutionMenuContext::FGetUIActionFromLevelPlaySettings::CreateRaw(this, &SDesignerView::GetResolutionMenuAction);

	return UToolMenus::Get()->GenerateWidget(ULevelEditorPlaySettings::GetCommonResolutionsMenuName(), CommonResolutionMenuContext);
}

TSharedRef<SWidget> SDesignerView::GetScreenSizingFillMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	CreateScreenFillEntry(MenuBuilder, EDesignPreviewSizeMode::FillScreen);
	CreateScreenFillEntry(MenuBuilder, EDesignPreviewSizeMode::Custom);
	CreateScreenFillEntry(MenuBuilder, EDesignPreviewSizeMode::CustomOnScreen);
	CreateScreenFillEntry(MenuBuilder, EDesignPreviewSizeMode::Desired);
	CreateScreenFillEntry(MenuBuilder, EDesignPreviewSizeMode::DesiredOnScreen);

	return MenuBuilder.MakeWidget();
}

void SDesignerView::CreateScreenFillEntry(FMenuBuilder& MenuBuilder, EDesignPreviewSizeMode SizeMode)
{
	const static UEnum* PreviewSizeEnum = StaticEnum<EDesignPreviewSizeMode>();

	// Add desired size option
	FUIAction DesiredSizeAction(
		FExecuteAction::CreateRaw(this, &SDesignerView::OnScreenFillRuleSelected, SizeMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &SDesignerView::GetIsScreenFillRuleSelected, SizeMode));

	FText EntryText = PreviewSizeEnum->GetDisplayNameTextByValue((int64)SizeMode);
	MenuBuilder.AddMenuEntry(EntryText, FText::GetEmpty(), FSlateIcon(), DesiredSizeAction, NAME_None, EUserInterfaceActionType::Check);
}

FText SDesignerView::GetScreenSizingFillText() const
{
	const static UEnum* PreviewSizeEnum = StaticEnum<EDesignPreviewSizeMode>();

	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		return PreviewSizeEnum->GetDisplayNameTextByValue((int64)DefaultWidget->DesignSizeMode);
	}

	return FText::GetEmpty();
}

bool SDesignerView::GetIsScreenFillRuleSelected(EDesignPreviewSizeMode SizeMode) const
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		return DefaultWidget->DesignSizeMode == SizeMode;
	}

	return false;
}

void SDesignerView::OnScreenFillRuleSelected(EDesignPreviewSizeMode SizeMode)
{
	if ( UUserWidget* DefaultWidget = GetDefaultWidget() )
	{
		DefaultWidget->DesignSizeMode = SizeMode;
		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}
}

const FSlateBrush* SDesignerView::GetAspectRatioSwitchImage() const
{
	if (PreviewHeight > PreviewWidth)
	{
		return FAppStyle::Get().GetBrush("UMGEditor.OrientPortrait");
	}
	return FAppStyle::Get().GetBrush("UMGEditor.OrientLandscape");
}

bool SDesignerView::GetAspectRatioSwitchEnabled() const
{
	return bCanPreviewSwapAspectRatio;
}

bool SDesignerView::GetFlipDeviceEnabled() const
{
	return PreviewWidth > PreviewHeight && !PreviewOverrideName.IsEmpty();
}

void SDesignerView::BeginTransaction(const FText& SessionName)
{
	if ( ScopedTransaction == nullptr )
	{
		ScopedTransaction = new FScopedTransaction(SessionName);

		for ( const FWidgetReference& SelectedWidget : GetSelectedWidgets() )
		{
			if ( SelectedWidget.IsValid() )
			{
				SelectedWidget.GetPreview()->Modify();
				SelectedWidget.GetTemplate()->Modify();
			}
		}
	}
}

bool SDesignerView::InTransaction() const
{
	return ScopedTransaction != nullptr;
}

void SDesignerView::EndTransaction(bool bCancel)
{
	if ( ScopedTransaction != nullptr )
	{
		if ( bCancel )
		{
			ScopedTransaction->Cancel();
		}

		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

FReply SDesignerView::HandleZoomToFitClicked()
{
	ZoomToFit(/*bInstantZoom*/ false);
	return FReply::Handled();
}

FReply SDesignerView::HandleSwapAspectRatioClicked()
{
	bSafeZoneFlipped = false;
	// If in default orientation (portrait for table/phone, landscape for monitor/laptop/TV)
	if ((WidthReadFromSettings < HeightReadFromSettings) == (PreviewWidth < PreviewHeight))
	{
		PreviewHeight = WidthReadFromSettings;
		PreviewWidth = HeightReadFromSettings;
	}
	else
	{
		PreviewHeight = HeightReadFromSettings;
		PreviewWidth = WidthReadFromSettings;
	}

	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	const UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(PreviewOverrideName, false);

	// Rescale the swapped sizes that are from the initial settings load
	if (DeviceProfile)
	{
		float TempScaleFactor = 1.0f;
		PlayInSettings->RescaleForMobilePreview(DeviceProfile, PreviewWidth, PreviewHeight, TempScaleFactor);
	}

	bPreviewIsPortrait = (PreviewHeight > PreviewWidth);
	GConfig->SetInt(*ConfigSectionName, TEXT("PreviewWidth"), PreviewWidth, GEditorPerProjectIni);
	GConfig->SetInt(*ConfigSectionName, TEXT("PreviewHeight"), PreviewHeight, GEditorPerProjectIni);
	GConfig->SetString(*ConfigSectionName, TEXT("PreviewAspectRatio"), *PreviewAspectRatio, GEditorPerProjectIni);
	GConfig->SetBool(*ConfigSectionName, TEXT("bIsInPortraitMode"), bPreviewIsPortrait, GEditorPerProjectIni);
	GConfig->SetString(*ConfigSectionName, TEXT("ProfileName"), *PreviewOverrideName, GEditorPerProjectIni);
	GConfig->SetFloat(*ConfigSectionName, TEXT("ScaleFactor"), ScaleFactor, GEditorPerProjectIni);
	GConfig->SetBool(*ConfigSectionName, TEXT("bCanPreviewSwapAspectRatio"), bCanPreviewSwapAspectRatio, GEditorPerProjectIni);

	if (!PreviewOverrideName.IsEmpty())
	{
		DesignerSafeZoneOverride = PlayInSettings->CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
	}
	else
	{
		FSlateApplication::Get().ResetCustomSafeZone();
		FSlateApplication::Get().GetSafeZoneSize(DesignerSafeZoneOverride, FVector2D(PreviewWidth, PreviewHeight));
	}
	FMargin SafeZoneRatio = DesignerSafeZoneOverride;
	SafeZoneRatio.Left /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Right /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Bottom /= (PreviewHeight / 2.0f);
	SafeZoneRatio.Top /= (PreviewHeight / 2.0f);
	FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, true);

	if (UUserWidget* DefaultWidget = GetDefaultWidget())
	{
		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}

	// Recreate the preview widget so PreConstruct fires with the new preview dimensions.
	// UpdatePreviewWidget (on next Tick) will call BroadcastDesignerChanged once the
	// new preview and its child widget trees are fully initialized.
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin())
	{
		Editor->RefreshPreview();
	}

	ResolutionTextFade.Play(this->AsShared());

	return FReply::Handled();
}

FReply SDesignerView::HandleFlipSafeZonesClicked()
{
	if (!PreviewOverrideName.IsEmpty())
	{
		ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
		if (!bSafeZoneFlipped)
		{
			DesignerSafeZoneOverride = PlaySettings->FlipCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
			bSafeZoneFlipped = true;
		}
		else
		{
			DesignerSafeZoneOverride = PlaySettings->CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, PreviewOverrideName, FVector2D(PreviewWidth, PreviewHeight));
			bSafeZoneFlipped = false;
		}
	}

	FMargin SafeZoneRatio = DesignerSafeZoneOverride;
	SafeZoneRatio.Left /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Right /= (PreviewWidth / 2.0f);
	SafeZoneRatio.Bottom /= (PreviewHeight / 2.0f);
	SafeZoneRatio.Top /= (PreviewHeight / 2.0f);
	FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, true);

	if (UUserWidget* DefaultWidget = GetDefaultWidget())
	{
		MarkDesignModifed(/*bRequiresRecompile*/ false);
	}
	BroadcastDesignerChanged();

	ResolutionTextFade.Play(this->AsShared());

	return FReply::Handled();
}

EVisibility SDesignerView::GetRulerVisibility() const
{
	return EVisibility::Visible;
}

bool SDesignerView::IsSelectableInSequencer(UWidget* const InWidget) const
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid()
		|| WidgetBlueprintEditor->GetCurrentAnimation() == UWidgetAnimation::GetNullAnimation())
	{
		return true;
	}

	const TSharedPtr<ISequencer>& Sequencer = WidgetBlueprintEditor->GetSequencer();
	if (!Sequencer.IsValid() || !Sequencer->IsViewportSelectionLimited())
	{
		return true;
	}

	return Sequencer->IsObjectSelectableInViewport(InWidget);
}

void SDesignerView::OnSelectedAnimationChanged()
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer>& ActiveSequencer = WidgetBlueprintEditor->GetSequencer();
	if (!ActiveSequencer.IsValid())
	{
		return;
	}

	if (ActiveSequencer->IsViewportSelectionLimited())
	{
		DeselectNonSequencerWidgets();
	}
}

void SDesignerView::OnSelectionLimitedChanged(const bool bInEnabled)
{
	if (bInEnabled)
	{
		DeselectNonSequencerWidgets();
	}
}

void SDesignerView::DeselectNonSequencerWidgets()
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer>& ActiveSequencer = WidgetBlueprintEditor->GetSequencer();
	if (!ActiveSequencer.IsValid())
	{
		return;
	}

	const TSet<FWidgetReference>& SelectedWidgets = WidgetBlueprintEditor->GetSelectedWidgets();
	if (SelectedWidgets.IsEmpty())
	{
		return;
	}

	TSet<FWidgetReference> NewSelection;

	for (const FWidgetReference& WidgetReference : SelectedWidgets)
	{
		UWidget* const Widget = WidgetReference.GetPreview();
		if (!IsValid(Widget))
		{
			continue;
		}

		if (!ActiveSequencer->IsViewportSelectionLimited()
			|| WidgetBlueprintEditor->GetCurrentAnimation() == UWidgetAnimation::GetNullAnimation()
			|| ActiveSequencer->IsObjectSelectableInViewport(Widget))
		{
			NewSelection.Add(WidgetReference);
		}
	}

	WidgetBlueprintEditor->SelectWidgets(NewSelection, false);
}

void SDesignerView::ToggleViewportSelectionLimited()
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid() || WidgetBlueprintEditor->GetCurrentAnimation() == UWidgetAnimation::GetNullAnimation())
	{
		return;
	}

	const TSharedPtr<ISequencer>& ActiveSequencer = WidgetBlueprintEditor->GetSequencer();
	if (!ActiveSequencer.IsValid())
	{
		return;
	}

	ActiveSequencer->SetViewportSelectionLimited(!ActiveSequencer->IsViewportSelectionLimited());
}

bool SDesignerView::CanToggleViewportSelectionLimited() const
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid() || WidgetBlueprintEditor->GetCurrentAnimation() == UWidgetAnimation::GetNullAnimation())
	{
		return false;
	}

	const TSharedPtr<ISequencer>& ActiveSequencer = WidgetBlueprintEditor->GetSequencer();
	if (!ActiveSequencer.IsValid())
	{
		return false;
	}

	return true;
}

bool SDesignerView::IsViewportSelectionLimited() const
{
	const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
	if (!WidgetBlueprintEditor.IsValid() || WidgetBlueprintEditor->GetCurrentAnimation() == UWidgetAnimation::GetNullAnimation())
	{
		return false;
	}

	const TSharedPtr<ISequencer>& ActiveSequencer = WidgetBlueprintEditor->GetSequencer();
	if (!ActiveSequencer.IsValid())
	{
		return false;
	}

	return ActiveSequencer->IsViewportSelectionLimited();
}

#undef LOCTEXT_NAMESPACE
