// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/SDisappearingBar.h"

#include "Delegates/Delegate.h"
#include "Layout/Children.h"

class FDragDropEvent;
struct FGeometry;

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// SDisappearingBar

void SDisappearingBar::Construct(const FArguments& InArgs)
{
	FadeCurve = FCurveSequence(0, 0.25f);
	SetColorAndOpacity(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &SDisappearingBar::GetFadeColorAndOpacity)));

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void SDisappearingBar::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( !FadeCurve.IsAtEnd() )
	{
		// Play the curve forward to fade the bar out
		if ( FadeCurve.IsInReverse() && FadeCurve.IsPlaying() )
		{
			FadeCurve.Reverse();
		}
		else if ( !FadeCurve.IsPlaying() )
		{
			FadeCurve.Play( this->AsShared() );
		}
	}
}

void SDisappearingBar::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if ( !FadeCurve.IsAtStart() )
	{
		// Play the curve in reverse to fade the bar back in
		if ( !FadeCurve.IsInReverse() && FadeCurve.IsPlaying() )
		{
			FadeCurve.Reverse();
		}
		else if ( !FadeCurve.IsPlaying() )
		{
			FadeCurve.PlayReverse( this->AsShared() );
		}
	}
}

FLinearColor SDisappearingBar::GetFadeColorAndOpacity() const
{
	return FLinearColor(1, 1, 1, 1 - FadeCurve.GetLerp());
}

#undef LOCTEXT_NAMESPACE
