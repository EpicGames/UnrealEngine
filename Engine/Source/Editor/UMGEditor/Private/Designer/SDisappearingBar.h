// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDragDropEvent;
class SWidget;
struct FGeometry;

/**
 * A widget that shows another widget as long as the mouse isn't hovering over it.
 */
class SDisappearingBar : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDisappearingBar)
	{
		_Visibility = EVisibility::HitTestInvisible;
	}

		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

private:
	FLinearColor GetFadeColorAndOpacity() const;

	FCurveSequence FadeCurve;
};
