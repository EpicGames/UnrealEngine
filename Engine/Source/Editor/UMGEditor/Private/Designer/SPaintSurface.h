// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Rendering/DrawElements.h"

struct FOnPaintHandlerParams;

struct FOnPaintHandlerParams
{
	const FPaintArgs& Args;
	const FGeometry& Geometry;
	const FSlateRect& ClippingRect;
	FSlateWindowElementList& OutDrawElements;
	const int32 Layer;
	const bool bEnabled;

	FOnPaintHandlerParams(const FPaintArgs& InArgs, const FGeometry& InGeometry, const FSlateRect& InClippingRect, FSlateWindowElementList& InOutDrawElements, int32 InLayer, bool bInEnabled )
		: Args(InArgs)
		, Geometry( InGeometry )
		, ClippingRect( InClippingRect )
		, OutDrawElements( InOutDrawElements )
		, Layer( InLayer )
		, bEnabled( bInEnabled )
	{
	}

};

/** Delegate type for allowing custom OnPaint handlers */
DECLARE_DELEGATE_RetVal_OneParam( 
	int32,
	FOnPaintHandler,
	const FOnPaintHandlerParams& );

/**
 * Widget with a handler for OnPaint, allows the designer to insert painting on different layers of the overlay when drawing widgets and effects
 * intermixed.
 */
class SPaintSurface : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPaintSurface )
		: _OnPaintHandler()
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		SLATE_EVENT( FOnPaintHandler, OnPaintHandler )
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs)
	{
		OnPaintHandler = InArgs._OnPaintHandler;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(128, 128);
	}

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		if( OnPaintHandler.IsBound() )
		{
			FOnPaintHandlerParams Params(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, bParentEnabled && IsEnabled() );
			OnPaintHandler.Execute( Params );
		}
		else
		{
			FSlateDrawElement::MakeDebugQuad(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry()
			);
		}

		return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled() );
	}

private:
	FOnPaintHandler OnPaintHandler;
};
