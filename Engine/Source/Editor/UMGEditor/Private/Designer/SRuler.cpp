// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/SRuler.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Geometry.h"
#include "Layout/SlateRect.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"

class FWidgetStyle;
struct FPointerEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "UMG"

namespace ScrubConstants
{
	/** The minimum amount of pixels between each major ticks on the widget */
	const int32 MinPixelsPerDisplayTick = 5;

	/**The smallest number of units between between major tick marks */
	const float MinDisplayTickSpacing = 0.1f;
}

/** Utility struct for converting between scrub range space and local/absolute screen space */
struct SRuler::FScrubRangeToScreen
{
	float RulerLengthSlateUnits;

	TRange<float> ViewInput;
	float ViewInputRange;
	float PixelsPerInput;

	FScrubRangeToScreen(TRange<float> InViewInput, const float InRulerLengthSlateUnits)
	{
		RulerLengthSlateUnits = InRulerLengthSlateUnits;

		ViewInput = InViewInput;
		ViewInputRange = ViewInput.Size<float>();
		PixelsPerInput = ViewInputRange > 0 ? ( RulerLengthSlateUnits / ViewInputRange ) : 0;

		// Round the float to the nearest four decimals, this keeps it stable as the user moves the ruler around
		// otherwise small variations will cause the spacing to change wildly as the user pans.
		PixelsPerInput = FMath::RoundToFloat(PixelsPerInput * 10000.0f) / 10000.0f;
	}

	/** Local Widget Space -> Curve Input domain. */
	float LocalXToInput(float ScreenX) const
	{
		float LocalX = ScreenX;
		return ( LocalX / PixelsPerInput ) + ViewInput.GetLowerBoundValue();
	}

	/** Curve Input domain -> local Widget Space */
	float InputToLocalX(float Input) const
	{
		return ( Input - ViewInput.GetLowerBoundValue() ) * PixelsPerInput;
	}
};

struct SRuler::FDrawTickArgs
{
	/** Geometry of the area */
	FGeometry AllottedGeometry;
	/** Clipping rect of the area */
	FSlateRect ClippingRect;
	/** Color of each tick */
	FLinearColor TickColor;
	/** Color of each tick */
	FLinearColor TextColor;
	/** Offset in Y where to start the tick */
	float TickOffset;
	/** Height in of major ticks */
	float MajorTickHeight;
	/** Start layer for elements */
	int32 StartLayer;
	/** Draw effects to apply */
	ESlateDrawEffect DrawEffects;
	/** Whether or not to only draw major ticks */
	bool bOnlyDrawMajorTicks;
};

/**
 * Gets the the next spacing value in the series 
 * to determine a good spacing value
 * E.g, .001,.005,.010,.050,.100,.500,1.000,etc
 */
static float GetNextSpacing( uint32 CurrentStep )
{
	if(CurrentStep & 0x01) 
	{
		// Odd numbers
		return FMath::Pow( 10.f, 0.5f*((float)(CurrentStep-1)) + 1.f );
	}
	else 
	{
		// Even numbers
		return 0.5f * FMath::Pow( 10.f, 0.5f*((float)(CurrentStep)) + 1.f );
	}
}

/////////////////////////////////////////////////////
// SRuler

void SRuler::Construct(const SRuler::FArguments& InArgs)
{
	Orientation = InArgs._Orientation;
	AbsoluteOrigin = FVector2D(0, 0);
	SlateToUnitScale = 1;

	MouseButtonDownHandler = InArgs._OnMouseButtonDown;
}

float SRuler::DetermineOptimalSpacing(float InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const
{
	if (InPixelsPerInput == 0.0f)
		return MinTickSpacing;
	
	uint32 CurStep = 0;

	// Start with the smallest spacing
	float Spacing = MinTickSpacing;

	while( Spacing * InPixelsPerInput < MinTick )
	{
		Spacing = MinTickSpacing * GetNextSpacing( CurStep );
		CurStep++;
	}

	return Spacing;
}

void SRuler::SetRuling(FVector2D InAbsoluteOrigin, float InSlateToUnitScale)
{
	AbsoluteOrigin = InAbsoluteOrigin;
	SlateToUnitScale = InSlateToUnitScale;
}

void SRuler::SetCursor(TOptional<FVector2D> InAbsoluteCursor)
{
	AbsoluteCursor = InAbsoluteCursor;
}

int32 SRuler::DrawTicks( FSlateWindowElementList& OutDrawElements, const struct FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const
{
	const float Spacing = DetermineOptimalSpacing( RangeToScreen.PixelsPerInput, ScrubConstants::MinPixelsPerDisplayTick, ScrubConstants::MinDisplayTickSpacing );

	// Sub divisions
	// @todo Sequencer may need more robust calculation
	const int32 Divider = 10;
	// For slightly larger halfway tick mark
	const int32 HalfDivider = Divider / 2;
	// Find out where to start from
	int32 OffsetNum = FMath::FloorToInt(RangeToScreen.ViewInput.GetLowerBoundValue() / Spacing);
	
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 7);

	TArray<FVector2D> LinePoints;
	LinePoints.AddUninitialized(2);

	// lines should not need anti-aliasing
	const bool bAntiAliasLines = false;

	float Number = 0;
	while ( ( Number = OffsetNum*Spacing ) < RangeToScreen.ViewInput.GetUpperBoundValue() )
	{
		// X position local to start of the widget area
		float XPos = RangeToScreen.InputToLocalX(Number);
		uint32 AbsOffsetNum = FMath::Abs(OffsetNum);

		if ( AbsOffsetNum % Divider == 0 )
		{
			FVector2D Offset = Orientation == Orient_Horizontal ? FVector2D(XPos, InArgs.TickOffset) : FVector2D(InArgs.TickOffset, XPos);
			FVector2D TickSize = Orientation == Orient_Horizontal ? FVector2D(1.0f, InArgs.MajorTickHeight) : FVector2D(InArgs.MajorTickHeight, 1.0f);

			LinePoints[0] = FVector2D(1.0f, 1.0f);
			LinePoints[1] = TickSize;

			// Draw each tick mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
				);

			if( !InArgs.bOnlyDrawMajorTicks )
			{
				float TextNumber = FMath::Abs(Number);

				FString FrameString = Spacing == ScrubConstants::MinDisplayTickSpacing ? FString::Printf(TEXT("%.1f"), TextNumber) : FString::Printf(TEXT("%.0f"), TextNumber);

				// If the orientation is vertical break the number up over multiple lines.
				if ( Orientation == Orient_Vertical )
				{
					for ( int32 CharIndex = 0; CharIndex < FrameString.Len() - 1; CharIndex += 2 )
					{
						FrameString.InsertAt(CharIndex + 1, '\n');
					}
				}

				// Space the text between the tick mark but slightly above
				const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);
				FVector2D TextOffset = Orientation == Orient_Horizontal ? 
					FVector2D(XPos - ( TextSize.X*0.5f ), FMath::Abs(InArgs.AllottedGeometry.GetLocalSize().Y - ( InArgs.MajorTickHeight + TextSize.Y )))
					:
					FVector2D(FMath::Abs(InArgs.AllottedGeometry.GetLocalSize().X - ( InArgs.MajorTickHeight + TextSize.X )), XPos - ( TextSize.Y*0.5f ));

				FSlateDrawElement::MakeText(
					OutDrawElements,
					InArgs.StartLayer, 
					InArgs.AllottedGeometry.ToPaintGeometry( TextSize, FSlateLayoutTransform(TextOffset) ), 
					FrameString, 
					SmallLayoutFont,
					InArgs.DrawEffects,
					InArgs.TextColor
				);
			}
		}
		else if( !InArgs.bOnlyDrawMajorTicks )
		{
			// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark
			const float MinorTickHeight = AbsOffsetNum % HalfDivider == 0 ? 7.0f : 4.0f;

			FVector2D Offset = Orientation == Orient_Horizontal ? FVector2D(XPos, FMath::Abs(InArgs.AllottedGeometry.GetLocalSize().Y - MinorTickHeight)) : FVector2D(FMath::Abs(InArgs.AllottedGeometry.GetLocalSize().X - MinorTickHeight), XPos);
			FVector2D TickSize = Orientation == Orient_Horizontal ? FVector2D(1, MinorTickHeight) : FVector2D(MinorTickHeight, 1);

			LinePoints[0] = FVector2D(1.0f,1.0f);
			LinePoints[1] = TickSize;

			// Draw each sub mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
			);
		}
		// Advance to next tick mark
		++OffsetNum;
	}

	// Draw line that runs along the bottom of all the ticks.
	{
		LinePoints[0] = Orientation == Orient_Horizontal ? FVector2D(0, InArgs.AllottedGeometry.Size.Y) : FVector2D(InArgs.AllottedGeometry.Size.X, 0);
		LinePoints[1] = FVector2D(InArgs.AllottedGeometry.Size.X, InArgs.AllottedGeometry.Size.Y);

		// Draw each sub mark
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InArgs.StartLayer,
			InArgs.AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			InArgs.DrawEffects,
			InArgs.TickColor,
			bAntiAliasLines
		);
	}

	// Draw the line that shows where the cursor is.
	if ( AbsoluteCursor.IsSet() )
	{
		FVector2D LocalCursor = InArgs.AllottedGeometry.AbsoluteToLocal(AbsoluteCursor.GetValue());

		LinePoints[0] = Orientation == Orient_Horizontal ? FVector2D(LocalCursor.X, 0) : FVector2D(0, LocalCursor.Y);
		LinePoints[1] = Orientation == Orient_Horizontal ? FVector2D(LocalCursor.X, InArgs.AllottedGeometry.Size.Y) : FVector2D(InArgs.AllottedGeometry.Size.X, LocalCursor.Y);

		// Draw each sub mark
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++InArgs.StartLayer,
			InArgs.AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			InArgs.DrawEffects,
			FLinearColor(FColor(0x19, 0xD1, 0x19)),
			bAntiAliasLines
		);
	}

	return InArgs.StartLayer;
}

int32 SRuler::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	static const FSlateBrush* WhiteBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	// Draw solid background
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		WhiteBrush,
		DrawEffects,
		FLinearColor(FColor(48,48,48))
	);

	FVector2f LocalOrigin = AllottedGeometry.AbsoluteToLocal(AbsoluteOrigin);

	float Scale = SlateToUnitScale < ScrubConstants::MinDisplayTickSpacing ? ScrubConstants::MinDisplayTickSpacing : SlateToUnitScale;
	
	float Origin = Orientation == Orient_Horizontal ? LocalOrigin.X : LocalOrigin.Y;
	float Min = 0 - ( Origin * SlateToUnitScale );
	float Max = Orientation == Orient_Horizontal ? AllottedGeometry.Size.X : AllottedGeometry.Size.Y;
	Max = (Max - Origin) * SlateToUnitScale;

	TRange<float> LocalViewRange(Min, Max);
	FScrubRangeToScreen RangeToScreen(LocalViewRange, Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y);

	const float MajorTickHeight = 9.0f;

	FDrawTickArgs TickArgs;
	TickArgs.AllottedGeometry = AllottedGeometry;
	TickArgs.bOnlyDrawMajorTicks = false;
	TickArgs.TickColor = FLinearColor(FColor(97, 97, 97));
	TickArgs.TextColor = FLinearColor(FColor(165, 165, 165));
	TickArgs.ClippingRect = MyCullingRect;
	TickArgs.DrawEffects = DrawEffects;
	TickArgs.StartLayer = LayerId + 1;
	TickArgs.TickOffset = Orientation == Orient_Horizontal ? FMath::Abs(AllottedGeometry.GetLocalSize().Y - MajorTickHeight) : FMath::Abs(AllottedGeometry.GetLocalSize().X - MajorTickHeight);
	TickArgs.MajorTickHeight = MajorTickHeight;

	LayerId = DrawTicks(OutDrawElements, RangeToScreen, TickArgs);

	return LayerId;
}

FReply SRuler::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseButtonDownHandler.IsBound() )
	{
		// If a handler is assigned, call it.
		return MouseButtonDownHandler.Execute(MyGeometry, MouseEvent);
	}
	else
	{
		// otherwise the event is unhandled.
		return FReply::Unhandled();
	}
}

FReply SRuler::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply SRuler::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FVector2D SRuler::ComputeDesiredSize( float ) const
{
	return Orientation == Orient_Horizontal ? FVector2D(100, 18) : FVector2D(18, 100);
}

FReply SRuler::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
