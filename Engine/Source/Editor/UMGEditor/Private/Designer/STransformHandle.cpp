// Copyright Epic Games, Inc. All Rights Reserved.

#include "Designer/STransformHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "Slate/WidgetTransform.h"
#include "Components/Widget.h"
#include "Components/CanvasPanelSlot.h"

#include "WidgetReference.h"
#include "IUMGDesigner.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// STransformHandle

void STransformHandle::Construct(const FArguments& InArgs, IUMGDesigner* InDesigner, ETransformDirection::Type InTransformDirection)
{
	TransformDirection = InTransformDirection;
	Designer = InDesigner;

	Action = ETransformAction::None;
	ScopedTransaction = nullptr;

	DragDirection = ComputeDragDirection(InTransformDirection);
	DragOrigin = ComputeOrigin(InTransformDirection);

	ChildSlot
	[
		SNew(SImage)
		.Visibility(this, &STransformHandle::GetHandleVisibility)
		.Image(FAppStyle::Get().GetBrush("UMGEditor.TransformHandle"))
	];
}

EVisibility STransformHandle::GetHandleVisibility() const
{
	ETransformMode::Type TransformMode = Designer->GetTransformMode();

	// Only show the handles for visible elements in the designer.
	FWidgetReference SelectedWidget = Designer->GetSelectedWidget();
	if ( SelectedWidget.IsValid() )
	{
		if ( !SelectedWidget.GetTemplate()->bHiddenInDesigner )
		{
			switch ( TransformMode )
			{
			case ETransformMode::Layout:
			{
				if ( UPanelSlot* TemplateSlot = SelectedWidget.GetTemplate()->Slot )
				{
					if ( CanResize(TemplateSlot, DragDirection) )
					{
						return EVisibility::Visible;
					}
				}
				break;
			}
			case ETransformMode::Render:
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

FReply STransformHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		Action = ComputeActionAtLocation(MyGeometry, MouseEvent);

		FWidgetReference SelectedWidget = Designer->GetSelectedWidget();
		UWidget* Preview = SelectedWidget.GetPreview();
		UWidget* Template = SelectedWidget.GetTemplate();

		if ( UCanvasPanelSlot* Slot = Cast<UCanvasPanelSlot>(Preview->Slot) )
		{
			StartingOffsets = Slot->GetOffsets();
		}

		MouseDownPosition = MouseEvent.GetScreenSpacePosition();

		ScopedTransaction = new FScopedTransaction(LOCTEXT("ResizeWidget", "Resize Widget"));
		Template->Modify();

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply STransformHandle::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( HasMouseCapture() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		const bool bRequiresRecompile = false;
		Designer->MarkDesignModifed(bRequiresRecompile);

		if ( ScopedTransaction )
		{
			delete ScopedTransaction;
			ScopedTransaction = nullptr;
		}

		Action = ETransformAction::None;
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply STransformHandle::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( Action != ETransformAction::None )
	{
		FWidgetReference SelectedWidget = Designer->GetSelectedWidget();

		UWidget* Template = SelectedWidget.GetTemplate();
		UWidget* Preview = SelectedWidget.GetPreview();

		{
			const FVector2D Delta = MouseEvent.GetScreenSpacePosition() - MouseDownPosition;
			const FVector2D TranslateAmount = Delta * (1.0f / (Designer->GetPreviewScale() * MyGeometry.Scale));

			Resize(Cast<UCanvasPanelSlot>(Preview->Slot), DragDirection, TranslateAmount);
			Resize(Cast<UCanvasPanelSlot>(Template->Slot), DragDirection, TranslateAmount);
		}

		ETransformMode::Type TransformMode = Designer->GetTransformMode();
		if ( TransformMode == ETransformMode::Render )
		{
			FWidgetTransform PreviewRenderTransform = Preview->GetRenderTransform();

			static const FName RenderTransformName(TEXT("RenderTransform"));

			FObjectEditorUtils::SetPropertyValue<UWidget, FWidgetTransform>(Preview, RenderTransformName, PreviewRenderTransform);
			FObjectEditorUtils::SetPropertyValue<UWidget, FWidgetTransform>(Template, RenderTransformName, PreviewRenderTransform);
		}
	}

	return FReply::Unhandled();
}

bool STransformHandle::CanResize(UPanelSlot* Slot, const FVector2D& Direction) const
{
	return Cast<UCanvasPanelSlot>(Slot) != nullptr;
}

void STransformHandle::Resize(UCanvasPanelSlot* Slot, const FVector2D& InDirection, const FVector2D& InAmount)
{
	if ( Slot == nullptr )
	{
		return;
	}

	FMargin Offsets = StartingOffsets;
	const FAnchorData LayoutData = Slot->GetLayout();
	FVector2f Alignment = UE::Slate::CastToVector2f(LayoutData.Alignment);

	FVector2f Direction = UE::Slate::CastToVector2f(InDirection);
	FVector2f Amount = UE::Slate::CastToVector2f(InAmount);	

	FVector2f Movement = Amount * Direction;
	FVector2f PositionMovement = Movement * ( FVector2f(1.0f, 1.0f) - Alignment);
	FVector2f SizeMovement = Movement;

	if ( Direction.X < 0 )
	{
		if ( LayoutData.Anchors.IsStretchedHorizontal() )
		{
			Offsets.Left -= Amount.X * Direction.X;
		}
		else
		{
			Offsets.Left -= PositionMovement.X;
			Offsets.Right += SizeMovement.X;
		}
	}

	if ( Direction.Y < 0 )
	{
		if ( LayoutData.Anchors.IsStretchedVertical() )
		{
			Offsets.Top -= Amount.Y * Direction.Y;
		}
		else
		{
			Offsets.Top -= PositionMovement.Y;
			Offsets.Bottom += SizeMovement.Y;
		}
	}

	if ( Direction.X > 0 )
	{
		if ( LayoutData.Anchors.IsStretchedHorizontal() )
		{
			Offsets.Right -= Amount.X * Direction.X;
		}
		else
		{
			Offsets.Left += (Movement * Alignment).X;
			Offsets.Right += Amount.X * Direction.X;
		}
	}

	if ( Direction.Y > 0 )
	{
		if ( LayoutData.Anchors.IsStretchedVertical() )
		{
			Offsets.Bottom -= Amount.Y * Direction.Y;
		}
		else
		{
			Offsets.Top += ( Movement * Alignment ).Y;
			Offsets.Bottom += Amount.Y * Direction.Y;
		}
	}

	FAnchorData NewLayoutData = LayoutData;
	NewLayoutData.Offsets = Offsets;
	static const FName LayoutDataName(TEXT("LayoutData"));
	FObjectEditorUtils::SetPropertyValue<UCanvasPanelSlot, FAnchorData>(Slot, LayoutDataName, NewLayoutData);
}

FCursorReply STransformHandle::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	ETransformAction CurrentAction = Action;
	if ( CurrentAction == ETransformAction::None )
	{
		CurrentAction = ComputeActionAtLocation(MyGeometry, MouseEvent);
	}

	ETransformMode::Type TransformMode = Designer->GetTransformMode();

	switch ( TransformDirection )
	{
	case ETransformDirection::TopLeft:
	case ETransformDirection::BottomRight:
		return FCursorReply::Cursor(EMouseCursor::ResizeSouthEast);
	case ETransformDirection::TopRight:
	case ETransformDirection::BottomLeft:
		return FCursorReply::Cursor(EMouseCursor::ResizeSouthWest);
	case ETransformDirection::TopCenter:
	case ETransformDirection::BottomCenter:
		return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
	case ETransformDirection::CenterLeft:
	case ETransformDirection::CenterRight:
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	return FCursorReply::Unhandled();
}

FVector2D STransformHandle::ComputeDragDirection(ETransformDirection::Type InTransformDirection) const
{
	switch ( InTransformDirection )
	{
	case ETransformDirection::TopLeft:
		return FVector2D(-1, -1);
	case ETransformDirection::TopCenter:
		return FVector2D(0, -1);
	case ETransformDirection::TopRight:
		return FVector2D(1, -1);

	case ETransformDirection::CenterLeft:
		return FVector2D(-1, 0);
	case ETransformDirection::CenterRight:
		return FVector2D(1, 0);

	case ETransformDirection::BottomLeft:
		return FVector2D(-1, 1);
	case ETransformDirection::BottomCenter:
		return FVector2D(0, 1);
	case ETransformDirection::BottomRight:
		return FVector2D(1, 1);
	}

	return FVector2D(0, 0);
}

FVector2D STransformHandle::ComputeOrigin(ETransformDirection::Type InTransformDirection) const
{
	FVector2D Size(10, 10);

	switch ( InTransformDirection )
	{
	case ETransformDirection::TopLeft:
		return Size * FVector2D(1, 1);
	case ETransformDirection::TopCenter:
		return Size * FVector2D(0.5, 1);
	case ETransformDirection::TopRight:
		return Size * FVector2D(0, 1);

	case ETransformDirection::CenterLeft:
		return Size * FVector2D(1, 0.5);
	case ETransformDirection::CenterRight:
		return Size * FVector2D(0, 0.5);

	case ETransformDirection::BottomLeft:
		return Size * FVector2D(1, 0);
	case ETransformDirection::BottomCenter:
		return Size * FVector2D(0.5, 0);
	case ETransformDirection::BottomRight:
		return Size * FVector2D(0, 0);
	}

	return FVector2D(0, 0);
}

ETransformAction STransformHandle::ComputeActionAtLocation(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	FVector2D GrabOriginOffset = LocalPosition - DragOrigin;
	if ( GrabOriginOffset.SizeSquared() < 36.f )
	{
		return ETransformAction::Primary;
	}
	else
	{
		return ETransformAction::Secondary;
	}
}

#undef LOCTEXT_NAMESPACE
