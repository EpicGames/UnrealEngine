// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IUMGDesigner;
class UCanvasPanelSlot;
class UPanelSlot;
struct FGeometry;
struct FPointerEvent;

namespace ETransformDirection
{
	enum Type
	{
		TopLeft = 0,
		TopCenter,
		TopRight,

		CenterLeft,
		CenterRight,

		BottomLeft,
		BottomCenter,
		BottomRight,

		MAX
	};
}


enum class ETransformAction
{
	None,
	Primary,
	Secondary
};

class STransformHandle : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STransformHandle) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, IUMGDesigner* InDesigner, ETransformDirection::Type InTransformDirection);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	// End SWidget interface
	
protected:
	EVisibility GetHandleVisibility() const;

	bool CanResize(UPanelSlot* Slot, const FVector2D& Direction) const;
	void Resize(UCanvasPanelSlot* Slot, const FVector2D& Direction, const FVector2D& Amount);

	ETransformAction ComputeActionAtLocation(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;

protected:
	FVector2D ComputeDragDirection(ETransformDirection::Type InTransformDirection) const;
	FVector2D ComputeOrigin(ETransformDirection::Type InTransformDirection) const;

protected:
	IUMGDesigner* Designer;
	ETransformDirection::Type TransformDirection;
	ETransformAction Action;

	FVector2D DragDirection;
	FVector2D DragOrigin;

	FVector2D MouseDownPosition;
	FMargin StartingOffsets;

	class FScopedTransaction* ScopedTransaction;
};
