// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/DetailWidgetExtensionHandler.h"
#include "Binding/WidgetBinding.h"
#include "Customizations/UMGDetailCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "Details/WidgetPropertyDragDropHandler.h"
#include "Engine/Blueprint.h"
#include "UMGEditorModule.h"
#include "UMGEditorProjectSettings.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

FDetailWidgetExtensionHandler::FDetailWidgetExtensionHandler(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: BlueprintEditor( InBlueprintEditor )
{}

namespace Private
{
	FDelegateProperty* FindDelegateProperty(const IPropertyHandle& PropertyHandle)
	{
		// Make the extension handler run for child struct/array properties if the parent is bindable.
		// This is so we can later disable any child rows of a bound property in ExtendWidgetRow().
		TSharedPtr<const IPropertyHandle> ParentPropertyHandle;
		for (TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = PropertyHandle.AsShared();
			CurrentPropertyHandle && CurrentPropertyHandle->GetProperty();
			CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle())
		{
			ParentPropertyHandle = CurrentPropertyHandle;
		}

		// Check to see if there's a delegate for the parent property.
		if (ParentPropertyHandle)
		{
			FProperty* ParentProperty = ParentPropertyHandle->GetProperty();
			FString DelegateName = ParentProperty->GetName() + "Delegate";

			if (UClass* ContainerClass = ParentProperty->GetOwner<UClass>())
			{
				FDelegateProperty* DelegateProperty = FindFProperty<FDelegateProperty>(ContainerClass, FName(*DelegateName));
				return DelegateProperty;
			}
		}

		return nullptr;
	}

	bool ShouldShowOldBindingWidget(const UWidgetBlueprint* WidgetBlueprint, const IPropertyHandle& PropertyHandle)
	{
		FProperty* Property = PropertyHandle.GetProperty();
		FString DelegateName = Property->GetName() + "Delegate";
		FDelegateProperty* DelegateProperty = FindDelegateProperty(PropertyHandle);
		if (DelegateProperty == nullptr)
		{
			return false;
		}

		const bool bIsEditable = Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst);
		const bool bDoSignaturesMatch = DelegateProperty->SignatureFunction->GetReturnProperty()->SameType(Property);

		if (!bIsEditable || !bDoSignaturesMatch)
		{
			return false;
		}

		if (!WidgetBlueprint->ArePropertyBindingsAllowed())
		{
			// Even if they don't want them on, we need to show them so they can remove them if they had any.
			if (WidgetBlueprint->Bindings.Num() == 0)
			{
				return false;
			}
		}

		return true;
	}
}

bool FDetailWidgetExtensionHandler::IsPropertyExtendable(const UClass* ObjectClass, const IPropertyHandle& PropertyHandle) const
{
	if (PropertyHandle.GetNumOuterObjects() != 1)
	{
		return false;
	}

	TArray<UObject*> Objects;
	PropertyHandle.GetOuterObjects(Objects);

	// We don't allow bindings on the CDO.
	if (Objects[0] != nullptr && Objects[0]->HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	if (BPEd == nullptr)
	{
		return false;
	}

	if (FDelegateProperty* DelegateProperty = Private::FindDelegateProperty(PropertyHandle))
	{
		return true;
	}

	const UWidget* Widget = Cast<UWidget>(Objects[0]);
	if (Widget == nullptr)
	{
		return false;
	}

	const UWidgetBlueprint* WidgetBlueprint = BPEd->GetWidgetBlueprintObj();

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	for (const TSharedPtr<IPropertyBindingExtension>& Extension : EditorModule.GetPropertyBindingExtensibilityManager()->GetExtensions())
	{
		if (Extension->CanExtend(WidgetBlueprint, Widget, PropertyHandle.AsShared()))
		{
			return true;
		}
	}

	return false;
}

void FDetailWidgetExtensionHandler::ExtendWidgetRow(
	FDetailWidgetRow& InWidgetRow,
	const IDetailLayoutBuilder& InDetailBuilder,
	const UClass* InObjectClass,
	TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	UWidgetBlueprint* WidgetBlueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
	const TWeakPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = BlueprintEditor;
	const TSharedRef<IPropertyHandle> PropertyHandleRef = InPropertyHandle.ToSharedRef();
	const IPropertyHandle& PropertyHandle = PropertyHandleRef.Get();

	const bool bShouldShowOldBindingWidget = Private::ShouldShowOldBindingWidget(WidgetBlueprint, PropertyHandle);

	UUMGEditorProjectSettings* UMGEditorProjectSettings = GetMutableDefault<UUMGEditorProjectSettings>();
	const bool bShouldReplaceValue = UMGEditorProjectSettings->bUseNewBindingFromDetailsView && !FBlueprintWidgetCustomization::IsBindableEvent(PropertyHandle.GetProperty());

	bool bShouldShowWidget = false;
	bShouldShowWidget |= bShouldShowOldBindingWidget;

	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	for (const UObject* Object : Objects)
	{
		const UWidget* Widget = Cast<UWidget>(Object);
		if (Widget == nullptr)
		{
			continue;
		}

		IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		for (const TSharedPtr<IPropertyBindingExtension>& Extension : EditorModule.GetPropertyBindingExtensibilityManager()->GetExtensions())
		{
			bShouldShowWidget |= Extension->CanExtend(WidgetBlueprint, Widget, InPropertyHandle);
		}
	}

	if (bShouldReplaceValue)
	{	//With the new binding picker mode, the current value is replaced by a binding picker, so we want to keep it enabled.
		InWidgetRow.IsValueEnabled(true);
	}
	else
	{
		InWidgetRow.IsValueEnabled(TAttribute<bool>::Create(
			[BlueprintEditorPtr, PropertyHandleRef]() {
				// NOTE: HasPropertyBindings is potentially expensive (see its implementation) and this will
				// get called for every bindable property's row, on every frame.
				// However, this is not currently a big concern because we only extend the widget row
				// in cases where there is one OuterObject (see IsPropertyExtendable above).
				// But the performance might degrade if we extend the widget row with the property binding
				// widget for a multiple-selection.
				constexpr bool bOnlyEditableValue = true;
				return !FBlueprintWidgetCustomization::HasPropertyBindings(BlueprintEditorPtr, PropertyHandleRef, bOnlyEditableValue);
			}));
	}

	if (!bShouldShowWidget)
	{
		return;
	}

	if (Objects.Num() > 0)
	{
		if (UWidget* Widget = Cast<UWidget>(Objects[0]))
		{
			InWidgetRow.DragDropHandler(MakeShared<FWidgetPropertyDragDropHandler>(Widget, InPropertyHandle, WidgetBlueprint));

			if (Widget != BlueprintEditor.Pin()->GetPreview())
			{
				UFunction* SignatureFunction = nullptr;
				if (FDelegateProperty* DelegateProperty = Private::FindDelegateProperty(PropertyHandleRef.Get()))
				{
					SignatureFunction = DelegateProperty->SignatureFunction;
				}

				TWeakPtr<const IDetailsView> DetailsView = InDetailBuilder.GetDetailsViewSharedPtr().ToWeakPtr();
				InWidgetRow.ExtensionContent()
					[
						FBlueprintWidgetCustomization::MakePropertyBindingWidget(BlueprintEditor, SignatureFunction, InPropertyHandle.ToSharedRef(), ConstCastWeakPtr<IDetailsView>(DetailsView), true, bShouldShowOldBindingWidget, FBlueprintWidgetCustomization::EPropertyBindingMode::Extension)
					];

				//Override the value content if a property is bound.
				if (bShouldReplaceValue && FBlueprintWidgetCustomization::HasPropertyBindings(BlueprintEditorPtr, PropertyHandleRef))
				{
					InWidgetRow.ValueContent()
						[
							FBlueprintWidgetCustomization::MakePropertyBindingWidget(BlueprintEditor, SignatureFunction, InPropertyHandle.ToSharedRef(), ConstCastWeakPtr<IDetailsView>(DetailsView), true, bShouldShowOldBindingWidget, FBlueprintWidgetCustomization::EPropertyBindingMode::Value)
						];
				}
			}
		}
	}
}
