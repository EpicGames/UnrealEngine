// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailPropertyExtensionHandler.h"

class FWidgetBlueprintEditor;

class FDetailWidgetExtensionHandler : public IDetailPropertyExtensionHandler
{
public:
	FDetailWidgetExtensionHandler(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;

	virtual void ExtendWidgetRow(
		FDetailWidgetRow& InWidgetRow,
		const IDetailLayoutBuilder& InDetailBuilder,
		const UClass* InObjectClass,
		TSharedPtr<IPropertyHandle> PropertyHandle) override;

private:
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
};
