// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/SFunctionSelector.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#if WITH_EDITOR
	#include "Components/PrimitiveComponent.h"
	#include "Components/StaticMeshComponent.h"
	#include "Engine/BlueprintGeneratedClass.h"
#endif // WITH_EDITOR
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Animation/WidgetAnimation.h"
#include "WidgetBlueprint.h"

#include "DetailLayoutBuilder.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "WidgetGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Components/WidgetComponent.h"
#include "Binding/PropertyBinding.h"
#include "WidgetBlueprintEditor.h"


#define LOCTEXT_NAMESPACE "SFunctionSelector"


/////////////////////////////////////////////////////
// SFunctionSelector

void SFunctionSelector::Construct(const FArguments& InArgs, TSharedRef<FWidgetBlueprintEditor> InEditor, UFunction* InAllowedSignature)
{
	Editor = InEditor;
	Blueprint = InEditor->GetWidgetBlueprintObj();

	CurrentFunction = InArgs._CurrentFunction;
	SelectedFunctionEvent = InArgs._OnSelectedFunction;
	ResetFunctionEvent = InArgs._OnResetFunction;

	BindableSignature = InAllowedSignature;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SFunctionSelector::OnGenerateDelegateMenu)
			.ContentPadding(1.0f)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 1.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(this, &SFunctionSelector::GetCurrentBindingText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.Visibility(this, &SFunctionSelector::GetGotoBindingVisibility)
			.OnClicked(this, &SFunctionSelector::HandleGotoBindingClicked)
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("GotoFunction", "Goto Function"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Search"))
			]
		]
	];
}

template <typename Predicate>
void SFunctionSelector::ForEachBindableFunction(UClass* FromClass, Predicate Pred) const
{
	// Walk up class hierarchy for native functions and properties
	for ( TFieldIterator<UFunction> FuncIt(FromClass, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt )
	{
		UFunction* Function = *FuncIt;

		// Only bind to functions that are callable from blueprints
		if ( !UEdGraphSchema_K2::CanUserKismetCallFunction(Function) )
		{
			continue;
		}

		// We ignore CPF_ReturnParm because all that matters for binding to script functions is that the number of out parameters match.
		if ( Function->IsSignatureCompatibleWith(BindableSignature, UFunction::GetDefaultIgnoredSignatureCompatibilityFlags() | CPF_ReturnParm) )
		{
			TSharedPtr<FFunctionInfo> Info = MakeShareable(new FFunctionInfo());
			Info->DisplayName = FText::FromName(Function->GetFName());
			Info->Tooltip = Function->GetMetaData("Tooltip");
			Info->FuncName = Function->GetFName();

			Pred(Info);
		}
	}
}

TSharedRef<SWidget> SFunctionSelector::OnGenerateDelegateMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("BindingActions");
	{
		if ( CanReset() )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ResetFunction", "Reset"),
				LOCTEXT("ResetFunctionTooltip", "Reset this function and clear it out."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cross"),
				FUIAction(FExecuteAction::CreateSP(this, &SFunctionSelector::HandleRemoveBinding))
			);
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateFunction", "Create Function"),
			LOCTEXT("CreateBindingToolTip", "Creates a new function"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Plus"),
			FUIAction(FExecuteAction::CreateSP(this, &SFunctionSelector::HandleCreateAndAddBinding))
		);
	}
	MenuBuilder.EndSection(); //CreateBinding

	// Bindable options
	{
		// Get the current skeleton class, think header for the blueprint.
		UBlueprintGeneratedClass* SkeletonClass = Cast<UBlueprintGeneratedClass>(Blueprint->SkeletonGeneratedClass);

		FillPropertyMenu(MenuBuilder, SkeletonClass);
	}

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	return
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.MaxHeight(DisplayMetrics.PrimaryDisplayHeight * 0.5)
		[
			MenuBuilder.MakeWidget()
		];
}

void SFunctionSelector::FillPropertyMenu(FMenuBuilder& MenuBuilder, UStruct* OwnerStruct)
{
	bool bFoundEntry = false;

	//---------------------------------------
	// Function Bindings

	if ( UClass* OwnerClass = Cast<UClass>(OwnerStruct) )
	{
		static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));

		MenuBuilder.BeginSection("Functions", LOCTEXT("Functions", "Functions"));
		{
			ForEachBindableFunction(OwnerClass, [&] (TSharedPtr<FFunctionInfo> Info) {
				bFoundEntry = true;

				MenuBuilder.AddMenuEntry(
					Info->DisplayName,
					FText::FromString(Info->Tooltip),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), FunctionIcon),
					FUIAction(FExecuteAction::CreateSP(this, &SFunctionSelector::HandleAddFunctionBinding, Info))
					);
			});
		}
		MenuBuilder.EndSection(); //Functions
	}

	// Get the current skeleton class, think header for the blueprint.
	UBlueprintGeneratedClass* SkeletonClass = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass);

	if ( bFoundEntry == false && OwnerStruct != SkeletonClass )
	{
		MenuBuilder.BeginSection("None", OwnerStruct->GetDisplayNameText());
		MenuBuilder.AddWidget(SNew(STextBlock).Text(LOCTEXT("None", "None")), FText::GetEmpty());
		MenuBuilder.EndSection(); //None
	}
}

FText SFunctionSelector::GetCurrentBindingText() const
{
	TOptional<FName> Current = CurrentFunction.Get();

	if (Current.IsSet())
	{
		if (Current.GetValue() == NAME_None)
		{
			return LOCTEXT("SelectFunction", "Select Function");
		}
		else
		{
			return FText::FromName(Current.GetValue());
		}
	}

	return LOCTEXT("MultipleValues", "Multiple Values");
}

bool SFunctionSelector::CanReset()
{
	TOptional<FName> Current = CurrentFunction.Get();
	if (Current.IsSet())
	{
		return Current.GetValue() != NAME_None;
	}

	return true;
}

void SFunctionSelector::HandleRemoveBinding()
{
	ResetFunctionEvent.ExecuteIfBound();
}

void SFunctionSelector::HandleAddFunctionBinding(TSharedPtr<FFunctionInfo> SelectedFunction)
{
	SelectedFunctionEvent.ExecuteIfBound(SelectedFunction->FuncName);
}

void SFunctionSelector::HandleCreateAndAddBinding()
{
	const FScopedTransaction Transaction(LOCTEXT("CreateDelegate", "Create Binding"));

	Blueprint->Modify();

	// Create the function graph.
	FString FunctionName = TEXT("DoCustomNavigation"); // TODO Change this to make it generic and exposed as a parameter.
	UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(
		Blueprint, 
		FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, FunctionName),
		UEdGraph::StaticClass(),
		UEdGraphSchema_K2::StaticClass());
	
	// Add the binding to the blueprint
	TSharedPtr<FFunctionInfo> SelectedFunction = MakeShareable(new FFunctionInfo());
	SelectedFunction->FuncName = FunctionGraph->GetFName();

	HandleAddFunctionBinding(SelectedFunction);

	const bool bUserCreated = true;
	FBlueprintEditorUtils::AddFunctionGraph(Blueprint, FunctionGraph, bUserCreated, BindableSignature);

	GotoFunction(FunctionGraph);
}

EVisibility SFunctionSelector::GetGotoBindingVisibility() const
{
	TOptional<FName> Current = CurrentFunction.Get();
	if (Current.IsSet())
	{
		if (Current.GetValue() != NAME_None)
		{
			return EVisibility::Visible;
		}
	}
	
	return EVisibility::Collapsed;
}

FReply SFunctionSelector::HandleGotoBindingClicked()
{
	TOptional<FName> Current = CurrentFunction.Get();
	if (ensure(Current.IsSet()))
	{
		if (ensure(Current.GetValue() != NAME_None))
		{
			TArray<UEdGraph*> AllGraphs;
			Blueprint->GetAllGraphs(AllGraphs);

			for (UEdGraph* Graph : AllGraphs)
			{
				if ( Graph->GetFName() == Current)
				{
					GotoFunction(Graph);
				}
			}
		}
	}

	return FReply::Handled();
}

void SFunctionSelector::GotoFunction(UEdGraph* FunctionGraph)
{
	Editor.Pin()->SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);
	Editor.Pin()->OpenDocument(FunctionGraph, FDocumentTracker::OpenNewDocument);
}


#undef LOCTEXT_NAMESPACE
