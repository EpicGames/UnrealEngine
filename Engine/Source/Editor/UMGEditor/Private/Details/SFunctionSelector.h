// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "WidgetBlueprintEditor.h"
#include "EdGraph/EdGraphSchema.h"
#include "PropertyHandle.h"

class FMenuBuilder;
class UEdGraph;
class UWidgetBlueprint;
struct FEditorPropertyPath;

class SFunctionSelector : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FFunctionDelegate, FName /*SelectedFunctionName*/);
	DECLARE_DELEGATE(FResetDelegate);

	SLATE_BEGIN_ARGS(SFunctionSelector)
		{}

	SLATE_ATTRIBUTE(TOptional<FName>, CurrentFunction)

	SLATE_EVENT(FFunctionDelegate, OnSelectedFunction)
	SLATE_EVENT(FResetDelegate, OnResetFunction)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FWidgetBlueprintEditor> InEditor, UFunction* InAllowedSignature);

protected:
	struct FFunctionInfo
	{
		FFunctionInfo()
		{
		}

		FName FuncName;
		FText DisplayName;
		FString Tooltip;
	};

	TSharedRef<SWidget> OnGenerateDelegateMenu();
	void FillPropertyMenu(FMenuBuilder& MenuBuilder, UStruct* OwnerStruct);

	FText GetCurrentBindingText() const;

	bool CanReset();
	void HandleRemoveBinding();

	void HandleAddFunctionBinding(TSharedPtr<FFunctionInfo> SelectedFunction);

	void HandleCreateAndAddBinding();
	void GotoFunction(UEdGraph* FunctionGraph);

	EVisibility GetGotoBindingVisibility() const;

	FReply HandleGotoBindingClicked();

private:

	template <typename Predicate>
	void ForEachBindableFunction(UClass* FromClass, Predicate Pred) const;

	TAttribute<TOptional<FName>> CurrentFunction;

	FFunctionDelegate SelectedFunctionEvent;
	FResetDelegate ResetFunctionEvent;

	TWeakPtr<FWidgetBlueprintEditor> Editor;
	UWidgetBlueprint* Blueprint;

	UFunction* BindableSignature;
};
