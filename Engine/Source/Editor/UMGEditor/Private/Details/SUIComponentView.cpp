// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUIComponentView.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "ScopedTransaction.h"
#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/DragAndDrop.h"
#include "Kismet2/SClassPickerDialog.h"
#include "SPositiveActionButton.h"
#include "UIComponentUtils.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#if WITH_EDITOR
#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "UMG"

struct FUIComponentListItem
{
	explicit FUIComponentListItem(UUIComponent* InComponent, UWidget* InWidget)
		: Component(InComponent),
		  Widget(InWidget)
	{}

	TWeakObjectPtr<UUIComponent> Component;
	TWeakObjectPtr<UWidget> Widget;
};

class FUIComponentDragDropOp final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUIComponentDragDropOp, FDecoratedDragDropOp)

	/** The template to create an instance */
	TSharedPtr<FUIComponentListItem> ListItem;

	/** Constructs the drag drop operation */
	static TSharedRef<FUIComponentDragDropOp> New(const TSharedPtr<FUIComponentListItem>& InListItem, FText InDragText)
	{
		TSharedRef<FUIComponentDragDropOp> Operation = MakeShared<FUIComponentDragDropOp>();
		Operation->ListItem = InListItem;
		Operation->DefaultHoverText = InDragText;
		Operation->CurrentHoverText = InDragText;
		Operation->Construct();

		return Operation;
	}
};

class SUIComponentListItem : public STableRow<TSharedPtr<FUIComponentListItem> >
{
public:
	SLATE_BEGIN_ARGS(SUIComponentListView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, TSharedPtr<FUIComponentListItem> InListItem)
	{
		ListItem = InListItem;
		BlueprintEditor = InBlueprintEditor;

		STableRow<TSharedPtr<FUIComponentListItem>>::Construct(
			STableRow<TSharedPtr<FUIComponentListItem>>::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			.Padding(FMargin(3.0f, 2.0f))
			.OnDragDetected(this, &SUIComponentListItem::OnDragDetected)
			.OnCanAcceptDrop(this, &SUIComponentListItem::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SUIComponentListItem::OnAcceptDrop)
			.Content()
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
				.Text(this, &SUIComponentListItem::GetComponentName)
				.IsReadOnly(true)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.IsSelected(this, &SUIComponentListItem::IsSelectedExclusively)
			],
			InOwnerTableView);
	}

	void BeginRename()
	{
		InlineTextBlock->EnterEditingMode();
	}

private:

	FText GetComponentName() const
	{
		if( TSharedPtr<FUIComponentListItem> Item = ListItem.Pin() )
		{
			if (Item->Component != nullptr)
			{
				return Item->Component->GetClass()->GetDisplayNameText();
			}
		}

		return FText::GetEmpty();
	}

	/** Called whenever a drag is detected by the list view. */
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
	{
		TSharedPtr<FUIComponentListItem> ListItemPinned = ListItem.Pin();
		if (ListItemPinned.IsValid())
		{
			FText DefaultText = LOCTEXT("DefaultDragDropFormat", "Move 1 item(s)");
			return FReply::Handled().BeginDragDrop(FUIComponentDragDropOp::New(ListItemPinned, DefaultText));
		}
		return FReply::Unhandled();
	}

	/** Called to determine whether a current drag operation is valid for this row. */
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FUIComponentListItem> InListItem)
	{
		TSharedPtr<FUIComponentDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FUIComponentDragDropOp>();
		if (DragDropOp.IsValid())
		{
			if (InItemDropZone == EItemDropZone::OntoItem)
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			}
			else
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok"));
			}
			return InItemDropZone;
		}
		return TOptional<EItemDropZone>();
	}

	/** Called to complete a drag and drop onto this drop. */
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FUIComponentListItem> InListItem)
	{
		TSharedPtr<FUIComponentDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FUIComponentDragDropOp>();
		if (DragDropOp.IsValid()
			&& DragDropOp->ListItem.IsValid() 
			&& DragDropOp->ListItem->Component != nullptr
			&& InListItem.IsValid() 
			&& InListItem->Component != nullptr
			&& DragDropOp->ListItem->Component != InListItem->Component
			&& InListItem->Widget != nullptr
			)
		{
			
			TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
			if (UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor ? WidgetBlueprintEditor->GetWidgetBlueprintObj() : nullptr)
			{
				// When we drop on another item, we expect to replace it, so we move it before that Item.
				// So we move it after only if it's Below
				const bool bMoveAfter = InItemDropZone == EItemDropZone::BelowItem;
				
				FText MoveErrorMessage;
				if (!FUIComponentUtils::MoveComponent(WidgetBlueprintEditor.Get(),
					WidgetBlueprint,
					DragDropOp->ListItem->Component.Get()->GetClass(),
					InListItem->Component.Get()->GetClass(),
					InListItem->Widget.Get()->GetFName(),
					bMoveAfter,
					MoveErrorMessage
					))
				{
					UE_LOGF(LogUMGEditor, Error, "%ls", *MoveErrorMessage.ToString());
				}
								
				return FReply::Handled();
			}
		}
		return FReply::Unhandled();
	}

private:
	TWeakPtr<FUIComponentListItem> ListItem;
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;
};

SUIComponentView::~SUIComponentView()
{
	if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
	{
		WidgetBlueprintEditorPin->OnWidgetPreviewReady.RemoveAll(this);		
		WidgetBlueprintEditorPin->OnSelectedWidgetsChanged.RemoveAll(this);
		WidgetBlueprintEditorPin->GetOnWidgetBlueprintTransaction().RemoveAll(this);
	}
}

void SUIComponentView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;

	InBlueprintEditor->GetOnWidgetBlueprintTransaction().AddSP(this, &SUIComponentView::OnWidgetBlueprintTransaction);
	InBlueprintEditor->OnSelectedWidgetsChanged.AddSP(this, &SUIComponentView::OnEditorSelectionChanged);
	InBlueprintEditor->OnWidgetPreviewReady.AddSP(this, &SUIComponentView::OnWidgetPreviewReady);
		
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(4.0f, 0.0f)
		[
			SNew(SPositiveActionButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.Text(LOCTEXT("AddComponentLabel", "Add Component"))
			.ToolTipText(LOCTEXT("AddComponentToolTip", "Add a Component to this widget."))		
			.OnClicked(this, &SUIComponentView::OnAddComponentButtonClicked)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()		
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[			
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(4.f)		
			[	SNew(SBox)
				.MinDesiredHeight(60)
				[
					SNew(SScrollBorder, ComponentListView.ToSharedRef())
					[
						SAssignNew(ComponentListView, SUIComponentListView)
						.SelectionMode(ESelectionMode::Single)
						.OnGenerateRow(this, &SUIComponentView::OnGenerateWidgetForComponent)
						.OnContextMenuOpening(this, &SUIComponentView::OnContextMenuOpening)
						.ListItemsSource(&Components)
					]
				]
			]			
		]		
	];

	UpdateComponentList();
	CreateCommandList();
}

void SUIComponentView::CreateCommandList()
{
	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SUIComponentView::OnDeleteSelectedComponent),
		FCanExecuteAction::CreateSP(this, &SUIComponentView::CanExecuteDeleteSelectedComponent)
	);
}

FReply SUIComponentView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		Reply = FReply::Handled();
	}

	return Reply;
}

void SUIComponentView::OnEditorSelectionChanged()
{
	// When the selection changes in the Widget Designer, we update the component List
	UpdateComponentList();
}
void SUIComponentView::OnWidgetPreviewReady()
{
	// When the the preview is recreated and is ready we update the component List
	UpdateComponentList();
}

void SUIComponentView::OnWidgetBlueprintTransaction()
{	
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin();
	UpdateComponentList();
}

FReply SUIComponentView::OnAddComponentButtonClicked()
{
	if (const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin())
	{
		// Gather classes already present on the selected widget so the class picker can hide related classes.
		TSet<const UClass*> ExistingComponentClasses;
		if (const UUserWidget* PreviewUserWidget = WidgetBlueprintEditor->GetPreview())
		{
			if (const UUIComponentUserWidgetExtension* Extension = PreviewUserWidget->GetExtension<UUIComponentUserWidgetExtension>())
			{
				if (UWidget* SelectedWidget = GetSelectedWidget())
				{
					for (const UUIComponent* Component : Extension->GetComponentsFor(SelectedWidget))
					{
						if (Component)
						{
							ExistingComponentClasses.Add(Component->GetClass());
						}
					}
				}
			}
		}

		UClass* ChosenClass = nullptr;
		const FText TitleText = LOCTEXT("AddComponentClassPickerTitle", "Pick a component to add to the widget.");
		const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, FUIComponentUtils::CreateClassViewerInitializationOptions(ExistingComponentClasses), ChosenClass, UUIComponent::StaticClass());
		if (ChosenClass)
		{
			const TSet<FWidgetReference>& SelectedWidgets = WidgetBlueprintEditor->GetSelectedWidgets();
			for (const FWidgetReference& SelectedObject : SelectedWidgets)
			{
				if (const UWidget* SelectedWidget = Cast<UWidget>(SelectedObject.GetPreview()))
				{
					UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
					FText AddErrorMessage;
					if (!FUIComponentUtils::AddComponent(WidgetBlueprintEditor.Get(), WidgetBlueprint, ChosenClass, SelectedWidget->GetFName(), AddErrorMessage))
					{
						UE_LOGF(LogUMGEditor, Error, "%ls", *AddErrorMessage.ToString());
					}
				}
			}
		}
	}
	return FReply::Handled();
}

TSharedPtr<SWidget> SUIComponentView::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

	MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SUIComponentView::CanExecuteDeleteSelectedComponent() const
{
	if (ComponentListView)
	{

		return ComponentListView->GetNumItemsSelected() > 0;
	}	

	return false;
}

void SUIComponentView::OnDeleteSelectedComponent()
{
	if (!ComponentListView)
	{
		return;
	}
	
	if (const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin())
	{
		if (UWidget* SelectedWidget = GetSelectedWidget())
		{
			for (TSharedPtr<FUIComponentListItem>& Item : ComponentListView->GetSelectedItems())
			{
				if (Item && Item->Component != nullptr)
				{
					UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj();
					FText RemoveErrorMessage;
					if (!FUIComponentUtils::RemoveComponent(WidgetBlueprintEditor.Get(), WidgetBlueprint, Item->Component->GetClass(), SelectedWidget->GetFName(), RemoveErrorMessage))
					{
						UE_LOGF(LogUMGEditor, Error, "%ls", *RemoveErrorMessage.ToString());
					}
				}
			}
		}		 
	}
}

TSharedRef<ITableRow> SUIComponentView::OnGenerateWidgetForComponent(TSharedPtr<FUIComponentListItem> InListItem, const TSharedRef< STableViewBase >& InOwnerTableView)
{
	return SNew(SUIComponentListItem, InOwnerTableView, BlueprintEditor.Pin(), InListItem);
}

UWidget* SUIComponentView::GetSelectedWidget() const
{
	UWidget* SelectedPreviewWidget = nullptr;
	if (const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin())
	{
		const TSet<FWidgetReference>& SelectedWidgets = WidgetBlueprintEditor->GetSelectedWidgets();
		if (SelectedWidgets.Num() == 1)
		{
			// Get the first element.
			const TSet<FWidgetReference>::TConstIterator SetIt(SelectedWidgets);
			SelectedPreviewWidget = SetIt->GetPreview();
		}
	}
	return SelectedPreviewWidget;
}

void SUIComponentView::UpdateComponentList()
{	
	Components.Reset();
	
	if (const TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin())
	{
		UWidget* SelectedPreviewWidget = GetSelectedWidget();
		const UUserWidget* PreviewUserWidget = WidgetBlueprintEditor->GetPreview();
		const UUIComponentUserWidgetExtension* UserWidgetExtension = PreviewUserWidget? PreviewUserWidget->GetExtension<UUIComponentUserWidgetExtension>() : nullptr;
					
		if (SelectedPreviewWidget && UserWidgetExtension)
		{
			TArray<UUIComponent*> ListOfComponents =  UserWidgetExtension->GetComponentsFor(SelectedPreviewWidget);
			Components.Reserve(ListOfComponents.Num());
			for (UUIComponent* Component : ListOfComponents)
			{
				Components.Emplace(MakeShared<FUIComponentListItem>(Component, SelectedPreviewWidget));
			}
		}		
	}
	if (ComponentListView.IsValid())
	{
		ComponentListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE 