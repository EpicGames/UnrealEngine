// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"

class FWidgetBlueprintEditor;
class FUICommandList;
struct FUIComponentListItem;
class UWidget;

typedef SListView<TSharedPtr<FUIComponentListItem> > SUIComponentListView;

class SUIComponentView final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUIComponentView) {}
	SLATE_END_ARGS()

	~SUIComponentView();

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	void CreateCommandList();
	
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:	
	void OnEditorSelectionChanged();
	void OnWidgetPreviewReady();	
	void OnWidgetBlueprintTransaction();
	
	FReply OnAddComponentButtonClicked();

	TSharedPtr<SWidget> OnContextMenuOpening();

	bool CanExecuteDeleteSelectedComponent() const;
	void OnDeleteSelectedComponent();
	
	TSharedRef<ITableRow> OnGenerateWidgetForComponent(TSharedPtr<FUIComponentListItem> InListItem, const TSharedRef< STableViewBase >& InOwnerTableView);
	
	UWidget* GetSelectedWidget() const;
	void UpdateComponentList();
	
	TSharedPtr<FUICommandList> CommandList;
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<SUIComponentListView> ComponentListView;
	TArray<TSharedPtr<FUIComponentListItem> > Components;
};
