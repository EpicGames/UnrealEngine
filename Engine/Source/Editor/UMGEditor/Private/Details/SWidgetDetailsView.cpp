// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/SWidgetDetailsView.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"

#include "IDetailsView.h"
#include "IDetailKeyframeHandler.h"
#include "Animation/UMGDetailKeyframeHandler.h"
#include "Details/DetailWidgetExtensionHandler.h"
#include "EditorClassUtils.h"

#include "Customizations/UMGDetailCustomizations.h"
#include "PropertyEditorModule.h"
#include "Customizations/SlateBrushCustomization.h"
#include "Customizations/SlateFontInfoCustomization.h"
#include "Customizations/WidgetTypeCustomization.h"
#include "Customizations/WidgetChildTypeCustomization.h"
#include "Customizations/WidgetNavigationCustomization.h"
#include "Customizations/CanvasSlotCustomization.h"
#include "Customizations/HorizontalAlignmentCustomization.h"
#include "Customizations/VerticalAlignmentCustomization.h"
#include "Customizations/SlateChildSizeCustomization.h"
#include "Customizations/TextJustifyCustomization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Blueprint/WidgetTree.h"
#include "WidgetBlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "WidgetBlueprintOperationUtils.h"
#include "UMGEditorModule.h"
#include "UMGEditorProjectSettings.h"

#include "SUIComponentView.h"

#define LOCTEXT_NAMESPACE "UMG"

void SWidgetDetailsView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;

	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);

	// Create a handler for keyframing via the details panel
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(BlueprintEditor)->bEnableWidgetAnimationEditor)
	{
		TSharedRef<IDetailKeyframeHandler> KeyframeHandler = MakeShareable(new FUMGDetailKeyframeHandler(InBlueprintEditor));
		PropertyView->SetKeyframeHandler(KeyframeHandler);
	}

	// Create a handler for property binding via the details panel
	TSharedRef<FDetailWidgetExtensionHandler> BindingHandler = MakeShareable( new FDetailWidgetExtensionHandler( InBlueprintEditor ) );
	PropertyView->SetExtensionHandler(BindingHandler);

	// Handle EditDefaultsOnly and EditInstanceOnly flags
	PropertyView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SWidgetDetailsView::IsPropertyVisible));

	// Notify us of object selection changes so we can update the package re-mapping
	PropertyView->SetOnObjectArrayChanged(FOnObjectArrayChanged::CreateSP(this, &SWidgetDetailsView::OnPropertyViewObjectArrayChanged));

	PropertyView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SWidgetDetailsView::IsDetailsPanelEditingAllowed));

	TSharedRef<SWidget> IsVariableCheckbox =
		SNew(SCheckBox)
		.IsChecked(this, &SWidgetDetailsView::GetIsVariable)
		.OnCheckStateChanged(this, &SWidgetDetailsView::HandleIsVariableChanged)
		.IsEnabled(this, &SWidgetDetailsView::IsDetailsPanelEditingAllowed)
		.Padding(FMargin(3.0f, 1.0f, 18.0f, 1.0f))
		.AddMetaData<FTagMetaData>(TEXT("Details.IsVariable"))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("IsVariable", "Is Variable"))
		];

	const UWidgetEditingProjectSettings* UMGEditorProjectSettings = FWidgetBlueprintEditorUtils::GetRelevantSettings(InBlueprintEditor);
	const bool bEnableUIComponent = UMGEditorProjectSettings ? UMGEditorProjectSettings->bEnableUIComponentsProperty : true; // default to true

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SAssignNew(BorderArea, SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SWidgetDetailsView::GetCategoryAreaVisibility)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 3.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("UMGEditor.CategoryIcon")))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 6.0f, 0.0f)
					[
						SNew(SBox)
						.WidthOverride(200.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SEditableTextBox)
							.SelectAllTextWhenFocused(true)
							.ToolTipText(LOCTEXT("CategoryToolTip", "Sets the category of the widget"))
							.HintText(LOCTEXT("Category", "Category"))
							.Text(this, &SWidgetDetailsView::GetCategoryText)
							.OnTextCommitted(this, &SWidgetDetailsView::HandleCategoryTextCommitted)
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SWidgetDetailsView::GetNameAreaVisibility)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 3.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(this, &SWidgetDetailsView::GetNameIcon)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 6.0f, 0.0f)
					[
						SNew(SBox)
						.WidthOverride(200.0f)
						.VAlign(VAlign_Center)
						[
							SAssignNew(NameTextBox, SEditableTextBox)
							.SelectAllTextWhenFocused(true)
							.HintText(LOCTEXT("Name", "Name"))
							.Text(this, &SWidgetDetailsView::GetNameText)
							.IsEnabled(this, &SWidgetDetailsView::IsWidgetNameFieldEnabled)
							.OnTextChanged(this, &SWidgetDetailsView::HandleNameTextChanged)
							.OnTextCommitted(this, &SWidgetDetailsView::HandleNameTextCommitted)
							.AddMetaData<FTagMetaData>(TEXT("Details.WidgetName"))
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						FWidgetBlueprintEditorUtils::GetRelevantSettings(BlueprintEditor)->bEnableMakeVariable ? IsVariableCheckbox : SNullWidget::NullWidget
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SAssignNew(ClassLinkArea, SBox)
					]
				]
				// Only if not the user widget
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.Padding(bEnableUIComponent ? 4.0f : 0.0f, 0.0f)
				[
					bEnableUIComponent ?
					SNew(SUIComponentView, InBlueprintEditor)
					.IsEnabled(this, &SWidgetDetailsView::IsWidgetNameFieldEnabled)
					.Visibility(this, &SWidgetDetailsView::GetComponentAreaVisibility)
					:
					SNullWidget::NullWidget
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			PropertyView.ToSharedRef()
		]
	];

	BlueprintEditor.Pin()->OnSelectedWidgetsChanging.AddRaw(this, &SWidgetDetailsView::OnEditorSelectionChanging);
	BlueprintEditor.Pin()->OnSelectedWidgetsChanged.AddRaw(this, &SWidgetDetailsView::OnEditorSelectionChanged);

	RegisterCustomizations();
	
	// Refresh the selection in the details panel.
	OnEditorSelectionChanged();
}

SWidgetDetailsView::~SWidgetDetailsView()
{
	if ( BlueprintEditor.IsValid() )
	{
		BlueprintEditor.Pin()->OnSelectedWidgetsChanging.RemoveAll(this);
		BlueprintEditor.Pin()->OnSelectedWidgetsChanged.RemoveAll(this);
	}

	// Unregister the property type layouts
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("Widget"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("WidgetChild"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("WidgetNavigation"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("PanelSlot"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("EHorizontalAlignment"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("EVerticalAlignment"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("SlateChildSize"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("SlateBrush"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("SlateFontInfo"));
	PropertyView->UnregisterInstancedCustomPropertyTypeLayout(TEXT("ETextJustify"));
}

void SWidgetDetailsView::RegisterCustomizations()
{
	check(BlueprintEditor.Pin());
	TSharedRef<FWidgetBlueprintEditor> BlueprintEditorRef = BlueprintEditor.Pin().ToSharedRef();

	PropertyView->RegisterInstancedCustomPropertyLayout(UWidget::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintWidgetCustomization::MakeInstance, BlueprintEditorRef, BlueprintEditorRef->GetBlueprintObj()));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("Widget"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWidgetTypeCustomization::MakeInstance, BlueprintEditorRef), nullptr);
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("WidgetChild"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWidgetChildTypeCustomization::MakeInstance, BlueprintEditor.Pin().ToSharedRef()), nullptr);
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("WidgetNavigation"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWidgetNavigationCustomization::MakeInstance, BlueprintEditorRef));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("PanelSlot"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCanvasSlotCustomization::MakeInstance, BlueprintEditorRef->GetBlueprintObj()));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("EHorizontalAlignment"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FHorizontalAlignmentCustomization::MakeInstance));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("EVerticalAlignment"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FVerticalAlignmentCustomization::MakeInstance));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("SlateChildSize"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateChildSizeCustomization::MakeInstance));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("SlateBrush"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateBrushStructCustomization::MakeInstance, false));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("SlateFontInfo"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateFontInfoStructCustomization::MakeInstance));
	PropertyView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ETextJustify"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTextJustifyCustomization::MakeInstance));

	TWeakPtr<FWidgetBlueprintEditor> WeakBlueprintEditor = BlueprintEditor;
	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	for (const IUMGEditorModule::FCustomPropertyTypeLayout& Layout : UMGEditorModule.GetAllInstancedCustomPropertyTypeLayout())
	{
		if (Layout.Type.GetAssetName().IsValid() && Layout.Delegate.IsBound())
		{
			PropertyView->RegisterInstancedCustomPropertyTypeLayout(Layout.Type.GetAssetName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([LocalDelegate = Layout.Delegate, WeakBlueprintEditor]
				{
					return LocalDelegate.Execute(WeakBlueprintEditor);
				}));
		}
	}
}

void SWidgetDetailsView::OnEditorSelectionChanging()
{
	ClearFocusIfOwned();

	// We force the destruction of the currently monitored object when selection is about to change, to ensure all migrations occur
	// immediately.
	SelectedObjects.Empty();
	PropertyView->SetObjects(SelectedObjects);
}

void SWidgetDetailsView::OnEditorSelectionChanged()
{
	// Clear selection in the property view.
	SelectedObjects.Empty();
	PropertyView->SetObjects(SelectedObjects);

	TOptional<bool> bIsWidgetSelection;

	// Add any selected widgets to the list of pending selected objects.
	const TSet<FWidgetReference>& SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();
	for ( const FWidgetReference& WidgetRef : SelectedWidgets )
	{
		// Edit actions will go directly to the preview widget, changes will be
		// propagated to the template via SWidgetDetailsView::NotifyPostChange
		SelectedObjects.Add(WidgetRef.GetPreview());
		bIsWidgetSelection = true;
	}

	// Add any selected objects (non-widgets) to the pending selected objects.
	const TSet<TWeakObjectPtr<UObject>>& Selection = BlueprintEditor.Pin()->GetSelectedObjects();
	for ( const TWeakObjectPtr<UObject> Selected : Selection )
	{
		if ( UObject* S = Selected.Get() )
		{
			SelectedObjects.Add(S);
			bIsWidgetSelection = bIsWidgetSelection.Get(true) && Cast<UWidget>(S) != nullptr;
		}
	}

	BorderArea->SetVisibility(bIsWidgetSelection.Get(false) ? EVisibility::Visible : EVisibility::Collapsed);

	// If only 1 valid selected object exists, update the class link to point to the right class.
	if ( SelectedObjects.Num() == 1 && SelectedObjects[0].IsValid() )
	{
		FEditorClassUtils::FSourceLinkParams SourceLinkParams;
		SourceLinkParams.bUseDefaultFormat = true;

		ClassLinkArea->SetContent(FEditorClassUtils::GetSourceLink(SelectedObjects[0]->GetClass(), SourceLinkParams));
	}
	else
	{
		ClassLinkArea->SetContent(SNullWidget::NullWidget);
	}

	// Update the preview view to look at the current selection set.
	const bool bForceRefresh = false;
	PropertyView->SetObjects(SelectedObjects, bForceRefresh);
}

void SWidgetDetailsView::OnPropertyViewObjectArrayChanged(const FString& InTitle, const TArray<UObject*>& InObjects)
{
	// Update the package remapping for all selected objects so that we can find the correct localization ID when editing text properties (since we edit a copy of the real data, not connected to the asset package)
	UBlueprint* UMGBlueprint = BlueprintEditor.Pin()->GetBlueprintObj();
	if (UMGBlueprint)
	{
		UPackage* UMGPackage = UMGBlueprint->GetOutermost();
		if (UMGPackage)
		{
			TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>> ObjectToPackageMapping;

			for (const TWeakObjectPtr<UObject> Object : InObjects)
			{
				ObjectToPackageMapping.Add(Object, UMGPackage);
			}

			PropertyView->SetObjectPackageOverrides(ObjectToPackageMapping);
		}
	}
}

void SWidgetDetailsView::ClearFocusIfOwned()
{
	static bool bIsReentrant = false;
	if ( !bIsReentrant )
	{
		bIsReentrant = true;
		// When the selection is changed, we may be potentially actively editing a property,
		// if this occurs we need need to immediately clear keyboard focus
		if ( FSlateApplication::Get().HasFocusedDescendants(AsShared()) )
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
		}
		bIsReentrant = false;
	}
}

bool SWidgetDetailsView::IsDetailsPanelEditingAllowed() const 
{
	return !SelectedObjects.ContainsByPredicate([](const TWeakObjectPtr<UObject> WeakObj)
		{
			if (UObject* ObjPtr = WeakObj.Get())
			{
				if (UWidget* Widget = Cast<UWidget>(ObjPtr))
				{
					return Widget->IsLockedInDesigner();
				}
			}
			return false;
		});
}

bool SWidgetDetailsView::IsWidgetNameFieldEnabled() const
{
	return IsSingleObjectSelected() && IsDetailsPanelEditingAllowed();
}

bool SWidgetDetailsView::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	const FProperty& Property = PropertyAndParent.Property;

	const bool bIsEditDefaultsOnly = Property.HasAnyPropertyFlags(CPF_DisableEditOnInstance);
	const bool bIsEditInstanceOnly = Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate);
	if (bIsEditDefaultsOnly || bIsEditInstanceOnly)
	{
		// EditDefaultsOnly properties are only visible when the CDO/root is selected, EditInstanceOnly are only visible when the CDO/root is *not* selected
		const bool bIsCDOSelected = IsWidgetCDOSelected();
		if ((bIsEditDefaultsOnly && !bIsCDOSelected) || (bIsEditInstanceOnly && bIsCDOSelected))
		{
			return false;
		}
	}
	return true;
}

bool SWidgetDetailsView::IsWidgetCDOSelected() const
{
	if ( SelectedObjects.Num() == 1 )
	{
		UWidget* Widget = Cast<UWidget>(SelectedObjects[0].Get());
		
		// since we're passing preview widget when selecting root (owner) widget, 
		// this is also considered as selecting CDO so the category shows up correctly
		if (Widget && Widget == BlueprintEditor.Pin()->GetPreview())
		{
			return true;
		}
	}

	return false;
}

EVisibility SWidgetDetailsView::GetNameAreaVisibility() const
{
	return IsWidgetCDOSelected() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SWidgetDetailsView::GetCategoryAreaVisibility() const
{
	return IsWidgetCDOSelected() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SWidgetDetailsView::GetComponentAreaVisibility() const
{
	if (IsWidgetCDOSelected() || SelectedObjects.Num() > 1)
	{
		return EVisibility::Collapsed;
	}
	
	return EVisibility::Visible;
}

void SWidgetDetailsView::HandleCategoryTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if ( SelectedObjects.Num() == 1 && !Text.IsEmptyOrWhitespace() )
	{
		if ( UUserWidget* Widget = Cast<UUserWidget>(SelectedObjects[0].Get()) )
		{
			ensureMsgf(IsWidgetCDOSelected(), TEXT("You can ony change the category if you are the preview widget."));

			UUserWidget* WidgetCDO = Widget->GetClass()->GetDefaultObject<UUserWidget>();
			WidgetCDO->PaletteCategory = Text;

			// Set the new category on the widget blueprint as well so that it's available when the blueprint isn't loaded.
			UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
			Blueprint->PaletteCategory = Text.ToString();

			// Immediately force a rebuild so that all palettes update to show it in a new category.
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

			// MarkBlueprintAsStructurallyModified will invalidate the selection. Reselect it.
			if (TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin())
			{
				if (Editor->GetPreview())
				{
					TSet<UObject*> NewSelectedObjects;
					NewSelectedObjects.Add(Editor->GetPreview());
					Editor->SelectObjects(NewSelectedObjects);
				}
			}
		}
	}
}

FText SWidgetDetailsView::GetCategoryText() const
{
	if ( SelectedObjects.Num() == 1 )
	{
		if ( UUserWidget* Widget = Cast<UUserWidget>(SelectedObjects[0].Get()) )
		{
			UUserWidget* WidgetCDO = Widget->GetClass()->GetDefaultObject<UUserWidget>();
			FText Category = WidgetCDO->PaletteCategory;
			if ( Category.EqualToCaseIgnored(UUserWidget::StaticClass()->GetDefaultObject<UUserWidget>()->PaletteCategory) )
			{
				return FText::GetEmpty();
			}
			else
			{
				return Category;
			}
		}
	}

	return FText::GetEmpty();
}

const FSlateBrush* SWidgetDetailsView::GetNameIcon() const
{
	if (SelectedObjects.Num() == 1)
	{
		const UWidget* Widget = Cast<UWidget>(SelectedObjects[0].Get());
		if (Widget)
		{
			const UClass* WidgetClass = Widget->GetClass();
			if (WidgetClass)
			{
				return FSlateIconFinder::FindIconBrushForClass(WidgetClass);
			}
		}
	}

	return nullptr;
}

FText SWidgetDetailsView::GetNameText() const
{
	if ( SelectedObjects.Num() == 1 )
	{
		UWidget* Widget = Cast<UWidget>(SelectedObjects[0].Get());
		if ( Widget )
		{
			return Widget->IsGeneratedName() ? FText::FromName(Widget->GetFName()) : Widget->GetLabelText();
		}
	}

	if (SelectedObjects.Num() > 1)
	{
		return LOCTEXT("MultipleWidgetsSelected", "Multiple Values");
	}
	
	return FText::GetEmpty();
}

bool SWidgetDetailsView::IsSingleObjectSelected() const
{
	return SelectedObjects.Num() == 1;
}

void SWidgetDetailsView::HandleNameTextChanged(const FText& Text)
{
	FText OutErrorMessage;
	if ( HandleVerifyNameTextChanged(Text, OutErrorMessage) )
	{
		OutErrorMessage = FText::GetEmpty();
	}

	// We delay the error message to be sure we are outside of attribute processing.
	// Otherwise, we might add or remove a widget (the error reporting one) which is not allowed during Prepass
	TWeakPtr<SEditableTextBox> WeakNameTextBox = NameTextBox;
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(
		[OutErrorMessage = MoveTemp(OutErrorMessage), WeakNameTextBox = MoveTemp(WeakNameTextBox)](double InCurrentTime, float InDeltaTime)
		{
			if (TSharedPtr<SEditableTextBox> NameTextBox = WeakNameTextBox.Pin())
			{
				NameTextBox->SetError(OutErrorMessage);
			}
			return EActiveTimerReturnType::Stop;
		})
	);
}

bool SWidgetDetailsView::HandleVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	if ( SelectedObjects.Num() == 1 )
	{
		UWidget* PreviewWidget = Cast<UWidget>(SelectedObjects[0].Get());
		
		TSharedRef<class FWidgetBlueprintEditor> BlueprintEditorRef = BlueprintEditor.Pin().ToSharedRef();

		FWidgetReference WidgetRef = BlueprintEditorRef->GetReferenceFromPreview(PreviewWidget);

		return FWidgetBlueprintEditorUtils::VerifyWidgetRename(BlueprintEditorRef, WidgetRef, InText, OutErrorMessage);
	}
	else
	{
		return false;
	}
}

void SWidgetDetailsView::HandleNameTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	static bool IsReentrant = false;

	if ( !IsReentrant )
	{
		IsReentrant = true;
		if ( SelectedObjects.Num() == 1 )
		{
			FText DummyText;
			if ( HandleVerifyNameTextChanged(Text, DummyText) )
			{
				UWidget* Widget = Cast<UWidget>(SelectedObjects[0].Get());
				if (Widget && !Widget->GetLabelText().EqualTo(Text))
				{
					FWidgetBlueprintEditorUtils::RenameWidget(BlueprintEditor.Pin().ToSharedRef(), Widget->GetFName(), Text.ToString());
				}
			}
		}
		IsReentrant = false;

		if (CommitType == ETextCommit::OnUserMovedFocus || CommitType == ETextCommit::OnCleared)
		{
			NameTextBox->SetError(FText::GetEmpty());
		}
	}
}

ECheckBoxState SWidgetDetailsView::GetIsVariable() const
{
	TOptional<ECheckBoxState> Result;
	for (const TWeakObjectPtr<UObject>& Obj : SelectedObjects)
	{
		if (UWidget* Widget = Cast<UWidget>(Obj.Get()))
		{
			ECheckBoxState WidgetState = Widget->bIsVariable ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			if (Result.IsSet() && WidgetState != Result.GetValue())
			{
				return ECheckBoxState::Undetermined;
			}
			Result = WidgetState;
		}
	}
	return Result.IsSet() ? Result.GetValue() : ECheckBoxState::Unchecked;
}

void SWidgetDetailsView::HandleIsVariableChanged(ECheckBoxState CheckState)
{
	if (SelectedObjects.Num() > 0)
	{
		UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
		check(Blueprint);
		TSharedPtr<FWidgetBlueprintEditor> BPEditor = BlueprintEditor.Pin();
		check(BPEditor);

		TArray<UWidget*, TInlineAllocator<16>> WidgetToModify;
		for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
		{
			if (UWidget* Widget = Cast<UWidget>(SelectedObject.Get()))
			{
				check(Blueprint->WidgetTree);
				FWidgetReference WidgetRef = BPEditor->GetReferenceFromTemplate(Blueprint->WidgetTree->FindWidget(Widget->GetFName()));
				if (WidgetRef.IsValid())
				{
					WidgetToModify.Add(WidgetRef.GetTemplate());
					WidgetToModify.Add(WidgetRef.GetPreview());
				}
			}
		}

		if (WidgetToModify.Num() > 0)
		{
			const FScopedTransaction Transaction(LOCTEXT("VariableToggle", "Variable Toggle"));
			for (UWidget* Widget : WidgetToModify)
			{
				FWidgetBlueprintOperationUtils::ToggleWidgetAsVariable(Blueprint, Widget, CheckState == ECheckBoxState::Checked, false);
			}

			// Refresh references and flush editors
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
}



void SWidgetDetailsView::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange)
{
	// During auto-key do not migrate values
	if( BlueprintEditor.Pin()->GetSequencer()->GetAutoChangeMode() == EAutoChangeMode::None )
	{
		TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();

		const bool bIsModify = true;
		Editor->MigrateFromChain(nullptr, PropertyAboutToChange, bIsModify);
	}
}

void SWidgetDetailsView::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	const static FName DesignerRebuildName("DesignerRebuild");

	if ( PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive && BlueprintEditor.Pin()->GetSequencer()->GetAutoChangeMode() == EAutoChangeMode::None )
	{
		TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();

		const bool bIsModify = false;
		Editor->MigrateFromChain(&PropertyChangedEvent, PropertyThatChanged, bIsModify);

		// Any time we migrate a property value we need to mark the blueprint as structurally modified so users don't need 
		// to recompile it manually before they see it play in game using the latest version.
		FBlueprintEditorUtils::MarkBlueprintAsModified(BlueprintEditor.Pin()->GetBlueprintObj());
		
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
		{
			ClearFocusIfOwned();
		}
	}

	// If the property that changed is marked as "DesignerRebuild" we invalidate
	// the preview.
	if ( PropertyChangedEvent.Property->HasMetaData(DesignerRebuildName) || PropertyThatChanged->GetActiveMemberNode()->GetValue()->HasMetaData(DesignerRebuildName) )
	{
		const bool bViewOnly = true;
		BlueprintEditor.Pin()->InvalidatePreview(bViewOnly);
	}
}

#undef LOCTEXT_NAMESPACE