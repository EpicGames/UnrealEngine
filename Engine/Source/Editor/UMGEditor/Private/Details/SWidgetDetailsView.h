// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"
#include "WidgetBlueprintEditor.h"

class IDetailsView;
class SBox;
class SEditableTextBox;

/**
 * The details view used in the designer section of the widget blueprint editor.
 */
class SWidgetDetailsView : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS( SWidgetDetailsView ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~SWidgetDetailsView();
	
	/** Gets the property view for this details panel */
	TSharedPtr<class IDetailsView> GetPropertyView() const { return PropertyView; }

	// FNotifyHook interface
	virtual void NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
	// End of FNotifyHook interface
	
private:
	/** Registers the designer specific customizations */
	void RegisterCustomizations();

	/** Handles the widget selection changing event in the editor, updates the details panel accordingly. */
	void OnEditorSelectionChanging();

	/** Handles the widget selection changed event in the editor, updates the details panel accordingly. */
	void OnEditorSelectionChanged();

	/** Handles the callback from the property detail view confirming the list of objects being edited has changed */
	void OnPropertyViewObjectArrayChanged(const FString& InTitle, const TArray<UObject*>& InObjects);

	bool IsDetailsPanelEditingAllowed() const;

	bool IsWidgetNameFieldEnabled() const;

	void ClearFocusIfOwned();

	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;
	
	bool IsWidgetCDOSelected() const;

	EVisibility GetNameAreaVisibility() const;
	
	EVisibility GetComponentAreaVisibility() const;

	const FSlateBrush* GetNameIcon() const;

	FText GetNameText() const;

	bool IsSingleObjectSelected() const;

	bool HandleVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage);

	void HandleNameTextChanged(const FText& Text);
	void HandleNameTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	ECheckBoxState GetIsVariable() const;

	void HandleIsVariableChanged(ECheckBoxState CheckState);

	EVisibility GetCategoryAreaVisibility() const;
	FText GetCategoryText() const;
	void HandleCategoryTextCommitted(const FText& Text, ETextCommit::Type CommitType);

private:
	/** The editor that owns this details view */
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;

	/** The container widget for the class link users can click to open another asset */
	TSharedPtr<SWidget> BorderArea;

	/** The name text box that users can use to rename their widgets */
	TSharedPtr<SEditableTextBox> NameTextBox;

	/** The container widget for the class link users can click to open another asset */
	TSharedPtr<SBox> ClassLinkArea;
	
	/** Property viewing widget */
	TSharedPtr<class IDetailsView> PropertyView;

	/** Selected objects for this detail view */
	TArray< TWeakObjectPtr<UObject> > SelectedObjects;
};
