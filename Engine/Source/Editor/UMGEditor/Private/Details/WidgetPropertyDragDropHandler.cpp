// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/WidgetPropertyDragDropHandler.h"

#include "Binding/WidgetBinding.h"
#include "Customizations/UMGDetailCustomizations.h"
#include "Details/WidgetPropertyDragDropOp.h"
#include "Engine/Blueprint.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

FWidgetPropertyDragDropHandler::FWidgetPropertyDragDropHandler(UWidget* InWidget, TSharedPtr<IPropertyHandle> InPropertyHandle, UWidgetBlueprint* InWidgetBP)
{
	OwnerWidget = InWidget;
	PropertyHandle = InPropertyHandle;
	WidgetBP = InWidgetBP;
}

TSharedPtr<FDragDropOperation> FWidgetPropertyDragDropHandler::CreateDragDropOperation() const
{
	if (!PropertyHandle.IsValid())
	{
		return MakeShared<FDragDropOperation>();
	}

	FProperty* WidgetProperty = PropertyHandle->GetProperty();
	if (WidgetProperty == nullptr)
	{
		return MakeShared<FDragDropOperation>();
	}
	if (UWidget* WidgetPtr = OwnerWidget.Get())
	{
		if (UWidgetBlueprint* WidgetBPPtr = WidgetBP.Get())
		{
			FCachedPropertyPath CachedPropertyPath(PropertyHandle->GeneratePathToProperty());
			CachedPropertyPath.Resolve(WidgetPtr);
			TArray<FFieldVariant> FieldPath;

			for (int32 SegNum = 0; SegNum < CachedPropertyPath.GetNumSegments(); SegNum++)
			{
				FieldPath.Add(CachedPropertyPath.GetSegment(SegNum).GetField());
			}
			TSharedPtr<FWidgetPropertyDragDropOp> DragOp = FWidgetPropertyDragDropOp::New(WidgetPtr, WidgetProperty->GetFName(), FieldPath, WidgetBPPtr);
			return DragOp;
		}
	}
	return MakeShared<FDragDropOperation>();
}

int32 FWidgetPropertyDragDropHandler::ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone)
{
	return 0;
}

bool FWidgetPropertyDragDropHandler::AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	return false;
}

TOptional<EItemDropZone> FWidgetPropertyDragDropHandler::CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	return TOptional<EItemDropZone>();
}

