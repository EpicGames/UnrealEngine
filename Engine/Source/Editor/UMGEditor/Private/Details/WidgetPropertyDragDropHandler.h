// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "IDetailDragDropHandler.h"
#include "Input/DragAndDrop.h"

class IPropertyHandle;
class UWidget;
class UWidgetBlueprint;

/** Handler for customizing the drag-and-drop behavior for function entry/result pins, allowing parameters to be reordered */
class FWidgetPropertyDragDropHandler : public IDetailDragDropHandler
{
public:
	FWidgetPropertyDragDropHandler(UWidget* InWidget, TSharedPtr<IPropertyHandle> InPropertyHandle, UWidgetBlueprint* InWidgetBP);

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;

	static int32 ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone);

	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TWeakObjectPtr<UWidget> OwnerWidget;
	TWeakObjectPtr<UWidgetBlueprint> WidgetBP;
};

