// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/WidgetPropertyDragDropOp.h"
#include "Components/Widget.h"
#include "CoreMinimal.h"
#include "ScopedTransaction.h"
#include "WidgetBlueprint.h"

#define LOCTEXT_NAMESPACE "WidgetPropertyDragDropOp"

FWidgetPropertyDragDropOp::FWidgetPropertyDragDropOp()
	: OwnerWidget(nullptr)
	, WidgetBP(nullptr)
	, Transaction(nullptr)
{
}

void FWidgetPropertyDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		check(Transaction);
		Transaction->Cancel();
	}
}

/** Constructs a new drag/drop operation */
TSharedRef<FWidgetPropertyDragDropOp> FWidgetPropertyDragDropOp::New(UWidget* InOwnerWidget, FName InPropertyName, TArray<FFieldVariant> InPropertyPath, UWidgetBlueprint* InWidgetBP)
{
	TSharedRef<FWidgetPropertyDragDropOp> Operation = MakeShared<FWidgetPropertyDragDropOp>();

	// Set the display text and the transaction name based on whether we're dragging a single or multiple widgets
	Operation->CurrentHoverText = Operation->DefaultHoverText = FText::FromName(InPropertyName);
	Operation->Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("Designer_DragWidgetProperty", "Drag Widget Property"));
	Operation->OwnerWidget = InOwnerWidget;
	Operation->DraggedPropertyPath = InPropertyPath;
	Operation->WidgetBP = InWidgetBP;

	Operation->Construct();
	return Operation;
}
#undef LOCTEXT_NAMESPACE