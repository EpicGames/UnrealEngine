// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDrop/SelectedWidgetDragDropOp.h"
#include "WidgetTemplate.h"
#include "WidgetBlueprintEditor.h"
#include "Components/PanelWidget.h"
#include "WidgetBlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

FSelectedWidgetDragDropOp::~FSelectedWidgetDragDropOp()
{
	if (bShowingMessage)
	{
		Designer->PopDesignerMessage();
	}
}

void FSelectedWidgetDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	OnDragDropEnded.Broadcast();
}

TSharedRef<FSelectedWidgetDragDropOp> FSelectedWidgetDragDropOp::New(TSharedPtr<FWidgetBlueprintEditor> Editor, IUMGDesigner* InDesigner, const TArray<FDraggingWidgetReference>& InWidgets)
{
	TSharedRef<FSelectedWidgetDragDropOp> Operation = MakeShareable(new FSelectedWidgetDragDropOp());
	Operation->bShowingMessage = false;
	Operation->Designer = InDesigner;

	for (const FDraggingWidgetReference& InDraggedWidget : InWidgets)
	{
		FItem DraggedWidget;
		DraggedWidget.bStayingInParent = false;

		if (UPanelWidget* PanelTemplate = InDraggedWidget.Widget.GetTemplate()->GetParent())
		{
			DraggedWidget.ParentWidget = Editor->GetReferenceFromTemplate(PanelTemplate);
			DraggedWidget.bStayingInParent = PanelTemplate->LockToPanelOnDrag() || GetDefault<UWidgetDesignerSettings>()->bLockToPanelOnDragByDefault;

			if (DraggedWidget.bStayingInParent)
			{
				Operation->bShowingMessage = true;
			}
		}

		// Cache the preview and template, it's not safe to query the preview/template while dragging the widget as it no longer
		// exists in the tree.
		DraggedWidget.Preview = InDraggedWidget.Widget.GetPreview();
		DraggedWidget.Template = InDraggedWidget.Widget.GetTemplate();

		DraggedWidget.DraggedOffset = InDraggedWidget.DraggedOffset;

		FWidgetBlueprintEditorUtils::ExportPropertiesToText(InDraggedWidget.Widget.GetTemplate()->Slot, DraggedWidget.ExportedSlotProperties);

		Operation->DraggedWidgets.Add(DraggedWidget);
	}

	// Set the display text based on whether we're dragging a single or multiple widgets
	if (InWidgets.Num() == 1)
	{
		FText DisplayText = InWidgets[0].Widget.GetTemplate()->GetLabelText();

		Operation->DefaultHoverText = DisplayText;
		Operation->CurrentHoverText = DisplayText;
	}
	else
	{
		Operation->CurrentHoverText = Operation->DefaultHoverText = LOCTEXT("DragMultipleWidgets", "Multiple Widgets");
	}

	if (Operation->bShowingMessage)
	{
		InDesigner->PushDesignerMessage(LOCTEXT("PressAltToMoveFromParent", "Press [Alt] to move the widget out of the current parent"));
	}

	Operation->Construct();
	return Operation;
}

#undef LOCTEXT_NAMESPACE
