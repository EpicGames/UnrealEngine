// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IUMGDesigner.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "WidgetReference.h"

class FWidgetTemplate;
class FWidgetBlueprintEditor;
class UWidget;

/**
 * This drag drop operation allows widget Selected in the viewport to be dragged and dropped into the designer.
 */
class FSelectedWidgetDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSelectedWidgetDragDropOp, FDecoratedDragDropOp);

	DECLARE_MULTICAST_DELEGATE(FOnDragDropEnded);
	FOnDragDropEnded OnDragDropEnded;

	virtual ~FSelectedWidgetDragDropOp();

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;

	struct FDraggingWidgetReference
	{
		FWidgetReference Widget;

		FVector2D DraggedOffset;
	};

	struct FItem
	{
		/** The slot properties for the old slot the widget was in, is used to attempt to reapply the same layout information */
		TMap<FName, FString> ExportedSlotProperties;

		/** The widget being dragged */
		UWidget* Template;

		/** The preview widget being dragged */
		UWidget* Preview;

		/** Can the drag drop change the widget's parent? */
		bool bStayingInParent;

		/** The original parent of the widget. */
		FWidgetReference ParentWidget;

		/** The offset of the original click location, as a percentage of the widget's size. */
		FVector2D DraggedOffset;
	};

	TArray<FItem> DraggedWidgets;

	bool bShowingMessage;

	IUMGDesigner* Designer;

	static TSharedRef<FSelectedWidgetDragDropOp> New(TSharedPtr<FWidgetBlueprintEditor> Editor, IUMGDesigner* InDesigner, const TArray<FDraggingWidgetReference>& InWidgets);
};
