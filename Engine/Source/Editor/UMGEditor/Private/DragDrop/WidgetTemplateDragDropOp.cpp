// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDrop/WidgetTemplateDragDropOp.h"

#include "Internationalization/Text.h"
#include "WidgetTemplate.h"

#define LOCTEXT_NAMESPACE "UMG"

TSharedRef<FWidgetTemplateDragDropOp> FWidgetTemplateDragDropOp::New(const TSharedPtr<FWidgetTemplate>& InTemplate)
{
	TSharedRef<FWidgetTemplateDragDropOp> Operation = MakeShareable(new FWidgetTemplateDragDropOp());
	Operation->Template = InTemplate;
	Operation->DefaultHoverText = InTemplate->Name;
	Operation->CurrentHoverText = InTemplate->Name;
	Operation->Construct();

	return Operation;
}

#undef LOCTEXT_NAMESPACE
