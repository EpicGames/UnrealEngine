// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/CanvasSlotExtension.h"
#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Rendering/DrawElements.h"
#include "Components/CanvasPanelSlot.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "Components/CanvasPanel.h"
#include "IUMGDesigner.h"
#include "ObjectEditorUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// SEventShim

class SEventShim : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEventShim)
		: _Content()
		, _OnMouseEnter()
		, _OnMouseLeave()
	{}
		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_EVENT(FSimpleDelegate, OnMouseEnter)
		SLATE_EVENT(FSimpleDelegate, OnMouseLeave)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		MouseEnter = InArgs._OnMouseEnter;
		MouseLeave = InArgs._OnMouseLeave;

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

		MouseEnter.ExecuteIfBound();
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);

		MouseLeave.ExecuteIfBound();
	}

private:
	FSimpleDelegate MouseEnter;
	FSimpleDelegate MouseLeave;
};

/////////////////////////////////////////////////////
// FCanvasSlotExtension

const double SnapDistance = 7.0;

static double DistancePointToLine2D(const FVector2D& LinePointA, const FVector2D& LinePointB, const FVector2D& PointC)
{
	FVector2D AB = LinePointB - LinePointA;
	FVector2D AC = PointC - LinePointA;

	double Distance = FVector2D::CrossProduct(AB, AC) / FVector2D::Distance(LinePointA, LinePointB);
	return FMath::Abs(Distance);
}

FCanvasSlotExtension::FCanvasSlotExtension()
	: bMovingAnchor(false)
	, bHoveringAnchor(false)
{
	ExtensionId = FName(TEXT("CanvasSlot"));
}

bool FCanvasSlotExtension::CanExtendSelection(const TArray< FWidgetReference >& Selection) const
{
	for ( const FWidgetReference& Widget : Selection )
	{
		if ( !Widget.GetTemplate()->Slot || !Widget.GetTemplate()->Slot->IsA(UCanvasPanelSlot::StaticClass()) )
		{
			return false;
		}
	}

	return Selection.Num() == 1;
}

void FCanvasSlotExtension::ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
{
	SelectionCache = Selection;

	AnchorWidgets.SetNumZeroed((uint8)EAnchorWidget::Count);
	AnchorWidgets[(uint8)EAnchorWidget::Center] = MakeAnchorWidget(EAnchorWidget::Center, 16, 16);

	AnchorWidgets[(uint8)EAnchorWidget::Left] = MakeAnchorWidget(EAnchorWidget::Left, 32, 16);
	AnchorWidgets[(uint8)EAnchorWidget::Right] = MakeAnchorWidget(EAnchorWidget::Right, 32, 16);
	AnchorWidgets[(uint8)EAnchorWidget::Top] = MakeAnchorWidget(EAnchorWidget::Top, 16, 32);
	AnchorWidgets[(uint8)EAnchorWidget::Bottom] = MakeAnchorWidget(EAnchorWidget::Bottom, 16, 32);

	AnchorWidgets[(uint8)EAnchorWidget::TopLeft] = MakeAnchorWidget(EAnchorWidget::TopLeft, 24, 24);
	AnchorWidgets[(uint8)EAnchorWidget::TopRight] = MakeAnchorWidget(EAnchorWidget::TopRight, 24, 24);
	AnchorWidgets[(uint8)EAnchorWidget::BottomLeft] = MakeAnchorWidget(EAnchorWidget::BottomLeft, 24, 24);
	AnchorWidgets[(uint8)EAnchorWidget::BottomRight] = MakeAnchorWidget(EAnchorWidget::BottomRight, 24, 24);


	TArray<FVector2D> AnchorPos;
	AnchorPos.SetNumZeroed((uint8)EAnchorWidget::Count);
	AnchorPos[(uint8)EAnchorWidget::Center] = FVector2D(-8, -8);
	
	AnchorPos[(uint8)EAnchorWidget::Left] = FVector2D(-32, -8);
	AnchorPos[(uint8)EAnchorWidget::Right] = FVector2D(0, -8);
	AnchorPos[(uint8)EAnchorWidget::Top] = FVector2D(-8, -32);
	AnchorPos[(uint8)EAnchorWidget::Bottom] = FVector2D(-8, 0);

	AnchorPos[(uint8)EAnchorWidget::TopLeft] = FVector2D(-24, -24);
	AnchorPos[(uint8)EAnchorWidget::TopRight] = FVector2D(0, -24);
	AnchorPos[(uint8)EAnchorWidget::BottomLeft] = FVector2D(-24, 0);
	AnchorPos[(uint8)EAnchorWidget::BottomRight] = FVector2D(0, 0);

	for ( int32 AnchorIndex = (int32)EAnchorWidget::Count - 1; AnchorIndex >= 0; AnchorIndex-- )
	{
		if ( !AnchorWidgets[AnchorIndex].IsValid() )
		{
			continue;
		}

		AnchorWidgets[AnchorIndex]->SlatePrepass();
		TAttribute<FVector2D> AnchorAlignment = TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(SharedThis(this), &FCanvasSlotExtension::GetAnchorAlignment, (EAnchorWidget)AnchorIndex));
		SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(AnchorWidgets[AnchorIndex].ToSharedRef(), EExtensionLayoutLocation::RelativeFromParent, AnchorPos[AnchorIndex], AnchorAlignment)));
	}
}

TSharedRef<SWidget> FCanvasSlotExtension::MakeAnchorWidget(EAnchorWidget AnchorType, float Width, float Height)
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBrush"))
		.OnMouseButtonDown(this, &FCanvasSlotExtension::HandleAnchorBeginDrag, AnchorType)
		.OnMouseButtonUp(this, &FCanvasSlotExtension::HandleAnchorEndDrag, AnchorType)
		.OnMouseMove(this, &FCanvasSlotExtension::HandleAnchorDragging, AnchorType)
		.Visibility(this, &FCanvasSlotExtension::GetAnchorVisibility, AnchorType)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AnchorMedallion")))
		.Padding(FMargin(0))
		[
			SNew(SEventShim)
			.OnMouseEnter(this, &FCanvasSlotExtension::OnMouseEnterAnchor)
			.OnMouseLeave(this, &FCanvasSlotExtension::OnMouseLeaveAnchor)
			[
				SNew(SBox)
				.WidthOverride(Width)
				.HeightOverride(Height)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SImage)
					.Image(this, &FCanvasSlotExtension::GetAnchorBrush, AnchorType)
				]
			]
		];
}

void FCanvasSlotExtension::OnMouseEnterAnchor()
{
	bHoveringAnchor = true;
}

void FCanvasSlotExtension::OnMouseLeaveAnchor()
{
	bHoveringAnchor = false;
}

const FSlateBrush* FCanvasSlotExtension::GetAnchorBrush(EAnchorWidget AnchorType) const
{
	switch ( AnchorType )
	{
	case EAnchorWidget::Center:
		return AnchorWidgets[(uint8)EAnchorWidget::Center]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Center.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Center");
	case EAnchorWidget::Left:
		return AnchorWidgets[(uint8)EAnchorWidget::Left]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Left.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Left");
	case EAnchorWidget::Right:
		return AnchorWidgets[(uint8)EAnchorWidget::Right]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Right.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Right");
	case EAnchorWidget::Top:
		return AnchorWidgets[(uint8)EAnchorWidget::Top]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Top.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Top");
	case EAnchorWidget::Bottom:
		return AnchorWidgets[(uint8)EAnchorWidget::Bottom]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Bottom.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.Bottom");
	case EAnchorWidget::TopLeft:
		return AnchorWidgets[(uint8)EAnchorWidget::TopLeft]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.TopLeft.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.TopLeft");
	case EAnchorWidget::BottomRight:
		return AnchorWidgets[(uint8)EAnchorWidget::BottomRight]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.BottomRight.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.BottomRight");
	case EAnchorWidget::TopRight:
		return AnchorWidgets[(uint8)EAnchorWidget::TopRight]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.TopRight.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.TopRight");
	case EAnchorWidget::BottomLeft:
		return AnchorWidgets[(uint8)EAnchorWidget::BottomLeft]->IsHovered() ? FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.BottomLeft.Hovered") : FAppStyle::Get().GetBrush("UMGEditor.AnchorGizmo.BottomLeft");
	}

	return FCoreStyle::Get().GetBrush("Selection");
}

EVisibility FCanvasSlotExtension::GetAnchorVisibility(EAnchorWidget AnchorType) const
{
	for ( const FWidgetReference& Selection : SelectionCache )
	{
		UWidget* PreviewWidget = Selection.GetPreview();
		if ( PreviewWidget && PreviewWidget->Slot && !PreviewWidget->bHiddenInDesigner )
		{
			if ( UCanvasPanelSlot* PreviewCanvasSlot = Cast<UCanvasPanelSlot>(PreviewWidget->Slot) )
			{
				FAnchorData PreviewLayout = PreviewCanvasSlot->GetLayout();
				switch ( AnchorType )
				{
				case EAnchorWidget::Center:
					return PreviewLayout.Anchors.Minimum == PreviewLayout.Anchors.Maximum ? EVisibility::Visible : EVisibility::Collapsed;
				case EAnchorWidget::Left:
				case EAnchorWidget::Right:
					return PreviewLayout.Anchors.Minimum.Y == PreviewLayout.Anchors.Maximum.Y ? EVisibility::Visible : EVisibility::Collapsed;
				case EAnchorWidget::Top:
				case EAnchorWidget::Bottom:
					return PreviewLayout.Anchors.Minimum.X == PreviewLayout.Anchors.Maximum.X ? EVisibility::Visible : EVisibility::Collapsed;
				}

				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

FVector2D FCanvasSlotExtension::GetAnchorAlignment(EAnchorWidget AnchorType) const
{
	for ( const FWidgetReference& Selection : SelectionCache )
	{
		UWidget* PreviewWidget = Selection.GetPreview();
		if ( PreviewWidget && PreviewWidget->Slot )
		{
			if ( UCanvasPanelSlot* PreviewCanvasSlot = Cast<UCanvasPanelSlot>(PreviewWidget->Slot) )
			{
				FVector2D Minimum = PreviewCanvasSlot->GetLayout().Anchors.Minimum;
				FVector2D Maximum = PreviewCanvasSlot->GetLayout().Anchors.Maximum;

				switch ( AnchorType )
				{
				case EAnchorWidget::Center:
				case EAnchorWidget::Left:
				case EAnchorWidget::Top:
				case EAnchorWidget::TopLeft:
					return Minimum;
				case EAnchorWidget::Right:
				case EAnchorWidget::Bottom:
				case EAnchorWidget::BottomRight:
					return Maximum;
				case EAnchorWidget::TopRight:
					return  FVector2D(Maximum.X, Minimum.Y);
				case EAnchorWidget::BottomLeft:
					return  FVector2D(Minimum.X, Maximum.Y);
				}
			}
		}
	}

	return FVector2D(0, 0);
}

bool FCanvasSlotExtension::GetCollisionSegmentsForSlot(UCanvasPanel* Canvas, int32 SlotIndex, TArray<FVector2D>& Segments)
{
	FGeometry ArrangedGeometry;
	if ( Canvas->GetGeometryForSlot(SlotIndex, ArrangedGeometry) )
	{
		GetCollisionSegmentsFromGeometry(ArrangedGeometry, Segments);
		return true;
	}
	return false;
}

bool FCanvasSlotExtension::GetCollisionSegmentsForSlot(UCanvasPanel* Canvas, UCanvasPanelSlot* Slot, TArray<FVector2D>& Segments)
{
	FGeometry ArrangedGeometry;
	if ( Canvas->GetGeometryForSlot(Slot, ArrangedGeometry) )
	{
		GetCollisionSegmentsFromGeometry(ArrangedGeometry, Segments);
		return true;
	}
	return false;
}

void FCanvasSlotExtension::GetCollisionSegmentsFromGeometry(FGeometry ArrangedGeometry, TArray<FVector2D>& Segments)
{
	Segments.SetNumUninitialized(8);

	// Left Side
	Segments[0] = FVector2D(ArrangedGeometry.Position);
	Segments[1] = FVector2D(ArrangedGeometry.Position) + FVector2D(0, ArrangedGeometry.GetLocalSize().Y);

	// Top Side
	Segments[2] = FVector2D(ArrangedGeometry.Position);
	Segments[3] = FVector2D(ArrangedGeometry.Position) + FVector2D(ArrangedGeometry.GetLocalSize().X, 0);

	// Right Side
	Segments[4] = FVector2D(ArrangedGeometry.Position) + FVector2D(ArrangedGeometry.GetLocalSize().X, 0);
	Segments[5] = FVector2D(ArrangedGeometry.Position) + ArrangedGeometry.GetLocalSize();

	// Bottom Side
	Segments[6] = FVector2D(ArrangedGeometry.Position) + FVector2D(0, ArrangedGeometry.GetLocalSize().Y);
	Segments[7] = FVector2D(ArrangedGeometry.Position) + ArrangedGeometry.GetLocalSize();
}

void FCanvasSlotExtension::Paint(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	//PaintCollisionLines(Selection, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	PaintDragPercentages(Selection, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
}

FReply FCanvasSlotExtension::HandleAnchorBeginDrag(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType)
{
	if (Event.GetEffectingButton() == EKeys::LeftMouseButton && AnchorWidgets[(int32)AnchorType].IsValid())
	{ 
		BeginTransaction(LOCTEXT("MoveAnchor", "Move Anchor"));

		bMovingAnchor = true;
		MouseDownPosition = Event.GetScreenSpacePosition();

		UCanvasPanelSlot* PreviewCanvasSlot = Cast<UCanvasPanelSlot>(SelectionCache[0].GetPreview()->Slot);
		BeginAnchors = PreviewCanvasSlot->GetLayout().Anchors;

		Designer->PushDesignerMessage(LOCTEXT("CenterAnchorControls", "Hold [Ctrl] to update widget position"));

		return FReply::Handled().CaptureMouse(AnchorWidgets[(int32)AnchorType].ToSharedRef());
	}
	
	return FReply::Unhandled();
}

FReply FCanvasSlotExtension::HandleAnchorEndDrag(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType)
{
	if (AnchorWidgets[(int32)AnchorType].IsValid() && AnchorWidgets[(int32)AnchorType]->HasMouseCapture() && Event.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		EndTransaction();

		bMovingAnchor = false;

		Designer->PopDesignerMessage();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void FCanvasSlotExtension::ProximitySnapValue(float SnapFrequency, float SnapProximity, FVector2D::FReal& Value)
{
	double MajorAnchorDiv = Value / SnapFrequency;
	double SubAnchorLinePos = MajorAnchorDiv - FMath::RoundToDouble(MajorAnchorDiv);

	if ( FMath::Abs(SubAnchorLinePos) <= SnapProximity )
	{
		Value = FMath::RoundToDouble(MajorAnchorDiv) * SnapFrequency;
	}
}

FReply FCanvasSlotExtension::HandleAnchorDragging(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType)
{
	if ( bMovingAnchor && !Event.GetCursorDelta().IsZero() )
	{
		float InverseSize = 1.0f / Designer->GetPreviewScale();

		if (SelectionCache.Num() >0)
		{
			FWidgetReference& Selection = SelectionCache[0];
			UWidget* PreviewWidget = Selection.GetPreview();
			if ( UCanvasPanel* Canvas = Cast<UCanvasPanel>(PreviewWidget->GetParent()) )
			{
				UCanvasPanelSlot* PreviewCanvasSlot = Cast<UCanvasPanelSlot>(PreviewWidget->Slot);

				FGeometry GeometryForSlot;
				if ( Canvas->GetGeometryForSlot(PreviewCanvasSlot, GeometryForSlot) )
				{
					FGeometry CanvasGeometry = Canvas->GetCanvasWidget()->GetCachedGeometry();
					FVector2D StartLocalPosition = CanvasGeometry.AbsoluteToLocal(MouseDownPosition);
					FVector2D NewLocalPosition = CanvasGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());
					FVector2D LocalPositionDelta = NewLocalPosition - StartLocalPosition;

					FVector2D AnchorDelta = LocalPositionDelta / CanvasGeometry.GetLocalSize();

					const FAnchorData OldLayoutData = PreviewCanvasSlot->GetLayout();
					FAnchorData LayoutData = OldLayoutData;
					
					switch ( AnchorType )
					{
					case EAnchorWidget::Center:
						LayoutData.Anchors.Maximum = BeginAnchors.Maximum + AnchorDelta;
						LayoutData.Anchors.Minimum = BeginAnchors.Minimum + AnchorDelta;
						LayoutData.Anchors.Minimum.X = FMath::Clamp(LayoutData.Anchors.Minimum.X, 0.0f, 1.0f);
						LayoutData.Anchors.Maximum.X = FMath::Clamp(LayoutData.Anchors.Maximum.X, 0.0f, 1.0f);
						LayoutData.Anchors.Minimum.Y = FMath::Clamp(LayoutData.Anchors.Minimum.Y, 0.0f, 1.0f);
						LayoutData.Anchors.Maximum.Y = FMath::Clamp(LayoutData.Anchors.Maximum.Y, 0.0f, 1.0f);
						break;
					}

					switch ( AnchorType )
					{
					case EAnchorWidget::Left:
					case EAnchorWidget::TopLeft:
					case EAnchorWidget::BottomLeft:
						LayoutData.Anchors.Minimum.X = BeginAnchors.Minimum.X + AnchorDelta.X;
						LayoutData.Anchors.Minimum.X = FMath::Clamp(LayoutData.Anchors.Minimum.X, 0.0f, LayoutData.Anchors.Maximum.X);
						break;
					}

					switch ( AnchorType )
					{
					case EAnchorWidget::Right:
					case EAnchorWidget::TopRight:
					case EAnchorWidget::BottomRight:
						LayoutData.Anchors.Maximum.X = BeginAnchors.Maximum.X + AnchorDelta.X;
						LayoutData.Anchors.Maximum.X = FMath::Clamp(LayoutData.Anchors.Maximum.X, LayoutData.Anchors.Minimum.X, 1.0f);
						break;
					}

					switch ( AnchorType )
					{
					case EAnchorWidget::Top:
					case EAnchorWidget::TopLeft:
					case EAnchorWidget::TopRight:
						LayoutData.Anchors.Minimum.Y = BeginAnchors.Minimum.Y + AnchorDelta.Y;
						LayoutData.Anchors.Minimum.Y = FMath::Clamp(LayoutData.Anchors.Minimum.Y, 0.0f, LayoutData.Anchors.Maximum.Y);
						break;
					}

					switch ( AnchorType )
					{
					case EAnchorWidget::Bottom:
					case EAnchorWidget::BottomLeft:
					case EAnchorWidget::BottomRight:
						LayoutData.Anchors.Maximum.Y = BeginAnchors.Maximum.Y + AnchorDelta.Y;
						LayoutData.Anchors.Maximum.Y = FMath::Clamp(LayoutData.Anchors.Maximum.Y, LayoutData.Anchors.Minimum.Y, 1.0f);
						break;
					}

					// Major percentage snapping
					if ( !FSlateApplication::Get().GetModifierKeys().IsShiftDown() )
					{
						const float MajorAnchorLine = 0.1f;
						const float MajorAnchorLineSnapDistance = 0.1f;

						if ( LayoutData.Anchors.Minimum.X != OldLayoutData.Anchors.Minimum.X )
						{
							ProximitySnapValue(MajorAnchorLine, MajorAnchorLineSnapDistance, LayoutData.Anchors.Minimum.X);
						}

						if ( LayoutData.Anchors.Minimum.Y != OldLayoutData.Anchors.Minimum.Y )
						{
							ProximitySnapValue(MajorAnchorLine, MajorAnchorLineSnapDistance, LayoutData.Anchors.Minimum.Y);
						}

						if ( LayoutData.Anchors.Maximum.X != OldLayoutData.Anchors.Maximum.X )
						{
							ProximitySnapValue(MajorAnchorLine, MajorAnchorLineSnapDistance, LayoutData.Anchors.Maximum.X);
						}

						if ( LayoutData.Anchors.Maximum.Y != OldLayoutData.Anchors.Maximum.Y )
						{
							ProximitySnapValue(MajorAnchorLine, MajorAnchorLineSnapDistance, LayoutData.Anchors.Maximum.Y);
						}
					}

					// Rebase the layout and restore the old value after calculating the new final layout
					// result.
					{
						PreviewCanvasSlot->SaveBaseLayout();
						PreviewCanvasSlot->SetLayout(LayoutData);
						PreviewCanvasSlot->RebaseLayout();

						LayoutData = PreviewCanvasSlot->GetLayout();
						PreviewCanvasSlot->SetLayout(OldLayoutData);
					}

					// If control is pressed reset all positional offset information
					if ( FSlateApplication::Get().GetModifierKeys().IsControlDown() )
					{
						FMargin NewOffsets = LayoutData.Offsets;

						switch ( AnchorType )
						{
							case EAnchorWidget::Center:
								NewOffsets = FMargin(0, 0, LayoutData.Anchors.IsStretchedHorizontal() ? 0 : LayoutData.Offsets.Right, LayoutData.Anchors.IsStretchedVertical() ? 0 : LayoutData.Offsets.Bottom);
								break;
							case EAnchorWidget::Left:
								NewOffsets.Left = 0;
								break;
							case EAnchorWidget::Right:
								NewOffsets.Right = 0;
								break;
							case EAnchorWidget::Top:
								NewOffsets.Top = 0;
								break;
							case EAnchorWidget::TopLeft:
								NewOffsets.Top = 0;
								NewOffsets.Left = 0;
								break;
							case EAnchorWidget::TopRight:
								NewOffsets.Top = 0;
								NewOffsets.Right = 0;
								break;
							case EAnchorWidget::Bottom:
								NewOffsets.Bottom = 0;
								break;
							case EAnchorWidget::BottomLeft:
								NewOffsets.Bottom = 0;
								NewOffsets.Left = 0;
								break;
							case EAnchorWidget::BottomRight:
								NewOffsets.Bottom = 0;
								NewOffsets.Right = 0;
								break;
						}
						LayoutData.Offsets = NewOffsets;
					}

					UWidget* TemplateWidget = Selection.GetTemplate();
					UCanvasPanelSlot* TemplateCanvasSlot = CastChecked<UCanvasPanelSlot>(TemplateWidget->Slot);

					static const FName LayoutDataName(TEXT("LayoutData"));

					FObjectEditorUtils::SetPropertyValue<UCanvasPanelSlot, FAnchorData>(PreviewCanvasSlot, LayoutDataName, LayoutData);
					FObjectEditorUtils::SetPropertyValue<UCanvasPanelSlot, FAnchorData>(TemplateCanvasSlot, LayoutDataName, LayoutData);
				}
			};

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void FCanvasSlotExtension::PaintDragPercentages(const TSet< FWidgetReference >& InSelection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Just show the percentage lines when we're moving the anchor gizmo
	if ( !(bMovingAnchor || bHoveringAnchor) )
	{
		return;
	}

	for ( const FWidgetReference& Selection : SelectionCache )
	{
		if ( UWidget* PreviewWidget = Selection.GetPreview() )
		{
			if ( UCanvasPanel* Canvas = Cast<UCanvasPanel>(PreviewWidget->GetParent()) )
			{
				UCanvasPanelSlot* PreviewCanvasSlot = CastChecked<UCanvasPanelSlot>(PreviewWidget->Slot);

				FGeometry CanvasGeometry;
				Designer->GetWidgetGeometry(Canvas, CanvasGeometry);
				// Ignore all widget scales and only use the designer scale (text doesn't need the designer scale, however the rendered lines do)
				const FGeometry IgnoreScale = CanvasGeometry.MakeChild(
					CanvasGeometry.GetLocalSize(), Inverse(CanvasGeometry.GetAccumulatedLayoutTransform().GetScale()) * Designer->GetPreviewScale() * AllottedGeometry.Scale
				);
				CanvasGeometry = Designer->MakeGeometryWindowLocal(IgnoreScale);

				FVector2D CanvasSize = CanvasGeometry.GetLocalSize();

				const FAnchorData LayoutData = PreviewCanvasSlot->GetLayout();
				const FVector2D AnchorMin = LayoutData.Anchors.Minimum;
				const FVector2D AnchorMax = LayoutData.Anchors.Maximum;

				auto DrawSegment =[&] (FVector2D Offset, FVector2D Start, FVector2D End, double Value, FVector2D TextTransform, bool InHorizontalLine) {
					PaintLineWithText(
						Start + Offset,
						End + Offset,
						FText::FromString(FString::Printf(TEXT("%.1f%%"), Value)),
						TextTransform,
						InHorizontalLine,
						CanvasGeometry, MyCullingRect, OutDrawElements, LayerId);
				};

				// Horizontal
				{
					auto DrawHorizontalSegment =[&] (FVector2D Offset, FVector2D TextTransform) {
						DrawSegment(Offset, FVector2D::ZeroVector, FVector2D(AnchorMin.X * CanvasSize.X, 0.f), AnchorMin.X * 100.f, FVector2D(1.f, TextTransform.Y), true); // Left
						DrawSegment(Offset, FVector2D(AnchorMax.X * CanvasSize.X, 0.f), FVector2D(CanvasSize.X, 0.f), AnchorMax.X * 100.f, FVector2D(0.f, TextTransform.Y), true); // Right

						if ( LayoutData.Anchors.IsStretchedHorizontal() )
						{
							DrawSegment(Offset, FVector2D(AnchorMin.X * CanvasSize.X, 0.f), FVector2D(AnchorMax.X * CanvasSize.X, 0.f), ( AnchorMax.X - AnchorMin.X ) * 100.f, FVector2D(0.5f, TextTransform.Y), true); // Center
						}
					};

					DrawHorizontalSegment(FVector2D(0.f, AnchorMin.Y * CanvasSize.Y), FVector2D(0.f, -1.f)); // Top

					if ( LayoutData.Anchors.IsStretchedVertical() )
					{
						DrawHorizontalSegment(FVector2D(0.f, AnchorMax.Y * CanvasSize.Y), FVector2D::ZeroVector); // Bottom
					}
				}

				// Vertical
				{
					auto DrawVerticalSegment =[&] (FVector2D Offset, FVector2D TextTransform) {
						DrawSegment(Offset, FVector2D::ZeroVector, FVector2D(0, AnchorMin.Y * CanvasSize.Y), AnchorMin.Y * 100.f, FVector2D(TextTransform.X, 1.f), false); // Top
						DrawSegment(Offset, FVector2D(0.f, AnchorMax.Y * CanvasSize.Y), FVector2D(0, CanvasSize.Y), AnchorMax.Y * 100.f, FVector2D(TextTransform.X, 0.f), false); // Bottom

						if ( LayoutData.Anchors.IsStretchedVertical() )
						{
							DrawSegment(Offset, FVector2D(0.f, AnchorMin.Y * CanvasSize.Y), FVector2D(0.f, AnchorMax.Y * CanvasSize.Y), (AnchorMax.Y - AnchorMin.Y) * 100.f, FVector2D(TextTransform.X, 0.5f), false); // Center
						}
					};

					DrawVerticalSegment(FVector2D(AnchorMin.X * CanvasSize.X, 0.f), FVector2D(-1.f, 0.f)); // Left

					if ( LayoutData.Anchors.IsStretchedHorizontal() )
					{
						DrawVerticalSegment(FVector2D(AnchorMax.X * CanvasSize.X, 0.f), FVector2D::ZeroVector); // Right
					}
				}
			}
		}
	}
}

void FCanvasSlotExtension::PaintLineWithText(FVector2D Start, FVector2D End, FText Text, FVector2D TextTransform, bool InHorizontalLine, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	TArray<FVector2D> LinePoints;
	LinePoints.AddUninitialized(2);
	LinePoints[0] = Start;
	LinePoints[1] = End;

	FLinearColor Color(0.5f, 0.75f, 1);
	const bool bAntialias = true;

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::None,
		Color,
		bAntialias);

	const float InverseDesignerScale = (1.f / Designer->GetPreviewScale());

	const FSlateFontInfo AnchorFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	const FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(Text, AnchorFont);
	FVector2D Offset = FVector2D::ZeroVector;

	if (InHorizontalLine)
	{
		// If the lines run horizontally due to the Y's being the same:

		Offset.X += ((End - Start).X - TextSize.X) * TextTransform.X;
		// Plus a little padding (2).
		Offset.Y += (TextSize.Y * TextTransform.Y) + (20.f * (TextTransform.Y >= 0.f ? 1.f : -1.f));
	}
	else
	{
		// If the lines are running vertically: 

		// Plus a little padding (2).
		Offset.X += (TextSize.X * TextTransform.X) + (20.f * (TextTransform.X >= 0.f ? 1.f : -1.f));
		Offset.Y += ((End - Start).Y - TextSize.Y) * TextTransform.Y;
	}

	const FGeometry ChildGeometry = AllottedGeometry.MakeChild(AllottedGeometry.GetLocalSize(), FSlateLayoutTransform(Start + Offset));

	// Draw drop shadow
	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		ChildGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(InverseDesignerScale, FVector2f(1.f, 1.f))),
		Text,
		AnchorFont,
		ESlateDrawEffect::None,
		FLinearColor::Black
	);

	// Draw normal text on top of drop shadow
	FSlateDrawElement::MakeText(
		OutDrawElements,
		++LayerId,
		ChildGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(InverseDesignerScale)),
		Text,
		AnchorFont,
		ESlateDrawEffect::None,
		FLinearColor::White
	);
}

void FCanvasSlotExtension::PaintCollisionLines(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	for ( const FWidgetReference& WidgetRef : Selection )
	{
		if ( !WidgetRef.IsValid() )
		{
			continue;
		}

		UWidget* Widget = WidgetRef.GetPreview();

		if ( UCanvasPanelSlot* CanvasSlot = Cast<UCanvasPanelSlot>(Widget->Slot) )
		{
			if ( UCanvasPanel* Canvas = CastChecked<UCanvasPanel>(CanvasSlot->Parent) )
			{
				//TODO UMG Only show guide lines when near them and dragging
				#if 0
				{
					FGeometry MyArrangedGeometry;
					if ( !Canvas->GetGeometryForSlot(CanvasSlot, MyArrangedGeometry) )
					{
						continue;
					}

					TArray<FVector2D> LinePoints;
					LinePoints.AddUninitialized(2);

					// Get the collision segments that we could potentially be docking against.
					TArray<FVector2D> MySegments;
					if ( GetCollisionSegmentsForSlot(Canvas, CanvasSlot, MySegments) )
					{
						for ( int32 MySegmentIndex = 0; MySegmentIndex < MySegments.Num(); MySegmentIndex += 2 )
						{
							FVector2D MySegmentBase = MySegments[MySegmentIndex];

							for ( int32 SlotIndex = 0; SlotIndex < Canvas->GetChildrenCount(); SlotIndex++ )
							{
								// Ignore the slot being dragged.
								if ( Canvas->GetSlots()[SlotIndex] == CanvasSlot )
								{
									continue;
								}

								// Get the collision segments that we could potentially be docking against.
								TArray<FVector2D> Segments;
								if ( GetCollisionSegmentsForSlot(Canvas, SlotIndex, Segments) )
								{
									for ( int32 SegmentBase = 0; SegmentBase < Segments.Num(); SegmentBase += 2 )
									{
										FVector2D PointA = Segments[SegmentBase];
										FVector2D PointB = Segments[SegmentBase + 1];

										FVector2D CollisionPoint = MySegmentBase;

										//TODO Collide against all sides of the arranged geometry.
										double Distance = DistancePointToLine2D(PointA, PointB, CollisionPoint);
										if ( Distance <= SnapDistance )
										{
											FVector2D FarthestPoint = FVector2D::Distance(PointA, CollisionPoint) > FVector2D::Distance(PointB, CollisionPoint) ? PointA : PointB;
											FVector2D NearestPoint = FVector2D::Distance(PointA, CollisionPoint) > FVector2D::Distance(PointB, CollisionPoint) ? PointB : PointA;

											LinePoints[0] = FarthestPoint;
											LinePoints[1] = FarthestPoint + ( NearestPoint - FarthestPoint ) * 100000;

											LinePoints[0].X = FMath::Clamp(LinePoints[0].X, 0.0f, MyCullingRect.Right - MyCullingRect.Left);
											LinePoints[0].Y = FMath::Clamp(LinePoints[0].Y, 0.0f, MyCullingRect.Bottom - MyCullingRect.Top);

											LinePoints[1].X = FMath::Clamp(LinePoints[1].X, 0.0f, MyCullingRect.Right - MyCullingRect.Left);
											LinePoints[1].Y = FMath::Clamp(LinePoints[1].Y, 0.0f, MyCullingRect.Bottom - MyCullingRect.Top);

											FLinearColor Color(0.5f, 0.75, 1);
											const bool bAntialias = true;

											FSlateDrawElement::MakeLines(
												OutDrawElements,
												LayerId,
												AllottedGeometry.ToPaintGeometry(),
												LinePoints,
												ESlateDrawEffect::None,
												Color,
												bAntialias);
										}
									}
								}
							}
						}
					}
				}
				#endif
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
