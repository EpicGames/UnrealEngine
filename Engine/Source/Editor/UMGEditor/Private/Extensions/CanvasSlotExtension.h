// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/Layout/Anchors.h"
#include "Widgets/SWidget.h"
#include "WidgetReference.h"
#include "DesignerExtension.h"

class FSlateWindowElementList;
class UCanvasPanel;
class UCanvasPanelSlot;

/** Set of anchor widget types */
enum class EAnchorWidget : uint8
{
	Center,
	Left,
	Right,
	Top,
	Bottom,
	TopLeft,
	TopRight,
	BottomLeft,
	BottomRight,

	Count
};

/**
 * The canvas slot extension provides design time widgets for widgets that are selected in the canvas.
 */
class FCanvasSlotExtension : public FDesignerExtension
{
public:
	FCanvasSlotExtension();

	virtual ~FCanvasSlotExtension() = default;

	virtual bool CanExtendSelection(const TArray< FWidgetReference >& Selection) const override;
	
	virtual void ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements) override;
	virtual void Paint(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;

private:

	FReply HandleAnchorBeginDrag(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType);
	FReply HandleAnchorEndDrag(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType);
	FReply HandleAnchorDragging(const FGeometry& Geometry, const FPointerEvent& Event, EAnchorWidget AnchorType);

	TSharedRef<SWidget> MakeAnchorWidget(EAnchorWidget AnchorType, float Width, float Height);

	void OnMouseEnterAnchor();
	void OnMouseLeaveAnchor();

	const FSlateBrush* GetAnchorBrush(EAnchorWidget AnchorType) const;
	EVisibility GetAnchorVisibility(EAnchorWidget AnchorType) const;
	FVector2D GetAnchorAlignment(EAnchorWidget AnchorType) const;

	static bool GetCollisionSegmentsForSlot(UCanvasPanel* Canvas, int32 SlotIndex, TArray<FVector2D>& Segments);
	static bool GetCollisionSegmentsForSlot(UCanvasPanel* Canvas, UCanvasPanelSlot* Slot, TArray<FVector2D>& Segments);
	static void GetCollisionSegmentsFromGeometry(FGeometry ArrangedGeometry, TArray<FVector2D>& Segments);

	void PaintCollisionLines(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void PaintDragPercentages(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void PaintLineWithText(FVector2D Start, FVector2D End, FText Text, FVector2D TextTransform, bool InHorizontalLine, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	void ProximitySnapValue(float SnapFrequency, float SnapProximity, FVector2D::FReal& Value);

private:

	/** */
	TArray< TSharedPtr<SWidget> > AnchorWidgets;

	/** */
	bool bMovingAnchor;

	/** */
	bool bHoveringAnchor;

	/** */
	FVector2D MouseDownPosition;

	/** */
	FAnchors BeginAnchors;
};
