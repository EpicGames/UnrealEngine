// Copyright Epic Games, Inc. All Rights Reserved.

#include "DesignerExtension.h"

#include "Components/Widget.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "UObject/WeakObjectPtr.h"
#include "WidgetBlueprint.h"
#include "WidgetReference.h"

class FText;

#define LOCTEXT_NAMESPACE "UMG"

// Designer Extension

FDesignerExtension::FDesignerExtension()
	: ScopedTransaction(nullptr)
{

}

FDesignerExtension::~FDesignerExtension()
{
	ensure(Designer == nullptr);
	ensure(ScopedTransaction == nullptr);
}

void FDesignerExtension::Initialize(IUMGDesigner* InDesigner, UWidgetBlueprint* InBlueprint)
{
	Designer = InDesigner;
	Blueprint = InBlueprint;
}

void FDesignerExtension::Uninitialize()
{
	Designer = nullptr;
	Blueprint.Reset();
	ensure(ScopedTransaction == nullptr);
}

FName FDesignerExtension::GetExtensionId() const
{
	return ExtensionId;
}

void FDesignerExtension::BeginTransaction(const FText& SessionName)
{
	if ( ensure(ScopedTransaction == nullptr) )
	{
		ScopedTransaction = new FScopedTransaction(SessionName);
	}

	for ( FWidgetReference& Selection : SelectionCache )
	{
		if ( Selection.IsValid() )
		{
			Selection.GetPreview()->Modify();
			Selection.GetTemplate()->Modify();
		}
	}
}

void FDesignerExtension::EndTransaction()
{
	if ( ensure(ScopedTransaction != nullptr) )
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
