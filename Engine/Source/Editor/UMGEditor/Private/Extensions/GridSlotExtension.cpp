// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/GridSlotExtension.h"

#include "Components/GridSlot.h"
#include "Components/PanelSlot.h"
#include "Components/Widget.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WidgetBlueprint.h"
#include "WidgetReference.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "UMG"

FGridSlotExtension::FGridSlotExtension()
{
	ExtensionId = FName(TEXT("GridSlot"));
}

bool FGridSlotExtension::CanExtendSelection(const TArray< FWidgetReference >& Selection) const
{
	for ( const FWidgetReference& Widget : Selection )
	{
		if ( !Widget.GetTemplate()->Slot || !Widget.GetTemplate()->Slot->IsA(UGridSlot::StaticClass()) )
		{
			return false;
		}
	}

	return Selection.Num() == 1;
}

void FGridSlotExtension::ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
{
	SelectionCache = Selection;

	TSharedRef<SButton> UpArrow = SNew(SButton)
		.Text(LOCTEXT("UpArrow", "\u2191"))
		.ContentPadding(FMargin(6, 2))
		.OnClicked(this, &FGridSlotExtension::HandleShiftRow, -1);

	TSharedRef<SButton> DownArrow = SNew(SButton)
		.Text(LOCTEXT("DownArrow", "\u2193"))
		.ContentPadding(FMargin(6, 2))
		.OnClicked(this, &FGridSlotExtension::HandleShiftRow, 1);

	TSharedRef<SButton> LeftArrow = SNew(SButton)
		.Text(LOCTEXT("LeftArrow", "\u2190"))
		.ContentPadding(FMargin(2, 6))
		.OnClicked(this, &FGridSlotExtension::HandleShiftColumn, -1);

	TSharedRef<SButton> RightArrow = SNew(SButton)
		.Text(LOCTEXT("RightArrow", "\u2192"))
		.ContentPadding(FMargin(2, 6))
		.OnClicked(this, &FGridSlotExtension::HandleShiftColumn, 1);

	UpArrow->SlatePrepass();
	DownArrow->SlatePrepass();
	LeftArrow->SlatePrepass();
	RightArrow->SlatePrepass();

	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(LeftArrow, EExtensionLayoutLocation::CenterLeft, FVector2D(-LeftArrow->GetDesiredSize().X, LeftArrow->GetDesiredSize().Y * -0.5f))));
	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(RightArrow, EExtensionLayoutLocation::CenterRight, FVector2D(0, RightArrow->GetDesiredSize().Y * -0.5f))));
	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(UpArrow, EExtensionLayoutLocation::TopCenter, FVector2D(UpArrow->GetDesiredSize().X * -0.5f, -UpArrow->GetDesiredSize().Y))));
	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(DownArrow, EExtensionLayoutLocation::BottomCenter, FVector2D(DownArrow->GetDesiredSize().X * -0.5f, 0))));
}

FReply FGridSlotExtension::HandleShiftRow(int32 ShiftAmount)
{
	BeginTransaction(LOCTEXT("MoveWidget", "Move Widget"));

	for ( FWidgetReference& Selection : SelectionCache )
	{
		ShiftRow(Selection.GetPreview(), ShiftAmount);
		ShiftRow(Selection.GetTemplate(), ShiftAmount);
	}

	EndTransaction();

	if (UWidgetBlueprint* BlueprintPtr = Blueprint.Get())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BlueprintPtr);
	}

	return FReply::Handled();
}

FReply FGridSlotExtension::HandleShiftColumn(int32 ShiftAmount)
{
	BeginTransaction(LOCTEXT("MoveWidget", "Move Widget"));

	for ( FWidgetReference& Selection : SelectionCache )
	{
		ShiftColumn(Selection.GetPreview(), ShiftAmount);
		ShiftColumn(Selection.GetTemplate(), ShiftAmount);
	}

	EndTransaction();

	if (UWidgetBlueprint* BlueprintPtr = Blueprint.Get())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BlueprintPtr);
	}

	return FReply::Handled();
}

void FGridSlotExtension::ShiftRow(UWidget* Widget, int32 ShiftAmount)
{
	UGridSlot* Slot = Cast<UGridSlot>(Widget->Slot);
	Slot->SetRow(FMath::Max(Slot->GetRow() + ShiftAmount, 0));
}

void FGridSlotExtension::ShiftColumn(UWidget* Widget, int32 ShiftAmount)
{
	UGridSlot* Slot = Cast<UGridSlot>(Widget->Slot);
	Slot->SetColumn(FMath::Max(Slot->GetColumn() + ShiftAmount, 0));
}

#undef LOCTEXT_NAMESPACE
