// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DesignerExtension.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"

class UWidget;
struct FWidgetReference;

class FGridSlotExtension : public FDesignerExtension
{
public:
	FGridSlotExtension();

	virtual ~FGridSlotExtension() = default;

	virtual bool CanExtendSelection(const TArray< FWidgetReference >& Selection) const override;
	
	virtual void ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements) override;

private:

	FReply HandleShiftRow(int32 ShiftAmount);
	FReply HandleShiftColumn(int32 ShiftAmount);

	void ShiftRow(UWidget* Widget, int32 ShiftAmount);
	void ShiftColumn(UWidget* Widget, int32 ShiftAmount);
};
