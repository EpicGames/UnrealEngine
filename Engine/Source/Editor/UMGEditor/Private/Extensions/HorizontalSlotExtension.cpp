// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/HorizontalSlotExtension.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "WidgetBlueprint.h"
#include "Widgets/Input/SButton.h"
#include "Components/HorizontalBoxSlot.h"
#include "Components/HorizontalBox.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

FHorizontalSlotExtension::FHorizontalSlotExtension()
{
	ExtensionId = FName(TEXT("HorizontalSlot"));
}

bool FHorizontalSlotExtension::CanExtendSelection(const TArray< FWidgetReference >& Selection) const
{
	for ( const FWidgetReference& Widget : Selection )
	{
		if ( !Widget.GetTemplate()->Slot || !Widget.GetTemplate()->Slot->IsA(UHorizontalBoxSlot::StaticClass()) )
		{
			return false;
		}
	}

	return Selection.Num() == 1;
}

void FHorizontalSlotExtension::ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
{
	SelectionCache = Selection;

	TSharedRef<SButton> LeftArrow = SNew(SButton)
		.Text(LOCTEXT("LeftArrow", "\u2190"))
		.ContentPadding(FMargin(2, 6))
		.IsEnabled(this, &FHorizontalSlotExtension::CanShift, -1)
		.OnClicked(this, &FHorizontalSlotExtension::HandleShift, -1);

	TSharedRef<SButton> RightArrow = SNew(SButton)
		.Text(LOCTEXT("RightArrow", "\u2192"))
		.ContentPadding(FMargin(2, 6))
		.IsEnabled(this, &FHorizontalSlotExtension::CanShift, 1)
		.OnClicked(this, &FHorizontalSlotExtension::HandleShift, 1);

	LeftArrow->SlatePrepass();
	RightArrow->SlatePrepass();

	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(LeftArrow, EExtensionLayoutLocation::CenterLeft, FVector2D(-LeftArrow->GetDesiredSize().X, LeftArrow->GetDesiredSize().Y * -0.5f))));
	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(RightArrow, EExtensionLayoutLocation::CenterRight, FVector2D(0, RightArrow->GetDesiredSize().Y * -0.5f))));
}

bool FHorizontalSlotExtension::CanShift(int32 ShiftAmount) const
{
	//TODO UMG Provide feedback if shifting is possible.  Tricky with multiple items selected, if we ever support that.
	return true;
}

FReply FHorizontalSlotExtension::HandleShift(int32 ShiftAmount)
{
	BeginTransaction(LOCTEXT("MoveWidget", "Move Widget"));

	for ( FWidgetReference& Selection : SelectionCache )
	{
		ShiftHorizontal(Selection.GetPreview(), ShiftAmount);
		ShiftHorizontal(Selection.GetTemplate(), ShiftAmount);
	}

	EndTransaction();

	//TODO UMG Reorder the live slot without rebuilding the structure

	if (UWidgetBlueprint* BlueprintPtr = Blueprint.Get())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintPtr);
	}

	return FReply::Handled();
}

void FHorizontalSlotExtension::ShiftHorizontal(UWidget* Widget, int32 ShiftAmount)
{
	UHorizontalBox* Parent = CastChecked<UHorizontalBox>(Widget->GetParent());

	Parent->Modify();
	int32 CurrentIndex = Parent->GetChildIndex(Widget);
	Parent->ShiftChild(CurrentIndex + ShiftAmount, Widget);
}

#undef LOCTEXT_NAMESPACE
