// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "WidgetReference.h"
#include "DesignerExtension.h"

class FHorizontalSlotExtension : public FDesignerExtension
{
public:
	FHorizontalSlotExtension();

	virtual ~FHorizontalSlotExtension() = default;

	virtual bool CanExtendSelection(const TArray< FWidgetReference >& Selection) const override;
	
	virtual void ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements) override;

private:

	bool CanShift(int32 ShiftAmount) const;

	FReply HandleShift(int32 ShiftAmount);
	
	void ShiftHorizontal(UWidget* Widget, int32 ShiftAmount);
};
