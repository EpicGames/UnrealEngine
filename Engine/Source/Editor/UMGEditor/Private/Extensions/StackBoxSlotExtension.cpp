// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/StackBoxSlotExtension.h"
#include "Engine/GameViewportClient.h"
#include "Components/StackBox.h"
#include "Components/StackBoxSlot.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/Optional.h"
#include "WidgetBlueprint.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "UMG"

FStackBoxSlotExtension::FStackBoxSlotExtension()
{
	ExtensionId = FName(TEXT("StackBoxSlot"));
}

bool FStackBoxSlotExtension::CanExtendSelection(const TArray< FWidgetReference >& Selection) const
{
	for ( const FWidgetReference& Widget : Selection )
	{
		if ( !Widget.GetTemplate()->Slot || !Widget.GetTemplate()->Slot->IsA(UStackBoxSlot::StaticClass()) )
		{
			return false;
		}
	}

	return Selection.Num() == 1;
}

void FStackBoxSlotExtension::ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
{
	bool bShowWidgets = false;
	TOptional<EOrientation> StackBoxType;
	for (const FWidgetReference& Widget : Selection)
	{
		check(Widget.GetTemplate());
		check(Widget.GetTemplate()->Slot);
		UStackBox* StackBox = CastChecked<UStackBox>(Widget.GetTemplate()->Slot->Parent);
		if (!StackBoxType.IsSet())
		{
			StackBoxType = StackBox->GetOrientation();
			bShowWidgets = true;
		}
		else if (StackBoxType.GetValue() != StackBox->GetOrientation())
		{
			bShowWidgets = false;
			break;
		}
	}

	SelectionCache = Selection;

	if (StackBoxType.IsSet())
	{
		TSharedPtr<SButton> ArrowA;
		TSharedPtr<SButton> ArrowB;
		if (StackBoxType == EOrientation::Orient_Vertical)
		{
			ArrowA = SNew(SButton)
				.Text(LOCTEXT("UpArrow", "\u2191"))
				.ContentPadding(FMargin(6.f, 2.f))
				.IsEnabled(bShowWidgets)
				.OnClicked(this, &FStackBoxSlotExtension::HandleShiftDirection, -1);

			ArrowB = SNew(SButton)
				.Text(LOCTEXT("DownArrow", "\u2193"))
				.ContentPadding(FMargin(6.f, 2.f))
				.IsEnabled(bShowWidgets)
				.OnClicked(this, &FStackBoxSlotExtension::HandleShiftDirection, 1);
		}
		else
		{
			ArrowA = SNew(SButton)
				.Text(LOCTEXT("LeftArrow", "\u2190"))
				.ContentPadding(FMargin(2.f, 6.f))
				.IsEnabled(bShowWidgets)
				.OnClicked(this, &FStackBoxSlotExtension::HandleShiftDirection, -1);

			ArrowB = SNew(SButton)
				.Text(LOCTEXT("RightArrow", "\u2192"))
				.ContentPadding(FMargin(2.f, 6.f))
				.IsEnabled(bShowWidgets)
				.OnClicked(this, &FStackBoxSlotExtension::HandleShiftDirection, 1);
		}

		ArrowA->SlatePrepass();
		ArrowB->SlatePrepass();

		SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(ArrowA.ToSharedRef(), EExtensionLayoutLocation::TopCenter, FVector2D(ArrowA->GetDesiredSize().X * -0.5f, -ArrowA->GetDesiredSize().Y))));
		SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(ArrowB.ToSharedRef(), EExtensionLayoutLocation::BottomCenter, FVector2D(ArrowB->GetDesiredSize().X * -0.5f, 0))));
	}
}

FReply FStackBoxSlotExtension::HandleShiftDirection(int32 ShiftAmount)
{
	BeginTransaction(LOCTEXT("MoveWidget", "Move Widget"));

	for ( FWidgetReference& Selection : SelectionCache )
	{
		ShiftDirection(Selection.GetPreview(), ShiftAmount);
		ShiftDirection(Selection.GetTemplate(), ShiftAmount);
	}

	EndTransaction();

	if (UWidgetBlueprint* BlueprintPtr = Blueprint.Get())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintPtr);
	}

	return FReply::Handled();
}

void FStackBoxSlotExtension::ShiftDirection(UWidget* Widget, int32 ShiftAmount)
{
	UStackBox* Parent = CastChecked<UStackBox>(Widget->GetParent());

	Parent->Modify();
	int32 CurrentIndex = Parent->GetChildIndex(Widget);
	Parent->ShiftChild(CurrentIndex + ShiftAmount, Widget);
}

#undef LOCTEXT_NAMESPACE
