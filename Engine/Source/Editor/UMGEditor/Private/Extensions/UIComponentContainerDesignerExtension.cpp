// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponentContainerDesignerExtension.h"
#include "Blueprint/UserWidget.h"
#include "Slate/SObjectWidget.h"
#include "Extensions/UIComponentContainer.h"
#include "Extensions/UIComponentWidgetBlueprintGeneratedClassExtension.h"

#include "WidgetBlueprint.h"
#include "UIComponentUtils.h"
#include "UIComponentWidgetBlueprintExtension.h"

TSharedRef<FDesignerExtension> FUIComponentContainerDesignerExtensionFactory::CreateDesignerExtension() const
{
	return StaticCastSharedRef<FUIComponentContainerDesignerExtension>(MakeShared<FUIComponentContainerDesignerExtension>());
}

/** Called every time the content of the designer changed. */
void FUIComponentContainerDesignerExtension::PreviewContentCreated(UUserWidget* PreviewWidget)
{
	const UWidgetBlueprint* WidgetBlueprint = Blueprint.Get();
	if (WidgetBlueprint == nullptr || PreviewWidget == nullptr)
	{
		return;
	}
	
	// Here we force to recreate the PreviewWidget Extension from the WidgetBlueprint Extension to make sure they are in sync
	if (UUIComponentWidgetBlueprintExtension* Extension = UUIComponentWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		Extension->GetOrCreateExtension(PreviewWidget);
	}
}

void FUIComponentContainerDesignerExtension::PreviewContentChanged(TSharedRef<SWidget> NewContent)
{
	const UWidgetBlueprint* WidgetBlueprint = Blueprint.Get();
	if (NewContent == SNullWidget::NullWidget || WidgetBlueprint == nullptr)
	{
		return;
	}	
	
	if (UUIComponentWidgetBlueprintExtension* Extension = UUIComponentWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		const SObjectWidget* ObjectWidget = StaticCastSharedPtr<SObjectWidget>(NewContent.ToSharedPtr()).Get();
		if (UUserWidget* PreviewWidget = ObjectWidget ? ObjectWidget->GetWidgetObject() : nullptr)
		{
			Extension->VerifyContainer(PreviewWidget);
		}
	}
}
