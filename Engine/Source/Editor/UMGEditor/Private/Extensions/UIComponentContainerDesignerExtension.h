// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DesignerExtension.h"
#include "IHasDesignerExtensibility.h"

class FUIComponentContainerDesignerExtensionFactory : public IDesignerExtensionFactory, public TSharedFromThis<FUIComponentContainerDesignerExtensionFactory>
{
public:
	FUIComponentContainerDesignerExtensionFactory() = default;
	virtual ~FUIComponentContainerDesignerExtensionFactory() = default;

	virtual TSharedRef<FDesignerExtension> CreateDesignerExtension() const override;
};

class FUIComponentContainerDesignerExtension : public FDesignerExtension
{
public:
	virtual void PreviewContentCreated(UUserWidget* PreviewWidget) override;
	virtual void PreviewContentChanged(TSharedRef<SWidget> NewContent) override;
};
