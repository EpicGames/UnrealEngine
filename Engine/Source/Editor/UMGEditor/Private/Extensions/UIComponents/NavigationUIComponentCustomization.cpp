// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponents/NavigationUIComponentCustomization.h"

#include "UIComponentWidgetBlueprintExtension.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "Extensions/UIComponents/NavigationUIComponent.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Details/SFunctionSelector.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "WidgetBlueprint.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FNavigationCustomizationUIComponent"

namespace UE::NavigationUIComponent::Private
{
struct FScopedNavigationUIComponentTransaction
{
	TSharedPtr<FWidgetBlueprintEditor> Editor;
	UNavigationUIComponent* NavigationUIComponent;
	FScopedTransaction Transaction;
	bool bModifiedProperty;

	FScopedNavigationUIComponentTransaction(TWeakPtr<FWidgetBlueprintEditor> InEditor, UNavigationUIComponent* InNavigationUIComponent, const FText& InTransactionText)
		: Editor(InEditor.Pin())
		, NavigationUIComponent(InNavigationUIComponent)
		, Transaction(InTransactionText)
		, bModifiedProperty(false)
	{
		if (NavigationUIComponent)
		{
			NavigationUIComponent->Modify();
		}
	}

	~FScopedNavigationUIComponentTransaction()
	{
		if (!bModifiedProperty)
		{
			Transaction.Cancel();
		}
	}

	void MarkPropertyModified(const FName& PropertyName)
	{
		if (NavigationUIComponent == nullptr)
		{
			return;
		}
		
		if (FProperty* ChangedProp = FindFProperty<FProperty>(UNavigationUIComponent::StaticClass(), PropertyName))
		{
			FPropertyChangedEvent PropChangedEvent(ChangedProp, EPropertyChangeType::ValueSet, MakeConstArrayView({Cast<UObject>(NavigationUIComponent)}));
			FBlueprintEditorUtils::MarkBlueprintAsModified(Editor->GetWidgetBlueprintObj(), PropChangedEvent);

			bModifiedProperty = true;
		}
	}
};
}

void FNavigationUIComponentCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FNavigationUIComponentCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CustomizeNavigationTransitionFunction(ChildBuilder, true);
	CustomizeNavigationTransitionFunction(ChildBuilder, false);
}

UNavigationUIComponent* FNavigationUIComponentCustomization::GetNavigationComponent() const
{
	TSharedPtr<FWidgetBlueprintEditor> EditorPinned = Editor.Pin();
	if (!EditorPinned.IsValid())
	{
		return nullptr;
	}

	UWidgetBlueprint* WidgetBlueprint = EditorPinned->GetWidgetBlueprintObj();
	if (WidgetBlueprint == nullptr)
	{
		return nullptr;
	}

	UUIComponentWidgetBlueprintExtension* Extension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint);
	if (Extension == nullptr)
	{
		return nullptr;
	}

	if (UUIComponent* Component = Extension->GetComponent(UNavigationUIComponent::StaticClass(), WidgetName))
	{
		if (Component->IsA<UNavigationUIComponent>())
		{
			return Cast<UNavigationUIComponent>(Component);
		}
	}
	return nullptr;
}

void FNavigationUIComponentCustomization::CustomizeNavigationTransitionFunction(class IDetailChildrenBuilder& ChildBuilder, bool bIsEnteredFunction)
{
	static UFunction* OnNavigationTransitionSignature = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/UMG")), TEXT("OnNavigationTransition__DelegateSignature"));

	const FText Name = bIsEnteredFunction ? LOCTEXT("OnNavigationEntered", "On Navigation Entered") : LOCTEXT("OnNavigationExited", "On Navigation Exited");

	ChildBuilder.AddCustomRow(Name)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(Name)
		]
		.ValueContent()
		.MaxDesiredWidth(300)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SFunctionSelector, Editor.Pin().ToSharedRef(), OnNavigationTransitionSignature)
				.CurrentFunction(this, &FNavigationUIComponentCustomization::GetNavigationTransitionFunction, bIsEnteredFunction)
				.OnSelectedFunction(this, &FNavigationUIComponentCustomization::HandleSelectedNavigationTransitionFunction, bIsEnteredFunction)
				.OnResetFunction(this, &FNavigationUIComponentCustomization::HandleResetNavigationTransitionFunction, bIsEnteredFunction)
			]
		];

}

TOptional<FName> FNavigationUIComponentCustomization::GetNavigationTransitionFunction(bool bIsEnteredFunction) const
{
	if ( const UNavigationUIComponent* NavigationUIComponent = GetNavigationComponent())
	{
		if (bIsEnteredFunction)
		{
			return  NavigationUIComponent->OnNavigationEntered;
		}
		else
		{
			return NavigationUIComponent->OnNavigationExited;
		}
	}
	return TOptional<FName>();
}

void FNavigationUIComponentCustomization::HandleSelectedNavigationTransitionFunction(FName SelectedFunction, bool bIsEnteredFunction)
{
	UNavigationUIComponent* NavigationUIComponent = GetNavigationComponent();
	if (NavigationUIComponent == nullptr)
	{
		return;
	}

	UE::NavigationUIComponent::Private::FScopedNavigationUIComponentTransaction NavigationUIComponentTransaction(Editor, NavigationUIComponent, LOCTEXT("SetNavigationTransition", "Set  Navigation Transition"));

	if (bIsEnteredFunction)
	{
		if (NavigationUIComponent->OnNavigationEntered != SelectedFunction)
		{
			NavigationUIComponent->OnNavigationEntered = SelectedFunction;
			NavigationUIComponentTransaction.MarkPropertyModified(GET_MEMBER_NAME_CHECKED(UNavigationUIComponent, OnNavigationEntered));
		}
	}
	else
	{
		if (NavigationUIComponent->OnNavigationExited != SelectedFunction)
		{
			NavigationUIComponent->OnNavigationExited = SelectedFunction;
			NavigationUIComponentTransaction.MarkPropertyModified(GET_MEMBER_NAME_CHECKED(UNavigationUIComponent, OnNavigationExited));
		}
	}
}

void FNavigationUIComponentCustomization::HandleResetNavigationTransitionFunction(bool bIsEnteredFunction)
{
	HandleSelectedNavigationTransitionFunction(NAME_None, bIsEnteredFunction);
}

#undef LOCTEXT_NAMESPACE