// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "WidgetBlueprintEditor.h"
#include "IPropertyTypeCustomization.h"
#include "StructUtils/InstancedStruct.h"
#include "PropertyHandle.h"

class IDetailChildrenBuilder;
class IPropertyUtilities;
class UNavigationUIComponent;

class FNavigationUIComponentCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(TWeakPtr<FWidgetBlueprintEditor> InEditor, FName InWidgetName)
	{
		return MakeShareable(new FNavigationUIComponentCustomization(InEditor, InWidgetName));
	}

	FNavigationUIComponentCustomization(TWeakPtr<FWidgetBlueprintEditor> InEditor, FName InWidgetName)
		: Editor(InEditor)
		, WidgetName(InWidgetName)
	{}
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	UNavigationUIComponent* GetNavigationComponent() const;

	void CustomizeNavigationTransitionFunction(class IDetailChildrenBuilder& ChildBuilder, bool bIsEnteredFunction);

	TOptional<FName> GetNavigationTransitionFunction(bool bIsEnteredFunction) const;
	void HandleSelectedNavigationTransitionFunction(FName SelectedFunction, bool bIsEnteredFunction);
	void HandleResetNavigationTransitionFunction(bool bIsEnteredFunction);
	
	TWeakPtr<FWidgetBlueprintEditor> Editor;
	FName WidgetName;
};
