// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/VerticalSlotExtension.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "WidgetBlueprint.h"
#include "Widgets/Input/SButton.h"
#include "Components/VerticalBoxSlot.h"
#include "Components/VerticalBox.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

FVerticalSlotExtension::FVerticalSlotExtension()
{
	ExtensionId = FName(TEXT("VerticalSlot"));
}

bool FVerticalSlotExtension::CanExtendSelection(const TArray< FWidgetReference >& Selection) const
{
	for ( const FWidgetReference& Widget : Selection )
	{
		if ( !Widget.GetTemplate()->Slot || !Widget.GetTemplate()->Slot->IsA(UVerticalBoxSlot::StaticClass()) )
		{
			return false;
		}
	}

	return Selection.Num() == 1;
}

void FVerticalSlotExtension::ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
{
	SelectionCache = Selection;

	TSharedRef<SButton> UpArrow = SNew(SButton)
		.Text(LOCTEXT("UpArrow", "\u2191"))
		.ContentPadding(FMargin(6, 2))
		.IsEnabled(this, &FVerticalSlotExtension::CanShift, -1)
		.OnClicked(this, &FVerticalSlotExtension::HandleShiftVertical, -1);

	TSharedRef<SButton> DownArrow = SNew(SButton)
		.Text(LOCTEXT("DownArrow", "\u2193"))
		.ContentPadding(FMargin(6, 2))
		.IsEnabled(this, &FVerticalSlotExtension::CanShift, 1)
		.OnClicked(this, &FVerticalSlotExtension::HandleShiftVertical, 1);

	UpArrow->SlatePrepass();
	DownArrow->SlatePrepass();

	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(UpArrow, EExtensionLayoutLocation::TopCenter, FVector2D(UpArrow->GetDesiredSize().X * -0.5f, -UpArrow->GetDesiredSize().Y))));
	SurfaceElements.Add(MakeShareable(new FDesignerSurfaceElement(DownArrow, EExtensionLayoutLocation::BottomCenter, FVector2D(DownArrow->GetDesiredSize().X * -0.5f, 0))));
}

bool FVerticalSlotExtension::CanShift(int32 ShiftAmount) const
{
	//TODO UMG Provide feedback if shifting is possible.  Tricky with multiple items selected, if we ever support that.
	return true;
}

FReply FVerticalSlotExtension::HandleShiftVertical(int32 ShiftAmount)
{
	BeginTransaction(LOCTEXT("MoveWidget", "Move Widget"));

	for ( FWidgetReference& Selection : SelectionCache )
	{
		ShiftVertical(Selection.GetPreview(), ShiftAmount);
		ShiftVertical(Selection.GetTemplate(), ShiftAmount);
	}

	EndTransaction();

	//TODO UMG Reorder the live slot without rebuilding the structure

	if (UWidgetBlueprint* BlueprintPtr = Blueprint.Get())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintPtr);
	}

	return FReply::Handled();
}

void FVerticalSlotExtension::ShiftVertical(UWidget* Widget, int32 ShiftAmount)
{
	UVerticalBox* Parent = CastChecked<UVerticalBox>(Widget->GetParent());

	Parent->Modify();
	int32 CurrentIndex = Parent->GetChildIndex(Widget);
	Parent->ShiftChild(CurrentIndex + ShiftAmount, Widget);
}

#undef LOCTEXT_NAMESPACE
