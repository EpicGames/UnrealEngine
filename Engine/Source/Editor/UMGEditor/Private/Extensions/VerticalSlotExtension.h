// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "WidgetReference.h"
#include "DesignerExtension.h"

class FVerticalSlotExtension : public FDesignerExtension
{
public:
	FVerticalSlotExtension();

	virtual ~FVerticalSlotExtension() = default;

	virtual bool CanExtendSelection(const TArray< FWidgetReference >& Selection) const override;
	
	virtual void ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements) override;

private:

	bool CanShift(int32 ShiftAmount) const;

	FReply HandleShiftVertical(int32 ShiftAmount);

	void ShiftVertical(UWidget* Widget, int32 ShiftAmount);
};
