// Copyright Epic Games, Inc. All Rights Reserved.


#include "FieldNotification/CustomizationHelper.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_FunctionEntry.h"
#include "DetailLayoutBuilder.h"
#include "BlueprintEditorModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "WidgetBlueprint.h"


#define LOCTEXT_NAMESPACE "GraphFunctionDetailsCustomization"

namespace UE::FieldNotification
{

const FName FCustomizationHelper::MetaData_FieldNotify = "FieldNotify";



void FCustomizationHelper::CustomizeVariableDetails(IDetailLayoutBuilder& DetailLayout)
{
	PropertiesBeingCustomized.Reset();

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 0)
	{
		for (TWeakObjectPtr<UObject>& Obj : ObjectsBeingCustomized)
		{
			UPropertyWrapper* PropertyWrapper = Cast<UPropertyWrapper>(Obj.Get());
			FProperty* PropertyBeingCustomized = PropertyWrapper ? PropertyWrapper->GetProperty() : nullptr;
			if (PropertyBeingCustomized && !PropertyBeingCustomized->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				PropertiesBeingCustomized.Emplace(PropertyBeingCustomized);

				const FText ToolTip = LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system.");
				DetailLayout.EditCategory("Variable")
					.AddCustomRow(LOCTEXT("FieldNotify", "Field Notify"))
					.NameContent()
					[
						SNew(STextBlock)
						.Font(DetailLayout.GetDetailFont())
						.Text(LOCTEXT("FieldNotify", "Field Notify"))
						.ToolTipText(ToolTip)
					]
					.ValueContent()
					[
						SNew(SCheckBox)
						.IsChecked_Raw(this, &FCustomizationHelper::IsPropertyFieldNotifyChecked)
						.OnCheckStateChanged_Raw(this, &FCustomizationHelper::HandlePropertyFieldNotifyCheckStateChanged)
						.ToolTipText(ToolTip)
					];
			}
		}
	}
}


void FCustomizationHelper::CustomizeFunctionDetails(IDetailLayoutBuilder& DetailLayout)
{
	FunctionsBeingCustomized.Reset();

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 0)
	{
		for (TWeakObjectPtr<UObject>& Obj : ObjectsBeingCustomized)
		{
			if (UK2Node_FunctionEntry* Function = FindFunctionEntry(Obj.Get()))
			{
				FunctionsBeingCustomized.Emplace(Function);

				const FText ToolTip = LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system.");
				DetailLayout.EditCategory("Graph")
					.AddCustomRow(LOCTEXT("FieldNotify", "Field Notify"))
					.NameContent()
					[
						SNew(STextBlock)
						.Font(DetailLayout.GetDetailFont())
						.Text(LOCTEXT("FieldNotify", "Field Notify"))
						.ToolTipText(ToolTip)
					]
					.ValueContent()
					[
						SNew(SCheckBox)
						.IsChecked_Raw(this, &FCustomizationHelper::IsFunctionFieldNotifyChecked)
						.OnCheckStateChanged_Raw(this, &FCustomizationHelper::HandleFunctionFieldNotifyCheckStateChanged)
						.ToolTipText(ToolTip)
					];
			}
		}
	}
}


ECheckBoxState FCustomizationHelper::IsPropertyFieldNotifyChecked() const
{
	TOptional<ECheckBoxState> ResultState;
	for (const TWeakFieldPtr<FProperty>& WeakProperty : PropertiesBeingCustomized)
	{
		if (const FProperty* Property = WeakProperty.Get())
		{
			ECheckBoxState State = Property->HasMetaData(MetaData_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			if (ResultState.IsSet() && ResultState.GetValue() != State)
			{
				return ECheckBoxState::Undetermined;
			}
			ResultState = State;
		}
		else
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return ResultState.IsSet() ? ResultState.GetValue() : ECheckBoxState::Undetermined;
}


void FCustomizationHelper::HandlePropertyFieldNotifyCheckStateChanged(ECheckBoxState CheckBoxState)
{
	if (UWidgetBlueprint* WidgetBlueprintPtr = Blueprint.Get())
	{
		for (TWeakFieldPtr<FProperty>& WeakProperty : PropertiesBeingCustomized)
		{
			if (FProperty* Property = WeakProperty.Get())
			{
				for (FBPVariableDescription& VariableDescription : WidgetBlueprintPtr->NewVariables)
				{
					if (VariableDescription.VarName == Property->GetFName())
					{
						if (CheckBoxState == ECheckBoxState::Checked)
						{
							Property->SetMetaData(MetaData_FieldNotify, TEXT(""));
							VariableDescription.SetMetaData(MetaData_FieldNotify, FString());
						}
						else
						{
							Property->RemoveMetaData(MetaData_FieldNotify);
							VariableDescription.RemoveMetaData(MetaData_FieldNotify);
						}
					}
				}
			}
		}
	}
}


ECheckBoxState FCustomizationHelper::IsFunctionFieldNotifyChecked() const
{
	TOptional<ECheckBoxState> ResultState;
	for (const TWeakObjectPtr<UK2Node_FunctionEntry>& WeakFunction : FunctionsBeingCustomized)
	{
		if (const UK2Node_FunctionEntry* Function = WeakFunction.Get())
		{
			ECheckBoxState State = Function->MetaData.HasMetaData(MetaData_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			if (ResultState.IsSet() && ResultState.GetValue() != State)
			{
				return ECheckBoxState::Undetermined;
			}
			ResultState = State;
		}
		else
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return ResultState.IsSet() ? ResultState.GetValue() : ECheckBoxState::Undetermined;
}


void FCustomizationHelper::HandleFunctionFieldNotifyCheckStateChanged(ECheckBoxState CheckBoxState)
{
	if (UWidgetBlueprint* WidgetBlueprintPtr = Blueprint.Get())
	{
		for (TWeakObjectPtr<UK2Node_FunctionEntry>& WeakFunction : FunctionsBeingCustomized)
		{
			if (UK2Node_FunctionEntry* Function = WeakFunction.Get())
			{
				if (CheckBoxState == ECheckBoxState::Checked)
				{
					Function->MetaData.SetMetaData(MetaData_FieldNotify, FString());
				}
				else
				{
					Function->MetaData.RemoveMetaData(MetaData_FieldNotify);
				}
			}
		}
	}
}


UK2Node_FunctionEntry* FCustomizationHelper::FindFunctionEntry(UObject* Object) const
{
	if (UK2Node_FunctionEntry* Function = Cast<UK2Node_FunctionEntry>(Object))
	{
		return Function;
	}
	if (UEdGraph* Graph = Cast<UEdGraph>(Object))
	{
		for (UEdGraphNode* GraphNode : Graph->Nodes)
		{
			if (UK2Node_FunctionEntry* Function = Cast<UK2Node_FunctionEntry>(GraphNode))
			{
				return Function;
			}
		}
	}
	return nullptr;
}


} //namespace
#undef LOCTEXT_NAMESPACE
