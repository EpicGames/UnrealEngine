// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Styling/SlateTypes.h"

class IBlueprintEditor;
class UWidgetBlueprint;

namespace UE::FieldNotification
{
	class FCustomizationHelper;
}

class FGraphFunctionDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedPtr<IDetailCustomization> MakeInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor);

	FGraphFunctionDetailsCustomization(UWidgetBlueprint* InBlueprint);

	//~ IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	TPimplPtr<UE::FieldNotification::FCustomizationHelper> Helper;
};
