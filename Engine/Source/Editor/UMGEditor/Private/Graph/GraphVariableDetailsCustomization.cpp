// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphVariableDetailsCustomization.h"
#include "FieldNotification/CustomizationHelper.h"
#include "DetailLayoutBuilder.h"
#include "BlueprintEditorModule.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "WidgetBlueprint.h"


#define LOCTEXT_NAMESPACE "GraphVariableDetailsCustomization"

FGraphVariableDetailsCustomization::FGraphVariableDetailsCustomization(UWidgetBlueprint* InBlueprint)
	: Helper(MakePimpl<UE::FieldNotification::FCustomizationHelper>(InBlueprint))
{}

TSharedPtr<IDetailCustomization> FGraphVariableDetailsCustomization::MakeInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor)
{
	const TArray<UObject*>* Objects = (InBlueprintEditor.IsValid() ? InBlueprintEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects)
	{
		TOptional<UWidgetBlueprint*> FinalBlueprint;
		for (UObject* Object : *Objects)
		{
			UWidgetBlueprint* Blueprint = Cast<UWidgetBlueprint>(Object);
			if (Blueprint == nullptr)
			{
				return nullptr;
			}
			if (FinalBlueprint.IsSet() && FinalBlueprint.GetValue() != Blueprint)
			{
				return nullptr;
			}
			FinalBlueprint = Blueprint;
		}

		if (FinalBlueprint.IsSet())
		{
			return MakeShareable(new FGraphVariableDetailsCustomization(FinalBlueprint.GetValue()));
		}
	}

	return nullptr;
}

void FGraphVariableDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	Helper->CustomizeVariableDetails(DetailLayout);
}

#undef LOCTEXT_NAMESPACE