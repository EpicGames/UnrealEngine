// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "EdGraphSchema_K2.h"
#include "FieldNotificationId.h"
#include "SGraphPin.h"
#include "UObject/Class.h"
#include "Widgets/SFieldNotificationGraphPin.h"


class FUMGGraphPanelPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
			{
				if (PinStructType == FFieldNotificationId::StaticStruct())
				{
					return SNew(UE::FieldNotification::SFieldNotificationGraphPin, InPin);
				}
			}
		}

		return nullptr;
	}
};
