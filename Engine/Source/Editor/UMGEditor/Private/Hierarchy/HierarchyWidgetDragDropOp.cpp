// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchy/HierarchyWidgetDragDropOp.h"

#include "Blueprint/WidgetTree.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

#define LOCTEXT_NAMESPACE "UMG"

TSharedRef<FHierarchyWidgetDragDropOp> FHierarchyWidgetDragDropOp::New(UWidgetBlueprint* Blueprint, const TArray<FWidgetReference>& InWidgets)
{
	check(InWidgets.Num() > 0);

	TSharedRef<FHierarchyWidgetDragDropOp> Operation = MakeShareable(new FHierarchyWidgetDragDropOp());

	// Set the display text and the transaction name based on whether we're dragging a single or multiple widgets
	if (InWidgets.Num() == 1)
	{
		Operation->CurrentHoverText = Operation->DefaultHoverText = InWidgets[0].GetTemplate()->GetLabelText();
	}
	else
	{
		Operation->CurrentHoverText = Operation->DefaultHoverText = LOCTEXT("Designer_DragMultipleWidgets", "Multiple Widgets");
	}

	for (const auto& Widget : InWidgets)
	{
		Operation->WidgetReferences.Add(Widget);
	}

	Operation->Construct();
	
	Blueprint->WidgetTree->SetFlags(RF_Transactional);
	Blueprint->WidgetTree->Modify();

	return Operation;
}

const TArrayView<const FWidgetReference> FHierarchyWidgetDragDropOp::GetWidgetReferences() const
{
	return MakeArrayView(WidgetReferences);
}

bool FHierarchyWidgetDragDropOp::HasOriginatedFrom(const TSharedPtr<FWidgetBlueprintEditor>& BlueprintEditor) const
{
	for (const FWidgetReference& Widget : WidgetReferences)
	{
		if (Widget.GetWidgetEditor() != BlueprintEditor)
		{
			return false;
		}
	}
	return true;
}
#undef LOCTEXT_NAMESPACE
