// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchy/SHierarchyView.h"
#include "WidgetBlueprint.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SScrollBorder.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "Hierarchy/SHierarchyViewItem.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Widgets/Input/SSearchBox.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Views/TreeFilterHandler.h"

#define LOCTEXT_NAMESPACE "UMG"

void SHierarchyView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, USimpleConstructionScript* InSCS)
{
	BlueprintEditor = InBlueprintEditor;
	bRebuildTreeRequested = false;
	bIsUpdatingSelection = false;

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SHierarchyView::OnObjectsReplaced);

	// Create the filter for searching in the tree
	SearchBoxWidgetFilter = MakeShareable(new WidgetTextFilter(WidgetTextFilter::FItemToStringArray::CreateSP(this, &SHierarchyView::GetWidgetFilterStrings)));

	UWidgetBlueprint* Blueprint = GetBlueprint();
	Blueprint->OnChanged().AddRaw(this, &SHierarchyView::OnBlueprintChanged);
	Blueprint->OnCompiled().AddRaw(this, &SHierarchyView::OnBlueprintChanged);

	FilterHandler = MakeShareable(new TreeFilterHandler< TSharedPtr<FHierarchyModel> >());
	FilterHandler->SetFilter(SearchBoxWidgetFilter.Get());
	FilterHandler->SetRootItems(&RootWidgets, &TreeRootWidgets);
	FilterHandler->SetGetChildrenDelegate(TreeFilterHandler< TSharedPtr<FHierarchyModel> >::FOnGetChildren::CreateRaw(this, &SHierarchyView::WidgetHierarchy_OnGetChildren));

	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SHierarchyView::BeginRename),
		FCanExecuteAction::CreateSP(this, &SHierarchyView::CanRename)
	);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.AutoHeight()
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("SearchWidgets", "Search Widgets"))
				.OnTextChanged(this, &SHierarchyView::OnSearchChanged)
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(TreeViewArea, SBorder)
				.Padding(0)
				.BorderImage( FAppStyle::GetBrush( "NoBrush" ) )
			]
		]
	];

	RebuildTreeView();

	BlueprintEditor.Pin()->OnSelectedWidgetsChanged.AddRaw(this, &SHierarchyView::OnEditorSelectionChanged);

	bRefreshRequested = true;
}

SHierarchyView::~SHierarchyView()
{
	UWidgetBlueprint* Blueprint = GetBlueprint();
	if ( Blueprint )
	{
		Blueprint->OnChanged().RemoveAll(this);
		Blueprint->OnCompiled().RemoveAll(this);
	}

	if ( BlueprintEditor.IsValid() )
	{
		BlueprintEditor.Pin()->OnSelectedWidgetsChanged.RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

void SHierarchyView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if ( bRebuildTreeRequested || bRefreshRequested )
	{
		if ( bRebuildTreeRequested )
		{
			RebuildTreeView();
		}

		RefreshTree();

		UpdateItemsExpansionFromModel();

		OnEditorSelectionChanged();

		bRefreshRequested = false;
		bRebuildTreeRequested = false;
	}
}

void SHierarchyView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();
}

void SHierarchyView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	BlueprintEditor.Pin()->ClearHoveredWidget();
}

void SHierarchyView::WidgetHierarchy_OnMouseClick(TSharedPtr<FHierarchyModel> InItem)
{
	BlueprintEditor.Pin()->PasteDropLocation = FVector2D(0, 0);
}

FReply SHierarchyView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{

	if ( BlueprintEditor.Pin()->DesignerCommandList->ProcessCommandBindings(InKeyEvent) )
	{
		return FReply::Handled();
	}

	if ( CommandList->ProcessCommandBindings(InKeyEvent) )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SHierarchyView::BeginRename()
{
	TArray< TSharedPtr<FHierarchyModel> > SelectedItems = WidgetTreeView->GetSelectedItems();
	SelectedItems[0]->RequestBeginRename();
}

bool SHierarchyView::CanRename() const
{
	TArray< TSharedPtr<FHierarchyModel> > SelectedItems = WidgetTreeView->GetSelectedItems();
	return SelectedItems.Num() == 1 && SelectedItems[0]->CanRename();
}

void SHierarchyView::GetWidgetFilterStrings(TSharedPtr<FHierarchyModel> Item, TArray<FString>& OutStrings)
{
	Item->GetFilterStrings(OutStrings);
}

void SHierarchyView::OnSearchChanged(const FText& InFilterText)
{
	bRefreshRequested = true;
	const bool bFilteringEnabled = !InFilterText.IsEmpty();
	if (bFilteringEnabled != FilterHandler->GetIsEnabled())
	{
		FilterHandler->SetIsEnabled(bFilteringEnabled);
		if (bFilteringEnabled)
		{
			SaveItemsExpansion();
		}
		else
		{
			RestoreItemsExpansion();
		}
	}
	SearchBoxWidgetFilter->SetRawFilterText(InFilterText);
	SearchBoxPtr->SetError(SearchBoxWidgetFilter->GetFilterErrorText());
}

FText SHierarchyView::GetSearchText() const
{
	return SearchBoxWidgetFilter->GetRawFilterText();
}

void SHierarchyView::OnEditorSelectionChanged()
{
	if ( !bIsUpdatingSelection )
	{
		WidgetTreeView->ClearSelection();

		if ( RootWidgets.Num() > 0 )
		{
			RootWidgets[0]->RefreshSelection();
		}

		RestoreSelectedItems();
	}
}

UWidgetBlueprint* SHierarchyView::GetBlueprint() const
{
	if ( BlueprintEditor.IsValid() )
	{
		UBlueprint* BP = BlueprintEditor.Pin()->GetBlueprintObj();
		return CastChecked<UWidgetBlueprint>(BP);
	}

	return nullptr;
}

void SHierarchyView::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	if ( InBlueprint )
	{
		bRefreshRequested = true;
	}
}

TSharedPtr<SWidget> SHierarchyView::WidgetHierarchy_OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, CommandList);

	FWidgetBlueprintEditorUtils::CreateWidgetContextMenu(MenuBuilder, BlueprintEditor.Pin().ToSharedRef(), FVector2D(0, 0));

	MenuBuilder.BeginSection("Expansion", LOCTEXT("Expansion", "Expansion"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT( "CollapseAll_Label", "Collapse All" ),
		LOCTEXT( "CollapseAll_Tooltip", "Collapses this item and all children" ),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SHierarchyView::SetItemExpansionRecursive_SelectedItems, false))
	);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "ExpandAll_Label", "Expand All" ),
		LOCTEXT( "ExpandAll_Tooltip", "Expands this item and all children" ),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SHierarchyView::SetItemExpansionRecursive_SelectedItems, true))
	);

	return MenuBuilder.MakeWidget();
}

void SHierarchyView::WidgetHierarchy_OnGetChildren(TSharedPtr<FHierarchyModel> InParent, TArray< TSharedPtr<FHierarchyModel> >& OutChildren)
{
	InParent->GatherChildren(OutChildren);
}

TSharedRef< ITableRow > SHierarchyView::WidgetHierarchy_OnGenerateRow(TSharedPtr<FHierarchyModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SHierarchyViewItem, OwnerTable, InItem)
		.HighlightText(this, &SHierarchyView::GetSearchText);
}

void SHierarchyView::WidgetHierarchy_OnSelectionChanged(TSharedPtr<FHierarchyModel> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if ( SelectInfo != ESelectInfo::Direct )
	{
		bIsUpdatingSelection = true;

		TArray< TSharedPtr<FHierarchyModel> > SelectedItems = WidgetTreeView->GetSelectedItems();

		TSet<FWidgetReference> Clear;
		BlueprintEditor.Pin()->SelectWidgets(Clear, false);

		for ( TSharedPtr<FHierarchyModel>& Item : SelectedItems )
		{
			Item->OnSelection();
		}

		if ( RootWidgets.Num() > 0 )
		{
			RootWidgets[0]->RefreshSelection();
		}

		BlueprintEditor.Pin()->PasteDropLocation = FVector2D(0, 0);

		bIsUpdatingSelection = false;
	}
}

void SHierarchyView::WidgetHierarchy_OnExpansionChanged(TSharedPtr<FHierarchyModel> Item, bool bExpanded)
{
	Item->SetExpanded(bExpanded);
}

FReply SHierarchyView::HandleDeleteSelected()
{
	TSet<FWidgetReference> SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();
	
	FWidgetBlueprintEditorUtils::DeleteWidgets(GetBlueprint(), FWidgetBlueprintEditorUtils::ResolveWidgetTemplates(SelectedWidgets), FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::WarnAndAskUser);

	return FReply::Handled();
}

void SHierarchyView::RefreshTree()
{
	RootWidgets.Empty();
	RootWidgets.Add( MakeShareable(new FHierarchyRoot(BlueprintEditor.Pin())) );

	FilterHandler->RefreshAndFilterTree();
}

void SHierarchyView::RebuildTreeView()
{
	float OldScrollOffset = 0.0f;

	if (WidgetTreeView.IsValid())
	{
		OldScrollOffset = WidgetTreeView->GetScrollOffset();
	}

	SAssignNew(WidgetTreeView, STreeView< TSharedPtr<FHierarchyModel> >)
		.SelectionMode(ESelectionMode::Multi)
		.OnGetChildren(FilterHandler.ToSharedRef(), &TreeFilterHandler< TSharedPtr<FHierarchyModel> >::OnGetFilteredChildren)
		.OnGenerateRow(this, &SHierarchyView::WidgetHierarchy_OnGenerateRow)
		.OnSelectionChanged(this, &SHierarchyView::WidgetHierarchy_OnSelectionChanged)
		.OnExpansionChanged(this, &SHierarchyView::WidgetHierarchy_OnExpansionChanged)
		.OnContextMenuOpening(this, &SHierarchyView::WidgetHierarchy_OnContextMenuOpening)
		.OnSetExpansionRecursive(this, &SHierarchyView::SetItemExpansionRecursive)
		.TreeItemsSource(&TreeRootWidgets)
		.OnMouseButtonClick(this, &SHierarchyView::WidgetHierarchy_OnMouseClick);

	FilterHandler->SetTreeView(WidgetTreeView.Get());

	TreeViewArea->SetContent(
		SNew(SScrollBorder, WidgetTreeView.ToSharedRef())
		[
			WidgetTreeView.ToSharedRef()
		]);

	// Restore the previous scroll offset
	WidgetTreeView->SetScrollOffset(OldScrollOffset);
}

void SHierarchyView::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	if ( !bRebuildTreeRequested )
	{
		for ( const auto& Entry : ReplacementMap )
		{
			if ( Entry.Key->IsA<UVisual>() )
			{
				bRefreshRequested = true;
				bRebuildTreeRequested = true;
			}
		}
	}
}

void SHierarchyView::UpdateItemsExpansionFromModel()
{
	for (TSharedPtr<FHierarchyModel>& Model : RootWidgets)
	{
		RecursiveExpand(Model, EExpandBehavior::FromModel);
	}
}

void SHierarchyView::RestoreItemsExpansion()
{
	for (TSharedPtr<FHierarchyModel>& Model : RootWidgets)
	{
		RecursiveExpand(Model, EExpandBehavior::RestoreFromPrevious);
	}
}

void SHierarchyView::SaveItemsExpansion()
{
	ExpandedItemNames.Empty();

	if (WidgetTreeView.IsValid())
	{
		TSet< TSharedPtr<FHierarchyModel> > ExpandedItems;
		WidgetTreeView->GetExpandedItems(ExpandedItems);

		for (TSharedPtr<FHierarchyModel> Item : ExpandedItems)
		{
			if (Item.IsValid())
			{
				ExpandedItemNames.Add(Item->GetUniqueName());
			}
		}
	}
}

void SHierarchyView::RecursiveExpand(TSharedPtr<FHierarchyModel>& Model, EExpandBehavior ExpandBehavior)
{
	bool bShouldExpandItem = true;

	switch (ExpandBehavior)
	{
		case EExpandBehavior::NeverExpand:
		{
			bShouldExpandItem = false;
		}
		break;

		case EExpandBehavior::RestoreFromPrevious:
		{
			bShouldExpandItem = ExpandedItemNames.Contains(Model->GetUniqueName());
		}
		break;

		case EExpandBehavior::AlwaysExpand:
		{
			bShouldExpandItem = true;
		}
		break;

		case EExpandBehavior::FromModel:
		default:
		{
			bShouldExpandItem = Model->IsExpanded();
		}
		break;
	}

	WidgetTreeView->SetItemExpansion(Model, bShouldExpandItem);

	TArray< TSharedPtr<FHierarchyModel> > Children;
	Model->GatherChildren(Children);

	for (TSharedPtr<FHierarchyModel>& ChildModel : Children)
	{
		RecursiveExpand(ChildModel, ExpandBehavior);
	}
}

void SHierarchyView::RestoreSelectedItems()
{
	// Use single selection mode to update the navigation item as we select.
	// Otherwise keyboard navigation will be off.
	WidgetTreeView->SetSelectionMode(ESelectionMode::Single);

	for ( TSharedPtr<FHierarchyModel>& Model : RootWidgets )
	{
		RecursiveSelection(Model);
	}

	WidgetTreeView->SetSelectionMode(ESelectionMode::Multi);
}

void SHierarchyView::RecursiveSelection(TSharedPtr<FHierarchyModel>& Model)
{
	if ( Model->ContainsSelection() )
	{
		// Expand items that contain selection.
		WidgetTreeView->SetItemExpansion(Model, true);

		TArray< TSharedPtr<FHierarchyModel> > Children;
		Model->GatherChildren(Children);

		for ( TSharedPtr<FHierarchyModel>& ChildModel : Children )
		{
			RecursiveSelection(ChildModel);
		}
	}

	if ( Model->IsSelected() )
	{
		WidgetTreeView->SetItemSelection(Model, true, ESelectInfo::Direct);
		WidgetTreeView->RequestScrollIntoView(Model);
	}
}

void SHierarchyView::SetItemExpansionRecursive(TSharedPtr<FHierarchyModel> Model, bool bInExpansionState)
{
	if (Model.IsValid())
	{
		RecursiveExpand(Model, bInExpansionState ? EExpandBehavior::AlwaysExpand : EExpandBehavior::NeverExpand);
	}
}

void SHierarchyView::SetItemExpansionRecursive_SelectedItems(const bool bInExpansionState)
{
	for (const TSharedPtr<FHierarchyModel>& Item : WidgetTreeView->GetSelectedItems())
	{
		SetItemExpansionRecursive(Item, bInExpansionState);
	}
}

//@TODO UMG Drop widgets onto the tree, when nothing is present, if there is a root node present, what happens then, let the root node attempt to place it?

#undef LOCTEXT_NAMESPACE
