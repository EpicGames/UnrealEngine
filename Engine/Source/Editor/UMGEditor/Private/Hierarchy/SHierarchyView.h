// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Misc/TextFilter.h"
#include "Framework/Views/TreeFilterHandler.h"

class FHierarchyModel;
class FMenuBuilder;
class USimpleConstructionScript;
class UWidgetBlueprint;

/**
 * The tree view presenting the widget hierarchy.  This allows users to edit the hierarchy of widgets easily by dragging and 
 * dropping them logically, which in some cases may be significantly easier than doing it visually in the widget designer.
 */
class SHierarchyView : public SCompoundWidget
{
public:
	typedef TTextFilter< TSharedPtr<FHierarchyModel> > WidgetTextFilter;

public:
	SLATE_BEGIN_ARGS( SHierarchyView ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, USimpleConstructionScript* InSCS);

	virtual ~SHierarchyView();

	// Begin SWidget
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End SWidget

private:
	void BuildWrapWithMenu(FMenuBuilder& Menu);
	TSharedPtr<SWidget> WidgetHierarchy_OnContextMenuOpening();
	void WidgetHierarchy_OnGetChildren(TSharedPtr<FHierarchyModel> InParent, TArray< TSharedPtr<FHierarchyModel> >& OutChildren);
	TSharedRef< ITableRow > WidgetHierarchy_OnGenerateRow(TSharedPtr<FHierarchyModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void WidgetHierarchy_OnSelectionChanged(TSharedPtr<FHierarchyModel> SelectedItem, ESelectInfo::Type SelectInfo);
	void WidgetHierarchy_OnExpansionChanged(TSharedPtr<FHierarchyModel> Item, bool bExpanded);
	void WidgetHierarchy_OnMouseClick(TSharedPtr<FHierarchyModel> InItem);

private:
	void BeginRename();
	bool CanRename() const;

	/** @returns the current blueprint being edited */
	UWidgetBlueprint* GetBlueprint() const;

	/** Called when the blueprint is structurally changed. */
	void OnBlueprintChanged(UBlueprint* InBlueprint);

	/** Called when the selected widget has changed.  The treeview then needs to match the new selection. */
	void OnEditorSelectionChanged();

	/** Notifies the details panel that new widgets have been selected. */
	void ShowDetailsForObjects(TArray<UWidget*> Widgets);

	/** Rebuilds the tree structure based on the current filter options */
	void RefreshTree();

	/** Completely regenerates the treeview */
	void RebuildTreeView();

	/** Handles the menu clicking to delete the selected widgets. */
	FReply HandleDeleteSelected();

	/** Deletes the selected widgets from the hierarchy */
	void DeleteSelected();

	/** Wraps the selected widget using a user selected panel */
	void WrapSelectedWidgets(UClass* WidgetClass);
	
	/** Called when the filter text is changed. */
	void OnSearchChanged(const FText& InFilterText);

	/** Gets the search text currently being used to filter the list, also used to highlight text */
	FText GetSearchText() const;

	/**  Gets an array of strings used for filtering/searching the specified widget. */
	void GetWidgetFilterStrings(TSharedPtr<FHierarchyModel> Widget, TArray<FString>& OutStrings);

	/** Called when a Blueprint is recompiled and live objects are swapped out for replacements */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Sets the expansion state of hierarchy view items based on their model. */
	void UpdateItemsExpansionFromModel();

	/** Stores the names of all currently expanded nodes in the hierarchy view. */
	void SaveItemsExpansion();

	/** Sets the expansion state of hierarchy view items based on the state saved by SaveItemsExpansion. */
	void RestoreItemsExpansion();

	enum class EExpandBehavior : uint8
	{
		NeverExpand,
		AlwaysExpand,
		RestoreFromPrevious,
		FromModel
	};

	/** Recursively expands the models based on the expansion set. */
	void RecursiveExpand(TSharedPtr<FHierarchyModel>& Model, EExpandBehavior ExpandBehavior);

	/**  */
	void RestoreSelectedItems();

	/**  */
	void RecursiveSelection(TSharedPtr<FHierarchyModel>& Model);

	/** Handler for recursively expanding/collapsing items */
	void SetItemExpansionRecursive(TSharedPtr<FHierarchyModel> Model, bool bInExpansionState);
	
	void SetItemExpansionRecursive_SelectedItems(const bool bInExpansionState);

private:

	/** Cached pointer to the blueprint editor that owns this tree. */
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;

	/** Commands specific to the hierarchy. */
	TSharedPtr<FUICommandList> CommandList;

	/** Handles filtering the hierarchy based on an IFilter. */
	TSharedPtr< TreeFilterHandler< TSharedPtr<FHierarchyModel> > > FilterHandler;

	/** The source root widgets for the tree. */
	TArray< TSharedPtr<FHierarchyModel> > RootWidgets;

	/** The root widgets which are actually displayed by the TreeView which will be managed by the TreeFilterHandler. */
	TArray< TSharedPtr<FHierarchyModel> > TreeRootWidgets;

	/** The widget containing the treeview */
	TSharedPtr<SBorder> TreeViewArea;

	/** The widget hierarchy slate treeview widget */
	TSharedPtr< STreeView< TSharedPtr<FHierarchyModel> > > WidgetTreeView;

	/** The unique names of all nodes expanded in the tree view */
	TSet< FName > ExpandedItemNames;

	/** The search box used to update the filter text */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** The filter used by the search box */
	TSharedPtr<WidgetTextFilter> SearchBoxWidgetFilter;

	/** Has a full refresh of the tree been requested?  This happens when the user is filtering the tree */
	bool bRefreshRequested;

	/** Is the tree in such a changed state that the whole widget needs rebuilding? */
	bool bRebuildTreeRequested;

	/** Flag to ignore selections while the hierarchy view is updating the selection. */
	bool bIsUpdatingSelection;
};
