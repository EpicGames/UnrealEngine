// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchy/SHierarchyViewItem.h"
#include "Components/NamedSlotInterface.h"
#include "Blueprint/UserWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"

#include "Styling/CoreStyle.h"
#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "Components/PanelWidget.h"

#include "Kismet2/BlueprintEditorUtils.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragDrop/WidgetTemplateDragDropOp.h"
#include "DragDrop/SelectedWidgetDragDropOp.h"
#include "Hierarchy/HierarchyWidgetDragDropOp.h"

#include "WidgetTemplate.h"

#include "Widgets/Text/SInlineEditableTextBlock.h"

#include "Blueprint/WidgetTree.h"
#include "WidgetBlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "WidgetBlueprintOperationUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Editor
{
	extern bool bDisplayWidgetVariableGuids;
}

/**
*
*/
class FHierarchyWidgetDragDropOpImpl : public FHierarchyWidgetDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FHierarchyWidgetDragDropOpImpl, FHierarchyWidgetDragDropOp)

	FHierarchyWidgetDragDropOpImpl(FHierarchyWidgetDragDropOp& HierarchyWidgetDragDropOp);
	virtual ~FHierarchyWidgetDragDropOpImpl();

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;

	struct FItem
	{
		/** The slot properties for the old slot the widget was in, is used to attempt to reapply the same layout information */
		TMap<FName, FString> ExportedSlotProperties;

		/** The widget being dragged and dropped */
		FWidgetReference Widget;

		/** The original parent of the widget. */
		UWidget* WidgetParent;
	};

	TArray<FItem> DraggedWidgets;

	/** The widget being dragged and dropped */
	FScopedTransaction* Transaction;

	/** Constructs a new drag/drop operation */
	static TSharedRef<FHierarchyWidgetDragDropOpImpl> New(UWidgetBlueprint* Blueprint, const TArray<FWidgetReference>& InWidgets);
};

FHierarchyWidgetDragDropOpImpl::FHierarchyWidgetDragDropOpImpl(FHierarchyWidgetDragDropOp& HierarchyWidgetDragDropOp) 
	: FHierarchyWidgetDragDropOp(HierarchyWidgetDragDropOp)
{
}

TSharedRef<FHierarchyWidgetDragDropOpImpl> FHierarchyWidgetDragDropOpImpl::New(UWidgetBlueprint* Blueprint, const TArray<FWidgetReference>& InWidgets)
{
	check(InWidgets.Num() > 0);

	TSharedRef<FHierarchyWidgetDragDropOp> HierarchyWidgetDragDropOp = FHierarchyWidgetDragDropOp::New(Blueprint, InWidgets);
	TSharedRef<FHierarchyWidgetDragDropOpImpl> Operation = MakeShareable(new FHierarchyWidgetDragDropOpImpl(*HierarchyWidgetDragDropOp));

	// Set the display text and the transaction name based on whether we're dragging a single or multiple widgets
	if (InWidgets.Num() == 1)
	{
		Operation->CurrentHoverText = Operation->DefaultHoverText = InWidgets[0].GetTemplate()->GetLabelText();
		Operation->Transaction = new FScopedTransaction(LOCTEXT("Designer_MoveWidget", "Move Widget"));
	}
	else
	{
		Operation->CurrentHoverText = Operation->DefaultHoverText = LOCTEXT("Designer_DragMultipleWidgets", "Multiple Widgets");
		Operation->Transaction = new FScopedTransaction(LOCTEXT("Designer_MoveWidgets", "Move Widgets"));
	}

	// Add an FItem for each widget in the drag operation
	for (const auto& Widget : InWidgets)
	{
		FItem DraggedWidget;

		DraggedWidget.Widget = Widget;

		FWidgetBlueprintEditorUtils::ExportPropertiesToText(Widget.GetTemplate()->Slot, DraggedWidget.ExportedSlotProperties);

		UWidget* WidgetTemplate = Widget.GetTemplate();
		WidgetTemplate->Modify();

		DraggedWidget.WidgetParent = WidgetTemplate->GetParent();
		if (DraggedWidget.WidgetParent)
		{
			DraggedWidget.WidgetParent->Modify();
		}

		Operation->DraggedWidgets.Add(DraggedWidget);
	}

	Operation->Construct();
	
	Blueprint->WidgetTree->SetFlags(RF_Transactional);
	Blueprint->WidgetTree->Modify();

	return Operation;
}

FHierarchyWidgetDragDropOpImpl::~FHierarchyWidgetDragDropOpImpl()
{
	delete Transaction;
}

void FHierarchyWidgetDragDropOpImpl::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if ( !bDropWasHandled )
	{
		Transaction->Cancel();
	}
}

//////////////////////////////////////////////////////////////////////////

TOptional<EItemDropZone> ProcessHierarchyDragDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, bool bIsDrop, TSharedPtr<FWidgetBlueprintEditor> BlueprintEditor, FWidgetReference TargetItem, TOptional<int32> Index = TOptional<int32>())
{
	UWidget* TargetTemplate = TargetItem.GetTemplate();

	if (TSharedPtr<FHierarchyWidgetDragDropOpImpl> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOpImpl>())
	{
		if (!HierarchyDragDropOp->HasOriginatedFrom(BlueprintEditor))
		{
			return TOptional<EItemDropZone>();
		}
	}

	// We do not support to dragging a Widget from the Viewport to the Hierarchy panel
	if (TSharedPtr<FSelectedWidgetDragDropOp> SelectedWidgetDragDropOp = DragDropEvent.GetOperationAs<FSelectedWidgetDragDropOp>())
	{
		return TOptional<EItemDropZone>();
	}

	if ( TargetTemplate && ( DropZone == EItemDropZone::AboveItem || DropZone == EItemDropZone::BelowItem ) )
	{
		if ( UPanelWidget* TargetParentTemplate = TargetTemplate->GetParent() )
		{
			int32 InsertIndex = TargetParentTemplate->GetChildIndex(TargetTemplate);
			InsertIndex += ( DropZone == EItemDropZone::AboveItem ) ? 0 : 1;
			InsertIndex = FMath::Clamp(InsertIndex, 0, TargetParentTemplate->GetChildrenCount());

			FWidgetReference TargetParentTemplateRef = BlueprintEditor->GetReferenceFromTemplate(TargetParentTemplate);
			TOptional<EItemDropZone> ParentZone = ProcessHierarchyDragDrop(DragDropEvent, EItemDropZone::OntoItem, bIsDrop, BlueprintEditor, TargetParentTemplateRef, InsertIndex);
			if ( ParentZone.IsSet() )
			{
				return DropZone;
			}
			else
			{
				DropZone = EItemDropZone::OntoItem;
			}
		}
	}
	else
	{
		DropZone = EItemDropZone::OntoItem;
	}

	UWidgetBlueprint* Blueprint = BlueprintEditor->GetWidgetBlueprintObj();
	check( Blueprint != nullptr && Blueprint->WidgetTree != nullptr );

	const auto ShouldPreventDropOnTargetExtensions = [](UWidget* Target, const TSharedPtr<FDecoratedDragDropOp>& DecoratedDragDropOp) -> bool
		{
			FText DropOnTargetFailureText = FText::GetEmpty();
			const bool bShouldPreventDropOnTargetExtensions = FWidgetBlueprintEditorUtils::ShouldPreventDropOnTargetExtensions(Target, DecoratedDragDropOp, DropOnTargetFailureText);

			if (bShouldPreventDropOnTargetExtensions && DecoratedDragDropOp.IsValid())
			{
				DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				DecoratedDragDropOp->CurrentHoverText = DropOnTargetFailureText;
			}

			return bShouldPreventDropOnTargetExtensions;
		};

	// Is this a drag/drop op to create a new widget in the tree?
	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid() && !DragDropOp->IsOfType<FHierarchyWidgetDragDropOpImpl>())
	{
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = nullptr;
		if (DragDropOp->IsOfType<FDecoratedDragDropOp>())
		{
			DecoratedDragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(DragDropOp);
			DecoratedDragDropOp->ResetToDefaultToolTip();
		}

		if ( ShouldPreventDropOnTargetExtensions(TargetTemplate, DecoratedDragDropOp) )
		{
			return TOptional<EItemDropZone>();
		}

		if (bIsDrop)
		{
			if (UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp))
			{
				FText OutErrorMessage;
				if (FWidgetBlueprintOperationUtils::AddWidget(Blueprint, Widget, TargetTemplate, Index.IsSet() ? Index.GetValue() : -1, OutErrorMessage))
				{
					if (DecoratedDragDropOp.IsValid())
					{
						DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
					}
					return EItemDropZone::OntoItem;
				}
				else
				{
					if (DecoratedDragDropOp.IsValid())
					{
						DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
						DecoratedDragDropOp->CurrentHoverText = OutErrorMessage;
					}
					FWidgetBlueprintOperationUtils::RemoveTransientWidgetFromTree(Blueprint, Widget);
				}
			}
		}
		else
		{
			FText OutErrorMessage;
			if (FWidgetBlueprintOperationUtils::CanAddToParent(Blueprint, TargetTemplate, OutErrorMessage))
			{
				if (DecoratedDragDropOp.IsValid())
				{
					DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				}
				return EItemDropZone::OntoItem;

			}
			else
			{
				if (DecoratedDragDropOp.IsValid())
				{
					DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
					DecoratedDragDropOp->CurrentHoverText = OutErrorMessage;
				}
			}
		}
	}

	TSharedPtr<FHierarchyWidgetDragDropOpImpl> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOpImpl>();
	if ( HierarchyDragDropOp.IsValid() )
	{
		HierarchyDragDropOp->ResetToDefaultToolTip();

		// If the target item is valid we're dealing with a normal widget in the hierarchy, otherwise we should assume it's
		// the null case and we should be adding it as the root widget.
		if ( TargetItem.IsValid() )
		{
			const bool bIsDraggedObject = HierarchyDragDropOp->DraggedWidgets.ContainsByPredicate([TargetItem](const FHierarchyWidgetDragDropOpImpl::FItem& DraggedItem)
			{
				return DraggedItem.Widget == TargetItem;
			});

			if ( bIsDraggedObject )
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				return TOptional<EItemDropZone>();
			}

			FText OutErrorMessage;

			if (!FWidgetBlueprintOperationUtils::CanAddToParent(Blueprint, TargetTemplate, OutErrorMessage))
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				HierarchyDragDropOp->CurrentHoverText = OutErrorMessage;
				return TOptional<EItemDropZone>();

			}

			UPanelWidget* NewParent = Cast<UPanelWidget>(TargetItem.GetTemplate());
			if (NewParent && !NewParent->CanHaveMultipleChildren() && HierarchyDragDropOp->DraggedWidgets.Num() > 1)
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				HierarchyDragDropOp->CurrentHoverText = LOCTEXT("CantHaveMultipleChildren", "Widget can't have multiple children.");
				return TOptional<EItemDropZone>();
			}

			if (ShouldPreventDropOnTargetExtensions(TargetTemplate, HierarchyDragDropOp))
			{
				return TOptional<EItemDropZone>();
			}

			bool bFoundNewParentInChildSet = false;
			for (const auto& DraggedWidget : HierarchyDragDropOp->DraggedWidgets)
			{
				UWidget* TemplateWidget = DraggedWidget.Widget.GetTemplate();

				if (!FWidgetBlueprintOperationUtils::IsParentChildCycleFree(Blueprint, TemplateWidget, NewParent))
				{
					bFoundNewParentInChildSet = true;
					break;
				}
			}

			if (bFoundNewParentInChildSet)
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				HierarchyDragDropOp->CurrentHoverText = LOCTEXT("CantMakeWidgetChildOfChildren", "Can't make widget a child of its children.");
				return TOptional<EItemDropZone>();
			}

			if ( bIsDrop )
			{
				TSet<FWidgetReference> SelectedTemplates;

				for (const auto& DraggedWidget : HierarchyDragDropOp->DraggedWidgets)
				{
					UWidget* TemplateWidget = DraggedWidget.Widget.GetTemplate();
					OutErrorMessage = FText::GetEmpty();

					if (FWidgetBlueprintOperationUtils::MoveWidget(Blueprint, TemplateWidget, NewParent, Index.IsSet() ? Index.GetValue() : -1, OutErrorMessage))
					{
						if (Index.IsSet() && NewParent)
						{
							Index = NewParent->GetChildIndex(TemplateWidget) + 1;
						}
					}

					SelectedTemplates.Add(BlueprintEditor->GetReferenceFromTemplate(TemplateWidget));
				}

				BlueprintEditor->SelectWidgets(SelectedTemplates, false);
			}

			HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
			return EItemDropZone::OntoItem;
		}
		else
		{
			HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			HierarchyDragDropOp->CurrentHoverText = LOCTEXT("CantHaveChildren", "Widget can't have children.");
		}

		return TOptional<EItemDropZone>();
	}

	return TOptional<EItemDropZone>();
}


//////////////////////////////////////////////////////////////////////////

FHierarchyModel::FHierarchyModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: bInitialized(false)
	, bIsSelected(false)
	, BlueprintEditor(InBlueprintEditor)
{

}

TOptional<EItemDropZone> FHierarchyModel::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
{
	return TOptional<EItemDropZone>();
}

FReply FHierarchyModel::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!IsRoot() && !IsLockedInDesigner())
	{
		TArray<FWidgetReference> DraggedItems;

		// Dragging multiple items?
		if (bIsSelected)
		{
			const TSet<FWidgetReference>& SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();
			if (SelectedWidgets.Num() > 1)
			{
				for (const auto& SelectedWidget : SelectedWidgets)
				{
					DraggedItems.Add(SelectedWidget);
				}
			}
		}

		if (DraggedItems.Num() == 0)
		{
			FWidgetReference ThisItem = AsDraggedWidgetReference();
			if (ThisItem.IsValid())
			{
				DraggedItems.Add(ThisItem);
			}
		}

		if (DraggedItems.Num() > 0)
		{
			return FReply::Handled().BeginDragDrop(FHierarchyWidgetDragDropOpImpl::New(BlueprintEditor.Pin()->GetWidgetBlueprintObj(), DraggedItems));
		}
	}

	return FReply::Unhandled();
}

void FHierarchyModel::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TArray<UWidget*> DragDropPreviewWidgets;
	DetermineDragDropPreviewWidgets(DragDropPreviewWidgets, DragDropEvent);
	// Move the remaining widgets into the transient package. Otherwise, they will remain outered to the WidgetTree and end up as properties in the BP class layout as a result.
	UWidgetBlueprint* BP = Cast<UWidgetBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj());
	if (BP)
	{
		for (UWidget* Widget : DragDropPreviewWidgets)
		{
			FHierarchyModel::RemovePreviewWidget(BP, Widget);
		}
	}
}

void FHierarchyModel::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if ( DecoratedDragDropOp.IsValid() )
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
	TArray<UWidget*> DragDropPreviewWidgets;
	DetermineDragDropPreviewWidgets(DragDropPreviewWidgets, DragDropEvent);
	// Move the remaining widgets into the transient package. Otherwise, they will remain outered to the WidgetTree and end up as properties in the BP class layout as a result.
	UWidgetBlueprint* BP = Cast<UWidgetBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj());
	if (BP)
	{
		for (UWidget* Widget : DragDropPreviewWidgets)
		{
			FHierarchyModel::RemovePreviewWidget(BP, Widget);
		}
	}
}

FReply FHierarchyModel::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone)
{
	return FReply::Unhandled();
}

bool FHierarchyModel::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	return false;
}

void FHierarchyModel::OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
{

}

bool FHierarchyModel::HasCircularReferences(UWidgetBlueprint* Blueprint, UWidget* Widget, TSharedPtr<FDragDropOperation>& DragDropOp)
{
	bool bHasCircularReferences = false;
	if (Widget)
	{
		if (!Blueprint->IsWidgetFreeFromCircularReferences(Cast<UUserWidget>(Widget)))
		{
			if (DragDropOp->IsOfType<FDecoratedDragDropOp>())
			{
				TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(DragDropOp);
				DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				DecoratedDragDropOp->CurrentHoverText = LOCTEXT("CircularReference", "This would cause a circular reference.");
			}

			bHasCircularReferences = true;
		}

		RemovePreviewWidget(Blueprint, Widget);
	}

	return bHasCircularReferences;
}

void FHierarchyModel::DetermineDragDropPreviewWidgets(TArray<UWidget*>& OutWidgets, const FDragDropEvent& DragDropEvent)
{
	OutWidgets.Empty();
	UWidgetBlueprint* Blueprint = Cast<UWidgetBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj());

	if (!Blueprint)
	{
		return;
	}

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp);

	if (Widget)
	{
		OutWidgets.Add(Widget);
	}

	// Mark the widgets for design-time rendering
	for (UWidget* OutWidget : OutWidgets)
	{
		OutWidget->SetDesignerFlags(BlueprintEditor.Pin()->GetCurrentDesignerFlags());
	}
}

void FHierarchyModel::RemovePreviewWidget(UWidgetBlueprint* Blueprint, UWidget* Widget)
{
	FWidgetBlueprintOperationUtils::RemoveTransientWidgetFromTree(Blueprint, Widget);
}

void FHierarchyModel::InitializeChildren()
{
	if ( !bInitialized )
	{
		bInitialized = true;
		GetChildren(Models);
	}
}

void FHierarchyModel::GatherChildren(TArray< TSharedPtr<FHierarchyModel> >& Children)
{
	InitializeChildren();

	Children.Append(Models);
}

bool FHierarchyModel::ContainsSelection()
{
	InitializeChildren();

	for ( TSharedPtr<FHierarchyModel>& Model : Models )
	{
		if ( Model->IsSelected() || Model->ContainsSelection() )
		{
			return true;
		}
	}

	return false;
}

void FHierarchyModel::RefreshSelection()
{
	InitializeChildren();

	UpdateSelection();

	for ( TSharedPtr<FHierarchyModel>& Model : Models )
	{
		Model->RefreshSelection();
	}
}

bool FHierarchyModel::IsSelected() const
{
	return bIsSelected;
}

//////////////////////////////////////////////////////////////////////////

FHierarchyRoot::FHierarchyRoot(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FHierarchyModel(InBlueprintEditor)
{
	RootText = FText::Format(LOCTEXT("RootWidgetFormat", "[{0}]"), FText::FromString(BlueprintEditor.Pin()->GetBlueprintObj()->GetName()));
}

FName FHierarchyRoot::GetUniqueName() const
{
	static const FName DesignerRootName(TEXT("WidgetDesignerRoot"));
	return DesignerRootName;
}

FText FHierarchyRoot::GetText() const
{
	return RootText;
}

const FSlateBrush* FHierarchyRoot::GetImage() const
{
	return nullptr;
}

FSlateFontInfo FHierarchyRoot::GetFont() const
{
	return FCoreStyle::GetDefaultFontStyle("Bold", 10);
}

void FHierarchyRoot::GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children)
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	UWidgetBlueprint* Blueprint = BPEd->GetWidgetBlueprintObj();

	// We only show the hierarchy of the widget tree we own.  If this is a subclass of a widget with a parent tree
	// we won't actually show those widgets here because we can't affect them in a useful inheritable way.
	if ( Blueprint->WidgetTree->RootWidget )
	{
		TSharedPtr<FHierarchyWidget> RootChild = MakeShareable(new FHierarchyWidget(BPEd->GetReferenceFromTemplate(Blueprint->WidgetTree->RootWidget), BPEd));
		Children.Add(RootChild);
	}

	// Grab any exposed named slots from the super classes CDO.  These slots can have content slotted into them by this subclass.
	TSet<FName> InheritedNamedSlotsWithContentInSameTree = Blueprint->GetInheritedNamedSlotsWithContentInSameTree();
	for ( const FName& SlotName : Blueprint->GetInheritedAvailableNamedSlots() )
	{
		if (InheritedNamedSlotsWithContentInSameTree.Contains(SlotName))
		{
			if (!Blueprint->WidgetTree->GetContentForSlot(SlotName))
			{
				continue;
			}
		}

		TSharedPtr<FNamedSlotModelSubclass> ChildItem = MakeShareable(new FNamedSlotModelSubclass(Blueprint, SlotName, BPEd));
		Children.Add(ChildItem);
	}
}

void FHierarchyRoot::OnSelection()
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	if ( UWidget* Default = BPEd->GetWidgetBlueprintObj()->GeneratedClass->GetDefaultObject<UWidget>() )
	{
		TSet<UObject*> SelectedObjects;

		//Switched from adding CDO to adding the preview, so that the root (owner) widget can be properly animate
		UUserWidget* PreviewWidget = BPEd->GetPreview();
		SelectedObjects.Add(PreviewWidget);

		BPEd->SelectObjects(SelectedObjects);
	}
}

void FHierarchyRoot::UpdateSelection()
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	if ( UWidget* Default = BPEd->GetWidgetBlueprintObj()->GeneratedClass->GetDefaultObject<UWidget>() )
	{
		const TSet< TWeakObjectPtr<UObject> >& SelectedObjects = BlueprintEditor.Pin()->GetSelectedObjects();

		TWeakObjectPtr<UObject> PreviewWidget = BPEd->GetPreview();

		bIsSelected = SelectedObjects.Contains(PreviewWidget);
	}
	else
	{
		bIsSelected = false;
	}
}

bool FHierarchyRoot::DoesWidgetOverrideFlowDirection() const
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
	if (UWidget* Default = BPEd->GetWidgetBlueprintObj()->GeneratedClass->GetDefaultObject<UWidget>())
	{
		return Default->GetFlowDirectionPreference() != EFlowDirectionPreference::Inherit;
	}

	return false;
}

TOptional<EItemDropZone> FHierarchyRoot::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
{
	bool bIsFreeFromCircularReferences = true;

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid())
	{
		if(UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj())
		{
			if (UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp))
			{
				bIsFreeFromCircularReferences = !HasCircularReferences(Blueprint, Widget, DragDropOp);
			}
		}
	}

	const bool bIsDrop = false;
	return bIsFreeFromCircularReferences ? ProcessHierarchyDragDrop(DragDropEvent, DropZone, bIsDrop, BlueprintEditor.Pin(), FWidgetReference()) : TOptional<EItemDropZone>();
}

FReply FHierarchyRoot::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone)
{
	const bool bIsDrop = true;
	TOptional<EItemDropZone> Zone = ProcessHierarchyDragDrop(DragDropEvent, DropZone, bIsDrop, BlueprintEditor.Pin(), FWidgetReference());
	if (Zone.IsSet())
	{
		TArray<UWidget*> DragDropPreviewWidgets;
		DetermineDragDropPreviewWidgets(DragDropPreviewWidgets, DragDropEvent);
		// Move the remaining widgets into the transient package. Otherwise, they will remain outered to the WidgetTree and end up as properties in the BP class layout as a result.
		UWidgetBlueprint* BP = Cast<UWidgetBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj());
		if (BP)
		{
			for (UWidget* Widget : DragDropPreviewWidgets)
			{
				FHierarchyModel::RemovePreviewWidget(BP, Widget);
			}
		}
	}
	return Zone.IsSet() ? FReply::Handled() : FReply::Unhandled();
}

//////////////////////////////////////////////////////////////////////////

FNamedSlotModelBase::FNamedSlotModelBase(FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FHierarchyModel(InBlueprintEditor)
	, SlotName(InSlotName)
{
}

const FSlateBrush* FNamedSlotModelBase::GetImage() const
{
	return nullptr;
}

FSlateFontInfo FNamedSlotModelBase::GetFont() const
{
	return FCoreStyle::GetDefaultFontStyle("Bold", 10);
}

FText FNamedSlotModelBase::GetText() const
{
	if (INamedSlotInterface* NamedSlotHost = GetNamedSlotHost())
	{
		TSet<FWidgetReference> SelectedWidgets;
		if ( UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName) )
		{
			return FText::Format(LOCTEXT("NamedSlotTextFormat", "{0} ({1})"), FText::FromName(SlotName), FText::FromName(SlotContent->GetFName()));
		}
	}

	return FText::FromName(SlotName);
}


void FNamedSlotModelBase::GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children)
{
	if (INamedSlotInterface* NamedSlotHost = GetNamedSlotHost())
	{
		TSet<FWidgetReference> SelectedWidgets;
		if ( UWidget* TemplateSlotContent = NamedSlotHost->GetContentForSlot(SlotName) )
		{
			TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();
			TSharedPtr<FHierarchyWidget> RootChild = MakeShareable(new FHierarchyWidget(BPEd->GetReferenceFromTemplate(TemplateSlotContent), BPEd));
			Children.Add(RootChild);
		}
	}
}

void FNamedSlotModelBase::OnSelection()
{
	// No-Op intentionally.
}

void FNamedSlotModelBase::UpdateSelection()
{
	// No-Op intentionally.
}

FWidgetReference FNamedSlotModelBase::AsDraggedWidgetReference() const
{
	if (INamedSlotInterface* NamedSlotHost = GetNamedSlotHost())
	{
		// Only assign content to the named slot if it is null.
		if (UWidget* Content = NamedSlotHost->GetContentForSlot(SlotName))
		{
			return BlueprintEditor.Pin()->GetReferenceFromTemplate(Content);
		}
	}

	return FWidgetReference();
}

TOptional<EItemDropZone> FNamedSlotModelBase::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
{
	UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid() && !DragDropOp->IsOfType<FHierarchyWidgetDragDropOpImpl>())
	{
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = nullptr;
		if (DragDropOp->IsOfType<FDecoratedDragDropOp>())
		{
			DecoratedDragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(DragDropOp);
			DecoratedDragDropOp->ResetToDefaultToolTip();
		}

		if (INamedSlotInterface* NamedSlotHost = GetNamedSlotHost())
		{
			// Only assign content to the named slot if it is null.
			if (NamedSlotHost->GetContentForSlot(SlotName) != nullptr)
			{
				if (DecoratedDragDropOp.IsValid())
				{
					DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
					DecoratedDragDropOp->CurrentHoverText = LOCTEXT("NamedSlotAlreadyFull", "Named Slot already has a child.");
				}
				return TOptional<EItemDropZone>();
			}

			if (UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp))
			{
				const bool bIsFreeFromCircularReferences = !HasCircularReferences(Blueprint, Widget, DragDropOp);
				if (DecoratedDragDropOp.IsValid())
				{
					DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				}
				return bIsFreeFromCircularReferences ? EItemDropZone::OntoItem : TOptional<EItemDropZone>();
			}
		}
	}

	TSharedPtr<FHierarchyWidgetDragDropOpImpl> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOpImpl>();
	if (HierarchyDragDropOp.IsValid() && HierarchyDragDropOp->DraggedWidgets.Num() == 1)
	{
		HierarchyDragDropOp->ResetToDefaultToolTip();

		if (INamedSlotInterface* NamedSlotHost = GetNamedSlotHost())
		{
			// Only assign content to the named slot if it is null.
			if (NamedSlotHost->GetContentForSlot(SlotName) != nullptr)
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				HierarchyDragDropOp->CurrentHoverText = LOCTEXT("NamedSlotAlreadyFull", "Named Slot already has a child.");
				return TOptional<EItemDropZone>();
			}

			bool bFoundNewParentInChildSet = false;
			UWidget* TemplateWidget = HierarchyDragDropOp->DraggedWidgets[0].Widget.GetTemplate();

			// Verify that the new location we're placing the widget is not inside of its existing children.
			Blueprint->WidgetTree->ForWidgetAndChildren(TemplateWidget, [&](UWidget* Widget) {
				if (GetNamedSlotHostWidget() == Widget)
				{
					bFoundNewParentInChildSet = true;
				}
			});

			if (bFoundNewParentInChildSet)
			{
				HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				HierarchyDragDropOp->CurrentHoverText = LOCTEXT("CantMakeWidgetChildOfChildren", "Can't make widget a child of its children.");
				return TOptional<EItemDropZone>();
			}

			HierarchyDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
			return EItemDropZone::OntoItem;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply FNamedSlotModelBase::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone)
{
	INamedSlotInterface* NamedSlotHost = GetNamedSlotHost();
	if (NamedSlotHost == nullptr)
	{
		return FReply::Unhandled();
	}
	if (NamedSlotHost->GetContentForSlot(SlotName) != nullptr)
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid() && !DragDropOp->IsOfType<FHierarchyWidgetDragDropOpImpl>())
	{
		FScopedTransaction Transaction(LOCTEXT("AddWidgetFromTemplate", "Add Widget"));

		UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
		Blueprint->WidgetTree->SetFlags(RF_Transactional);
		Blueprint->WidgetTree->Modify();

		if (UWidget* DroppingWidget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp))
		{
			Blueprint->OnVariableAdded(DroppingWidget->GetFName());
			DoDrop(NamedSlotHost, DroppingWidget);
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	TSharedPtr<FHierarchyWidgetDragDropOpImpl> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyWidgetDragDropOpImpl>();
	if (HierarchyDragDropOp.IsValid() && HierarchyDragDropOp->DraggedWidgets.Num() == 1)
	{
		UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
		Blueprint->WidgetTree->SetFlags(RF_Transactional);
		Blueprint->WidgetTree->Modify();

		UWidget* DroppingWidget = HierarchyDragDropOp->DraggedWidgets[0].Widget.GetTemplate();

		// We don't know if this widget is being removed from a named slot and RemoveFromParent is not enough to take care of this
		if (UWidget* SourceNamedSlotHostWidget = FWidgetBlueprintEditorUtils::FindNamedSlotHostWidgetForContent(DroppingWidget, Blueprint->WidgetTree))
		{
			if (TScriptInterface<INamedSlotInterface> SourceNamedSlotHost = TScriptInterface<INamedSlotInterface>(SourceNamedSlotHostWidget))
			{
				SourceNamedSlotHostWidget->SetFlags(RF_Transactional);
				SourceNamedSlotHostWidget->Modify();
				FWidgetBlueprintEditorUtils::RemoveNamedSlotHostContent(DroppingWidget, SourceNamedSlotHost);
			}
		}
		else
		{
			FName SourceSlotName = Blueprint->WidgetTree->FindSlotForContent(DroppingWidget);
			if (SourceSlotName != NAME_None)
			{
				Blueprint->WidgetTree->SetContentForSlot(SourceSlotName, nullptr);
			}
		}

		DroppingWidget->RemoveFromParent();

		DoDrop(NamedSlotHost, DroppingWidget);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FNamedSlotModelBase::DoDrop(INamedSlotInterface* NamedSlotHost, UWidget* DroppingWidget)
{
	UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();

	if (UObject* NamedSlotHostObject = Cast<UObject>(NamedSlotHost))
	{
		NamedSlotHostObject->SetFlags(RF_Transactional);
		NamedSlotHostObject->Modify();
	}
	
	NamedSlotHost->SetContentForSlot(SlotName, DroppingWidget);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	TSet<FWidgetReference> SelectedTemplates;
	SelectedTemplates.Add(BlueprintEditor.Pin()->GetReferenceFromTemplate(DroppingWidget));

	BlueprintEditor.Pin()->SelectWidgets(SelectedTemplates, false);
}

//////////////////////////////////////////////////////////////////////////

FNamedSlotModel::FNamedSlotModel(FWidgetReference InItem, FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FNamedSlotModelBase(InSlotName, InBlueprintEditor)
	, Item(InItem)
{
#if WITH_EDITOR

	// Revive trashed child of this NamedSlot, if any.
	if (INamedSlotInterface* TemplateWidget = Cast<INamedSlotInterface>(Item.GetTemplate()))
	{
		if (UWidget* SlotContent = TemplateWidget->GetContentForSlot(SlotName))
		{
			if (SlotContent->HasAllFlags(RF_Transient) && SlotContent->GetOuter() == GetTransientPackage() && SlotContent->GetFName().ToString().StartsWith("TRASH_") && BlueprintEditor.IsValid())
			{
				if (UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj())
				{
					FString NewName = SlotContent->GetFName().ToString();
					NewName.RemoveFromStart("TRASH_");
					SlotContent->ClearFlags(RF_Transient);
					SlotContent->Rename(*NewName, Blueprint->WidgetTree);
					TemplateWidget->SetContentForSlot(SlotName, SlotContent);
				}
			}
		}
	}

	// Update the list of bindings if any renaming has occurred.
	if (UUserWidget* TemplateWidget = Cast<UUserWidget>(Item.GetTemplate()))
	{
		TemplateWidget->AssignGUIDToBindings();
		if (!TemplateWidget->GetContentForSlot(SlotName))
		{
			TemplateWidget->UpdateBindingForSlot(SlotName);
		}
	}
#endif
}

FName FNamedSlotModel::GetUniqueName() const
{
	if ( const UWidget* WidgetTemplate = Item.GetTemplate() )
	{
		TStringBuilder<256> StringBuilder;
		StringBuilder.Append(WidgetTemplate->GetName());
		StringBuilder.Append(TEXT("."));
		SlotName.AppendString(StringBuilder);

		return FName(StringBuilder);
	}

	return FName();
}

INamedSlotInterface* FNamedSlotModel::GetNamedSlotHost() const
{
	return Cast<INamedSlotInterface>(Item.GetTemplate());
}

UWidget* FNamedSlotModel::GetNamedSlotHostWidget() const
{
	return Cast<UWidget>(Item.GetTemplate());
}

void FNamedSlotModel::OnSelection()
{
	TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();
	check(Editor.IsValid());

	FNamedSlotSelection Selection;
	Selection.NamedSlotHostWidget = Item;
	Selection.SlotName = SlotName;
	Editor->SetSelectedNamedSlot(Selection);
}

//////////////////////////////////////////////////////////////////////////

FNamedSlotModelSubclass::FNamedSlotModelSubclass(UWidgetBlueprint* InBlueprint, FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FNamedSlotModelBase(InSlotName, InBlueprintEditor)
	, Blueprint(InBlueprint)
{
}

FName FNamedSlotModelSubclass::GetUniqueName() const
{
	const FString UniqueSlot = TEXT("This.") + SlotName.ToString();
	return FName(*UniqueSlot);
}

INamedSlotInterface* FNamedSlotModelSubclass::GetNamedSlotHost() const
{
	return Blueprint->WidgetTree;
}

UWidget* FNamedSlotModelSubclass::GetNamedSlotHostWidget() const
{
	// CDO stored named slot elements don't have a host widget they can use/reference/talk about.
	return nullptr;
}

void FNamedSlotModelSubclass::OnSelection()
{
	TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();

	FNamedSlotSelection Selection;
	Selection.SlotName = SlotName;
	Editor->SetSelectedNamedSlot(Selection);
}

//////////////////////////////////////////////////////////////////////////

FHierarchyWidget::FHierarchyWidget(FWidgetReference InItem, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FHierarchyModel(InBlueprintEditor)
	, Item(InItem)
	, bEditing(false)
	, bNameTextValid(false)
{
}

FName FHierarchyWidget::GetUniqueName() const
{
	UWidget* WidgetTemplate = Item.GetTemplate();
	if ( WidgetTemplate )
	{
		return WidgetTemplate->GetFName();
	}

	return NAME_None;
}

FText FHierarchyWidget::GetText() const
{
	UWidget* WidgetTemplate = Item.GetTemplate();
	if ( WidgetTemplate )
	{
		return bEditing ? WidgetTemplate->GetLabelText() : WidgetTemplate->GetLabelTextWithMetadata();
	}

	return FText::GetEmpty();
}

FText FHierarchyWidget::GetImageToolTipText() const
{
	UWidget* WidgetTemplate = Item.GetTemplate();
	if ( WidgetTemplate )
	{
		UClass* WidgetClass = WidgetTemplate->GetClass();
		if ( WidgetClass->IsChildOf( UUserWidget::StaticClass() ) && WidgetClass->ClassGeneratedBy )
		{
			auto& Description = Cast<UWidgetBlueprint>( WidgetClass->ClassGeneratedBy )->BlueprintDescription;
			if ( Description.Len() > 0 )
			{
				return FText::FromString( Description );
			}
		}
		
		return WidgetClass->GetToolTipText();
	}
	
	return FText::GetEmpty();
}

FText FHierarchyWidget::GetLabelToolTipText() const
{
	FText ToolTipText = FText::GetEmpty();

	auto AppendNewLineOrSetValue = [&ToolTipText](const FText& NewText)
	{
		static const FTextFormat ToolTipNewLineFormat = LOCTEXT("ToolTipNewLineFormat", "{0}\n\n{1}");
		ToolTipText = ToolTipText.IsEmpty() ? NewText : FText::Format(ToolTipNewLineFormat, ToolTipText, NewText);
	};

	if (UWidget* WidgetTemplate = Item.GetTemplate())
	{
		// If the user has provided a name, give a tooltip with the widget type for easy reference
		if (!WidgetTemplate->IsGeneratedName())
		{
			AppendNewLineOrSetValue(FText::FromString(TEXT( "[" ) + WidgetTemplate->GetClass()->GetDisplayNameText().ToString() + TEXT( "]" ) ));
		}
	
		if (UE::UMG::Editor::bDisplayWidgetVariableGuids)
		{
			TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = BlueprintEditor.Pin();
			if (WidgetBlueprintEditor.IsValid())
			{
				if (const UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditor->GetWidgetBlueprintObj())
				{
					static const FTextFormat VariableGUIDFormat = LOCTEXT("WidgetVariableGUIDFormat", "GUID: {0}");
					const FGuid VariableGuid = WidgetBlueprint->WidgetVariableNameToGuidMap.FindRef(WidgetTemplate->GetFName());
					AppendNewLineOrSetValue(FText::Format(VariableGUIDFormat, FText::FromString(VariableGuid.ToString(EGuidFormats::DigitsWithHyphens))));
				}
			}
		}
	}

	return ToolTipText;
}

void FHierarchyWidget::GetFilterStrings(TArray<FString>& OutStrings) const
{
	FHierarchyModel::GetFilterStrings(OutStrings);

	UWidget* WidgetTemplate = Item.GetTemplate();
	if (WidgetTemplate && !WidgetTemplate->IsGeneratedName())
	{
		OutStrings.Add(WidgetTemplate->GetClass()->GetName());
		OutStrings.Add(WidgetTemplate->GetClass()->GetDisplayNameText().ToString());
	}
}

const FSlateBrush* FHierarchyWidget::GetImage() const
{
	if (Item.GetTemplate())
	{
		return FSlateIconFinder::FindIconBrushForClass(Item.GetTemplate()->GetClass());
	}
	return nullptr;
}

FSlateFontInfo FHierarchyWidget::GetFont() const
{
	UWidget* WidgetTemplate = Item.GetTemplate();
	if ( WidgetTemplate )
	{
		if ( !WidgetTemplate->IsGeneratedName() && WidgetTemplate->bIsVariable )
		{
			// TODO UMG Hacky move into style area
			return FCoreStyle::GetDefaultFontStyle("Bold", 10);
		}
	}

	static FName NormalFont("NormalFont");
	return FCoreStyle::Get().GetFontStyle(NormalFont);
}

TOptional<EItemDropZone> FHierarchyWidget::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
{
	bool bIsFreeFromCircularReferences = true;

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid())
	{
		UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
		if (Blueprint)
		{
			if (UWidget* Widget = FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(Blueprint, Blueprint->WidgetTree, DragDropOp))
			{
				bIsFreeFromCircularReferences = !HasCircularReferences(Blueprint, Widget, DragDropOp);
			}
		}
	}
	bool bIsDrop = false;
	return bIsFreeFromCircularReferences ? ProcessHierarchyDragDrop(DragDropEvent, DropZone, bIsDrop, BlueprintEditor.Pin(), Item) :  TOptional<EItemDropZone>();
}

void FHierarchyWidget::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if ( DecoratedDragDropOp.IsValid() )
	{
		DecoratedDragDropOp->ResetToDefaultToolTip();
	}
	FHierarchyModel::HandleDragLeave(DragDropEvent);
}

FReply FHierarchyWidget::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone)
{
	bool bIsDrop = true;
	TOptional<EItemDropZone> Zone = ProcessHierarchyDragDrop(DragDropEvent, DropZone, bIsDrop, BlueprintEditor.Pin(), Item);
	if (Zone.IsSet())
	{
		TArray<UWidget*> DragDropPreviewWidgets;
		DetermineDragDropPreviewWidgets(DragDropPreviewWidgets, DragDropEvent);
		// Move the remaining widgets into the transient package. Otherwise, they will remain outered to the WidgetTree and end up as properties in the BP class layout as a result.
		UWidgetBlueprint* BP = Cast<UWidgetBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj());
		if (BP)
		{
			for (UWidget* Widget : DragDropPreviewWidgets)
			{
				FHierarchyModel::RemovePreviewWidget(BP, Widget);
			}
		}
	}
	return Zone.IsSet() ? FReply::Handled() : FReply::Unhandled();
}

bool FHierarchyWidget::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{	
	 bNameTextValid = FWidgetBlueprintEditorUtils::VerifyWidgetRename(BlueprintEditor.Pin().ToSharedRef(), Item, InText, OutErrorMessage);
	 return bNameTextValid;
}

void FHierarchyWidget::OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || bNameTextValid)
	{
		FWidgetBlueprintEditorUtils::RenameWidget(BlueprintEditor.Pin().ToSharedRef(), Item.GetTemplate()->GetFName(), InText.ToString());
	}
	bNameTextValid = false;
}

void FHierarchyWidget::GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children)
{
	TSharedPtr<FWidgetBlueprintEditor> BPEd = BlueprintEditor.Pin();

	// Check for named slots
	if ( INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(Item.GetTemplate()) )
	{
		TArray<FName> SlotNames;
		NamedSlotHost->GetSlotNames(SlotNames);

		for ( FName& SlotName : SlotNames )
		{
			TSharedPtr<FNamedSlotModel> ChildItem = MakeShareable(new FNamedSlotModel(Item, SlotName, BPEd));
			Children.Add(ChildItem);
		}
	}
	
	// Check if it's a panel widget that can support children
	if ( UPanelWidget* PanelWidget = Cast<UPanelWidget>(Item.GetTemplate()) )
	{
		for ( int32 i = 0; i < PanelWidget->GetChildrenCount(); i++ )
		{
			UWidget* Child = PanelWidget->GetChildAt(i);
			if ( Child )
			{
				TSharedPtr<FHierarchyWidget> ChildItem = MakeShareable(new FHierarchyWidget(BPEd->GetReferenceFromTemplate(Child), BPEd));
				Children.Add(ChildItem);
			}
		}
	}
}

void FHierarchyWidget::OnSelection()
{
	TSet<FWidgetReference> SelectedWidgets;
	SelectedWidgets.Add(Item);

	BlueprintEditor.Pin()->SelectWidgets(SelectedWidgets, true);
}

void FHierarchyWidget::OnMouseEnter()
{
	BlueprintEditor.Pin()->SetHoveredWidget(Item);
}

void FHierarchyWidget::OnMouseLeave()
{
	BlueprintEditor.Pin()->ClearHoveredWidget();
}

bool FHierarchyWidget::IsHovered() const
{
	return BlueprintEditor.Pin()->GetHoveredWidget() == Item;
}

void FHierarchyWidget::UpdateSelection()
{
	const TSet<FWidgetReference>& SelectedWidgets = BlueprintEditor.Pin()->GetSelectedWidgets();
	bIsSelected = SelectedWidgets.Contains(Item);
}

bool FHierarchyWidget::CanRename() const
{
	return !IsLockedInDesigner();
}

void FHierarchyWidget::RequestBeginRename()
{
	RenameEvent.ExecuteIfBound();
}

void FHierarchyWidget::OnBeginEditing()
{
	bEditing = true;
}

void FHierarchyWidget::OnEndEditing()
{
	bEditing = false;
}

//////////////////////////////////////////////////////////////////////////

void SHierarchyViewItem::Construct(const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView, TSharedPtr<FHierarchyModel> InModel)
{
	bHovered = false;
	Model = InModel;
	Model->RenameEvent.BindSP(this, &SHierarchyViewItem::OnRequestBeginRename);

	SetHover(TAttribute<bool>::CreateSP(this, &SHierarchyViewItem::ShouldAppearHovered));

	STableRow< TSharedPtr<FHierarchyModel> >::Construct(
		STableRow< TSharedPtr<FHierarchyModel> >::FArguments()
		.OnCanAcceptDrop(this, &SHierarchyViewItem::HandleCanAcceptDrop)
		.OnAcceptDrop(this, &SHierarchyViewItem::HandleAcceptDrop)
		.OnDragDetected(this, &SHierarchyViewItem::HandleDragDetected)
		.OnDragEnter(this, &SHierarchyViewItem::HandleDragEnter)
		.OnDragLeave(this, &SHierarchyViewItem::HandleDragLeave)
		.Padding(0.0f)
		.Content()
		[
			SNew(SHorizontalBox)
			
			// Widget icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Model->GetImage())
				.ToolTipText(Model->GetImageToolTipText())
			]

			// Name of the widget
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SAssignNew(EditBox, SInlineEditableTextBlock)
				.Font(this, &SHierarchyViewItem::GetItemFont)
				.Text(this, &SHierarchyViewItem::GetItemText)
				.ToolTipText(Model->GetLabelToolTipText())
				.HighlightText(InArgs._HighlightText)
				.IsReadOnly(this, &SHierarchyViewItem::IsReadOnly)
				.OnEnterEditingMode(this, &SHierarchyViewItem::OnBeginNameTextEdit)
				.OnExitEditingMode(this, &SHierarchyViewItem::OnEndNameTextEdit)
				.OnVerifyTextChanged(this, &SHierarchyViewItem::OnVerifyNameTextChanged)
				.OnTextCommitted(this, &SHierarchyViewItem::OnNameTextCommited)
				.IsSelected(this, &SHierarchyViewItem::IsSelectedExclusively)
			]

			// Flow Direction Icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				// TODO Tooltip should tell you what the widget is setup to do
				.ToolTipText(LOCTEXT("NavigationHierarchyToolTip", "This widget overrides the navigation preference."))
				.Visibility_Lambda([InModel] { return InModel->DoesWidgetOverrideNavigation() ? EVisibility::Visible : EVisibility::Collapsed; })
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FEditorFontGlyphs::Arrows)
			]

			// Localization Flow Direction Icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				// TODO Tooltip should tell you what the widget is setup to do
				.ToolTipText(LOCTEXT("FlowDirectionHierarchyToolTip", "This widget overrides the culture/localization flow direction preference."))
				.Visibility_Lambda([InModel] { return InModel->DoesWidgetOverrideFlowDirection() ? EVisibility::Visible : EVisibility::Collapsed; })
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FEditorFontGlyphs::Exchange)
			]

			// Locked Icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FMargin(3, 1))
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
				.OnClicked(this, &SHierarchyViewItem::OnToggleLockedInDesigner)
				.Visibility(Model->CanControlLockedInDesigner() ? EVisibility::Visible : EVisibility::Hidden)
				.ToolTipText(LOCTEXT("WidgetLockedButtonToolTip", "Locks or Unlocks this widget and all children.  Locking a widget prevents it from being selected in the designer view by clicking on them.\n\nHolding [Shift] will only affect this widget and no children."))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ToggleLockedInDesigner")))
				[
					SNew(SBox)
					.MinDesiredWidth(12.0f)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(this, &SHierarchyViewItem::GetLockBrushForWidget)
					]
				]
			]

			// Visibility icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FMargin(3, 1))
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
				.OnClicked(this, &SHierarchyViewItem::OnToggleVisibility)
				.Visibility(Model->CanControlVisibility() ? EVisibility::Visible : EVisibility::Hidden)
				.ToolTipText(LOCTEXT("WidgetVisibilityButtonToolTip", "Toggle Widget's Editor Visibility"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
					.Text(this, &SHierarchyViewItem::GetVisibilityBrushForWidget)
				]
			]
		],
		InOwnerTableView);
}

SHierarchyViewItem::~SHierarchyViewItem()
{
	Model->RenameEvent.Unbind();
}

void SHierarchyViewItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bHovered = true;
	STableRow< TSharedPtr<FHierarchyModel> >::OnMouseEnter(MyGeometry, MouseEvent);

	Model->OnMouseEnter();
}

void SHierarchyViewItem::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	bHovered = false;
	STableRow< TSharedPtr<FHierarchyModel> >::OnMouseLeave(MouseEvent);

	Model->OnMouseLeave();
}

void SHierarchyViewItem::OnBeginNameTextEdit()
{
	Model->OnBeginEditing();

	InitialText = Model->GetText();
}

void SHierarchyViewItem::OnEndNameTextEdit()
{
	Model->OnEndEditing();
}

bool SHierarchyViewItem::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	return Model->OnVerifyNameTextChanged(InText, OutErrorMessage);
}

void SHierarchyViewItem::OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	// The model can return nice names "Border_53" becomes [Border] in some cases
	// This check makes sure we don't rename the object internally to that nice name.
	// Most common case would be the user enters edit mode by accident then just moves focus away.
	if (InitialText.EqualToCaseIgnored(InText))
	{
		return;
	}

	Model->OnNameTextCommited(InText, CommitInfo);
}

bool SHierarchyViewItem::IsReadOnly() const
{
	return !Model->CanRename();
}

void SHierarchyViewItem::OnRequestBeginRename()
{
	TSharedPtr<SInlineEditableTextBlock> SafeEditBox = EditBox.Pin();
	if ( SafeEditBox.IsValid() )
	{
		SafeEditBox->EnterEditingMode();
	}
}

FSlateFontInfo SHierarchyViewItem::GetItemFont() const
{
	return Model->GetFont();
}

FText SHierarchyViewItem::GetItemText() const
{
	return Model->GetText();
}

bool SHierarchyViewItem::ShouldAppearHovered() const
{
	return bHovered || Model->IsHovered();
}

void SHierarchyViewItem::HandleDragEnter(FDragDropEvent const& DragDropEvent)
{
	Model->HandleDragEnter(DragDropEvent);
	if (DragHoverExpandTimer.IsValid())
	{
		UnRegisterActiveTimer(DragHoverExpandTimer.ToSharedRef());
		DragHoverExpandTimer.Reset();
	}
	if (!IsItemExpanded())
	{
		DragHoverExpandTimer = RegisterActiveTimer( 0.3f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
		{
			if (!IsItemExpanded())
			{
				ToggleExpansion();
			}
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void SHierarchyViewItem::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	Model->HandleDragLeave(DragDropEvent);
	if (DragHoverExpandTimer.IsValid())
	{
		UnRegisterActiveTimer(DragHoverExpandTimer.ToSharedRef());
		DragHoverExpandTimer.Reset();
	}
}

TOptional<EItemDropZone> SHierarchyViewItem::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FHierarchyModel> TargetItem)
{
	return Model->HandleCanAcceptDrop(DragDropEvent, DropZone);
}

FReply SHierarchyViewItem::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return Model->HandleDragDetected(MyGeometry, MouseEvent);
}

FReply SHierarchyViewItem::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FHierarchyModel> TargetItem)
{
	return Model->HandleAcceptDrop(DragDropEvent, DropZone);
}

FReply SHierarchyViewItem::OnToggleVisibility()
{
	Model->SetIsVisible(!Model->IsVisible());

	return FReply::Handled();
}

FText SHierarchyViewItem::GetVisibilityBrushForWidget() const
{
	return Model->IsVisible() ? FEditorFontGlyphs::Eye : FEditorFontGlyphs::Eye_Slash;
}

void SHierarchyViewItem::ToggleLockedInDesigner(const bool bRecursive)
{
	if (Model.IsValid())
	{
		Model->SetIsLockedInDesigner(!Model->IsLockedInDesigner(), bRecursive);
	}
}

FReply SHierarchyViewItem::OnToggleLockedInDesigner()
{
	if ( Model.IsValid() )
	{
		const bool bRecursive = FSlateApplication::Get().GetModifierKeys().IsShiftDown() ? false : true;
		ToggleLockedInDesigner(bRecursive);
	}

	return FReply::Handled();
}

FText SHierarchyViewItem::GetLockBrushForWidget() const
{
	return Model.IsValid() && Model->IsLockedInDesigner() ? FEditorFontGlyphs::Lock : FEditorFontGlyphs::Unlock;
}

#undef LOCTEXT_NAMESPACE
