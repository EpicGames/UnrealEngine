// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "WidgetReference.h"
#include "WidgetBlueprintEditor.h"
#include "Hierarchy/IHierarchyViewItem.h"

class FHierarchyModel : public IHierarchyModel, public TSharedFromThis < FHierarchyModel >
{
public:
	FHierarchyModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~FHierarchyModel() = default;

	/** Gets the unique name of the item used to restore item expansion. */
	virtual FName GetUniqueName() const = 0;

	/* @returns the widget name to use for the tree item */
	virtual FText GetText() const = 0;

	/** @return The tooltip for the tree item image */
	virtual FText GetImageToolTipText() const { return FText::GetEmpty(); }

	/** @return The tooltip for the tree item label */
	virtual FText GetLabelToolTipText() const { return FText::GetEmpty(); }
	
	/** @param OutStrings - Returns an array of strings used for filtering/searching this item. */
	virtual void GetFilterStrings(TArray<FString>& OutStrings) const { OutStrings.Add(GetText().ToString()); }

	virtual const FSlateBrush* GetImage() const = 0;

	virtual FSlateFontInfo GetFont() const = 0;

	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent);
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent);

	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone);
	virtual FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone);

	virtual bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage);

	virtual void OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo);

	virtual void GatherChildren(TArray< TSharedPtr<FHierarchyModel> >& Children);

	virtual void OnSelection() = 0;

	virtual void OnMouseEnter() {}
	virtual void OnMouseLeave() {}

	void RefreshSelection();
	bool ContainsSelection();
	bool IsSelected() const;

	virtual bool IsHovered() const override { return false; }
	virtual bool IsVisible() const override { return true; }
	virtual bool CanControlVisibility() const override { return false; }
	virtual void SetIsVisible(bool IsVisible) { }

	virtual bool CanControlLockedInDesigner() const override { return false; }
	virtual bool IsLockedInDesigner() const override { return false; }
	virtual void SetIsLockedInDesigner(bool NewIsLocked, bool bRecursive)
	{
		if (bRecursive)
		{
			TArray< TSharedPtr<FHierarchyModel> > Children;
			GetChildren(Children);
			for (TSharedPtr<FHierarchyModel>& child : Children)
			{
				if (child.IsValid())
				{
					child->SetIsLockedInDesigner(NewIsLocked, bRecursive);
				}
			}
		}
	}

	virtual bool DoesWidgetOverrideNavigation() const { return false; }
	virtual bool DoesWidgetOverrideFlowDirection() const { return false; }

	virtual bool IsExpanded() const { return true; }
	virtual void SetExpanded(bool bIsExpanded) { }

	virtual bool CanRename() const { return false; }
	virtual void RequestBeginRename() { }

	virtual void OnBeginEditing() { }
	virtual void OnEndEditing() { }

	virtual bool IsRoot() const { return false; }

public:
	FSimpleDelegate RenameEvent;

protected:
	virtual void GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children) = 0;
	virtual void UpdateSelection() = 0;
	virtual FWidgetReference AsDraggedWidgetReference() const { return FWidgetReference(); }
	virtual bool HasCircularReferences(class UWidgetBlueprint* Blueprint, class UWidget* Widget, TSharedPtr<class FDragDropOperation>& DragDropOp);
	void DetermineDragDropPreviewWidgets(TArray<class UWidget*>& OutWidgets, const FDragDropEvent& DragDropEvent);
	void RemovePreviewWidget(class UWidgetBlueprint* Blueprint, class UWidget* Widget);

private:
	void InitializeChildren();

protected:

	bool bInitialized;
	bool bIsSelected;
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TArray< TSharedPtr<FHierarchyModel> > Models;
};

class FHierarchyRoot : public FHierarchyModel
{
public:
	FHierarchyRoot(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual ~FHierarchyRoot() = default;

	virtual FName GetUniqueName() const override;

	/* @returns the widget name to use for the tree item */
	virtual FText GetText() const override;

	virtual const FSlateBrush* GetImage() const override;

	virtual FSlateFontInfo GetFont() const override;

	virtual void OnSelection() override;

	virtual bool DoesWidgetOverrideFlowDirection() const override;
	virtual bool DoesWidgetOverrideNavigation() const override { return false; }

	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) override;
	virtual FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone) override;

	virtual bool IsRoot() const override { return true; }

protected:
	virtual void GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children) override;
	virtual void UpdateSelection() override;

private:
	FText RootText;
};

class FNamedSlotModelBase : public FHierarchyModel
{
public:
	FNamedSlotModelBase(FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual ~FNamedSlotModelBase() = default;

	virtual FName GetUniqueName() const = 0;
	virtual FText GetText() const override;

	virtual const FSlateBrush* GetImage() const override;
	virtual FSlateFontInfo GetFont() const override;

	virtual void OnSelection() override;
	
	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) override;
	virtual FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone) override;

protected:
	virtual INamedSlotInterface* GetNamedSlotHost() const = 0;
	virtual UWidget* GetNamedSlotHostWidget() const = 0;
	
	virtual void GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children) override;
	virtual void UpdateSelection() override;
	virtual FWidgetReference AsDraggedWidgetReference() const override;

	virtual void DoDrop(INamedSlotInterface* NamedSlotHost, UWidget* DroppingWidget);

protected:

	FName SlotName;
};

class FNamedSlotModel : public FNamedSlotModelBase
{
public:
	FNamedSlotModel(FWidgetReference InItem, FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual ~FNamedSlotModel() = default;

	virtual FName GetUniqueName() const override;
	virtual INamedSlotInterface* GetNamedSlotHost() const override;
	virtual UWidget* GetNamedSlotHostWidget() const override;

protected:
	virtual void OnSelection() override;

protected:

	FWidgetReference Item;
};

class FNamedSlotModelSubclass : public FNamedSlotModelBase
{
public:
	FNamedSlotModelSubclass(UWidgetBlueprint* Blueprint, FName InSlotName, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual ~FNamedSlotModelSubclass() = default;

	virtual FName GetUniqueName() const override;
	virtual INamedSlotInterface* GetNamedSlotHost() const override;
	virtual UWidget* GetNamedSlotHostWidget() const override;

protected:
	virtual void OnSelection() override;

protected:
	TWeakObjectPtr<UWidgetBlueprint> Blueprint;
};

class FHierarchyWidget : public FHierarchyModel
{
public:
	FHierarchyWidget(FWidgetReference InItem, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual ~FHierarchyWidget() = default;

	virtual FName GetUniqueName() const override;
	
	/* @returns the widget name to use for the tree item */
	virtual FText GetText() const override;
	virtual FText GetImageToolTipText() const override;
	virtual FText GetLabelToolTipText() const override;
	
	virtual void GetFilterStrings(TArray<FString>& OutStrings) const override;

	virtual const FSlateBrush* GetImage() const override;

	virtual FSlateFontInfo GetFont() const override;

	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) override;

	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone) override;

	virtual bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage) override;

	virtual void OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo) override;

	virtual void OnSelection() override;

	virtual void OnMouseEnter() override;
	virtual void OnMouseLeave() override;

	virtual bool IsHovered() const override;

	virtual bool IsVisible() const override
	{
		if ( UWidget* TemplateWidget = Item.GetTemplate() )
		{
			return !TemplateWidget->bHiddenInDesigner;
		}

		return true;
	}

	virtual bool CanControlVisibility() const override
	{
		return true;
	}

	virtual void SetIsVisible(bool IsVisible) override
	{
		Item.GetTemplate()->bHiddenInDesigner = !IsVisible;

		if ( UWidget* PreviewWidget = Item.GetPreview() )
		{
			PreviewWidget->bHiddenInDesigner = !IsVisible;
		}
	}

	virtual bool CanControlLockedInDesigner() const override
	{
		return true;
	}

	virtual bool IsLockedInDesigner() const override
	{
		UWidget* TemplateWidget = Item.GetTemplate();
		if (TemplateWidget)
		{
			return TemplateWidget->IsLockedInDesigner();
		}

		return false;
	}

	virtual void SetIsLockedInDesigner(bool NewIsLocked, bool bRecursive) override
	{
		FHierarchyModel::SetIsLockedInDesigner(NewIsLocked, bRecursive);

		if(Item.GetTemplate() && Item.GetPreview())
		{
			Item.GetTemplate()->SetLockedInDesigner(NewIsLocked);
			Item.GetPreview()->SetLockedInDesigner(NewIsLocked);
		}
	}

	virtual bool DoesWidgetOverrideFlowDirection() const override
	{
		UWidget* TemplateWidget = Item.GetTemplate();
		if (TemplateWidget)
		{
			return TemplateWidget->GetFlowDirectionPreference() != EFlowDirectionPreference::Inherit;
		}

		return false;
	}

	virtual bool DoesWidgetOverrideNavigation() const override
	{
		UWidget* TemplateWidget = Item.GetTemplate();
		if (TemplateWidget)
		{
			return TemplateWidget->Navigation != nullptr;
		}

		return false;
	}

	virtual bool IsExpanded() const override
	{
		if (UWidget* Template = Item.GetTemplate())
		{
			return Template->bExpandedInDesigner;
		}

		return false;
	}

	virtual void SetExpanded(bool bIsExpanded) override
	{
		if (UWidget* Template = Item.GetTemplate())
		{
			Template->bExpandedInDesigner = bIsExpanded;
		}
	}

	virtual bool CanRename() const override;
	virtual void RequestBeginRename() override;

	virtual void OnBeginEditing() override;
	virtual void OnEndEditing() override;

protected:
	virtual void GetChildren(TArray< TSharedPtr<FHierarchyModel> >& Children) override;
	virtual void UpdateSelection() override;
	virtual FWidgetReference AsDraggedWidgetReference() const override { return Item; }

private:
	FWidgetReference Item;
	bool bEditing;
	bool bNameTextValid;
};

/**
 * An widget item in the hierarchy tree view.
 */
class SHierarchyViewItem : public IHierarchyViewItem< FHierarchyModel >
{
public:
	SLATE_BEGIN_ARGS( SHierarchyViewItem ){}
		
		/** The current text to highlight */
		SLATE_ATTRIBUTE( FText, HighlightText )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef< STableViewBase >& InOwnerTableView, TSharedPtr<FHierarchyModel> InModel);
	virtual ~SHierarchyViewItem();

	// Begin SWidget
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// End SWidget

public:
	virtual FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void HandleDragEnter(const FDragDropEvent& DragDropEvent);
	void HandleDragLeave(const FDragDropEvent& DragDropEvent);

	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FHierarchyModel> TargetItem);
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FHierarchyModel> TargetItem);

private:

	void OnBeginNameTextEdit();
	void OnEndNameTextEdit();

	/** Called when text is being committed to check for validity */
	bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage);

	/** Called when text is committed on the node */
	void OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo);

	bool IsReadOnly() const;
	void OnRequestBeginRename();
	virtual bool ShouldAppearHovered() const override;

	/** Gets the font to use for the text item, bold for customized named items */
	FSlateFontInfo GetItemFont() const;

	/** @returns the widget name to use for the tree item */
	FText GetItemText() const;

	/** Handles clicking the visibility toggle */
	FReply OnToggleVisibility();

	/** Toggles lock status */
	virtual void ToggleLockedInDesigner(const bool bRecursive) override;

	/** Handles clicking the locked toggle */
	FReply OnToggleLockedInDesigner();

	/** Returns a brush representing the lock item of the widget's lock button */
	FText GetLockBrushForWidget() const;

	/** Returns a brush representing the visibility item of the widget's visibility button */
	FText GetVisibilityBrushForWidget() const;

private:

	/** Edit box for the name. */
	TWeakPtr<SInlineEditableTextBlock> EditBox;

	/* The mode that this tree item represents */
	TSharedPtr<FHierarchyModel> Model;

	/** Text when we start editing. */
	FText InitialText;

	/** Keep an internal IsHovered flag*/
	bool bHovered = false;
	
	/** If the user is dragging over a collapsed item, wait before expanding it */
	TSharedPtr<FActiveTimerHandle> DragHoverExpandTimer;
};
