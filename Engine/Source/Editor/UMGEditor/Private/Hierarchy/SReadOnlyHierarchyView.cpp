// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchy/SReadOnlyHierarchyView.h"

#include "Algo/Transform.h"
#include "Blueprint/WidgetTree.h"
#include "Components/NamedSlotInterface.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SSearchBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SReadOnlyHierarchyView)

#define LOCTEXT_NAMESPACE "SReadOnlyHierarchyView"

void SReadOnlyHierarchyView::Construct(const FArguments& InArgs, const UWidgetBlueprint* InWidgetBlueprint)
{
	OnSelectionChangedDelegate = InArgs._OnSelectionChanged;
	WidgetBlueprint = InWidgetBlueprint;
	ShowOnly = InArgs._ShowOnly;
	RootSelectionMode = InArgs._RootSelectionMode;
	InheritedNamedSlotSelectionMode = InArgs._InheritedNamedSlotSelectionMode;
	bExpandAll = InArgs._ExpandAll;

	SearchFilter = MakeShared<FTextFilter>(FTextFilter::FItemToStringArray::CreateSP(this, &SReadOnlyHierarchyView::GetFilterStringsForItem));
	FilterHandler = MakeShared<FTreeFilterHandler>();
	FilterHandler->SetFilter(SearchFilter.Get());
	FilterHandler->SetRootItems(&RootWidgets, &FilteredRootWidgets);
	FilterHandler->SetGetChildrenDelegate(FTreeFilterHandler::FOnGetChildren::CreateSP(this, &SReadOnlyHierarchyView::GetItemChildren));

	TreeView = SNew(STreeView<TSharedPtr<FItem>>)
		.OnGenerateRow(this, &SReadOnlyHierarchyView::GenerateRow)
		.OnGetChildren(FilterHandler.ToSharedRef(), &FTreeFilterHandler::OnGetFilteredChildren)
		.OnSelectionChanged(this, &SReadOnlyHierarchyView::OnSelectionChanged)
		.SelectionMode(InArgs._SelectionMode)
		.TreeItemsSource(&FilteredRootWidgets)
		.ClearSelectionOnClick(false)
		.OnSetExpansionRecursive(this, &SReadOnlyHierarchyView::SetItemExpansionRecursive);

	FilterHandler->SetTreeView(TreeView.Get());

	Refresh();

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox);

	if (InArgs._ShowSearch)
	{
		ContentBox->AddSlot()
			.Padding(2.0f)
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &SReadOnlyHierarchyView::SetRawFilterText)
			];
	}

	ContentBox->AddSlot()
		[
			TreeView.ToSharedRef()
		];

	ChildSlot
	[
		ContentBox
	];
}

SReadOnlyHierarchyView::~SReadOnlyHierarchyView()
{
}

FName SReadOnlyHierarchyView::GetItemName(const TSharedPtr<FItem>& Item) const 
{
	if (!Item.IsValid())
	{
		return FName();
	}

	if (const UWidgetBlueprint* WidgetBP = Item->WidgetBlueprint.Get())
	{
		return WidgetBP->GetFName();
	}

	if (const UWidget* Widget = Item->Widget.Get())
	{
		return Widget->GetFName();
	}

	const FName NamedSlotName = Item->InheritedNamedSlotName;
	if (!NamedSlotName.IsNone())
	{
		return NamedSlotName;
	}

	return FName();
}

void SReadOnlyHierarchyView::OnSelectionChanged(TSharedPtr<FItem> Selected, ESelectInfo::Type SelectionType)
{
	OnSelectionChangedDelegate.ExecuteIfBound(GetItemName(Selected), SelectionType);
}

void SReadOnlyHierarchyView::Refresh()
{
	RootWidgets.Reset();
	FilteredRootWidgets.Reset();
	RebuildTree();
	FilterHandler->RefreshAndFilterTree();
	if (bExpandAll)
	{
		ExpandAll();
	}
}

void SReadOnlyHierarchyView::SetItemExpansionRecursive(TSharedPtr<FItem> Item, bool bShouldBeExpanded)
{
	TreeView->SetItemExpansion(Item, bShouldBeExpanded);

	for (const TSharedPtr<FItem>& Child : Item->Children)
	{
		SetItemExpansionRecursive(Child, bShouldBeExpanded);
	}
}

void SReadOnlyHierarchyView::SetRawFilterText(const FText& Text)
{
	FilterHandler->SetIsEnabled(!Text.IsEmpty());
	SearchFilter->SetRawFilterText(Text);
	FilterHandler->RefreshAndFilterTree();
}

FText SReadOnlyHierarchyView::GetItemText(TSharedPtr<FItem> Item) const
{
	if (Item)
	{
		// The item is a Widget child in the tree
		if (const UWidget* Widget = Item->Widget.Get())
		{
			return Widget->GetLabelTextWithMetadata();
		}

		// The widget is the Root WidgetBlueprint
		if (const UWidgetBlueprint* Blueprint = Item->WidgetBlueprint.Get())
		{
			// If using self as the Selection mode, we use Self as the name, otherwise it's the WidgetBlueprint Name
			FText RootWidgetName = RootSelectionMode == ERootSelectionMode::Self ? FText(LOCTEXT("SelfText", "Self")) : FText::FromString(Blueprint->GetName());

			static const FText RootWidgetFormat = LOCTEXT("WidgetNameFormat", "[{0}]");
			return FText::Format(RootWidgetFormat, RootWidgetName);
		}
		const FName NamedSlotName = Item->InheritedNamedSlotName;
		if (!NamedSlotName.IsNone())
		{
			return FText::FromName(NamedSlotName);
		}
	}
	return FText::GetEmpty();
}

const FSlateBrush* SReadOnlyHierarchyView::GetIconBrush(TSharedPtr<FItem> Item) const
{
	if (const UWidget* Widget = Item->Widget.Get())
	{
		return FSlateIconFinder::FindIconBrushForClass(Widget->GetClass());
	}
	return nullptr;
}

FText SReadOnlyHierarchyView::GetIconToolTipText(TSharedPtr<FItem> Item) const
{
	if (const UWidget* Widget = Item->Widget.Get())
	{
		UClass* WidgetClass = Widget->GetClass();
		if (WidgetClass->IsChildOf(UUserWidget::StaticClass()) && WidgetClass->ClassGeneratedBy)
		{
			const FString& Description = Cast<UWidgetBlueprint>(WidgetClass->ClassGeneratedBy)->BlueprintDescription;
			if (Description.Len() > 0)
			{
				return FText::FromString(Description);
			}
		}

		return WidgetClass->GetToolTipText();
	}

	return FText::GetEmpty();
}

FText SReadOnlyHierarchyView::GetWidgetToolTipText(TSharedPtr<FItem> Item) const
{
	if (const UWidget* Widget = Item->Widget.Get())
	{
		if (!Widget->IsGeneratedName())
		{
			return FText::FromString(TEXT("[") + Widget->GetClass()->GetDisplayNameText().ToString() + TEXT("]"));
		}
	}

	return FText::GetEmpty();
}

TSharedRef<ITableRow> SReadOnlyHierarchyView::GenerateRow(TSharedPtr<FItem> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	bool bIsEnabled = true; 
	// When Root Selection Mode is Disabled we must check if this is a WidgetBlueprint
	if (RootSelectionMode == ERootSelectionMode::Disabled)
	{
		// If the Widget Blueprint is set, it's the root widget we disable it.
		if (const UWidgetBlueprint* Blueprint = Item->WidgetBlueprint.Get())
		{
			bIsEnabled = false;
		}
	}

	const FName NamedSlotName = Item->InheritedNamedSlotName;
	if (!NamedSlotName.IsNone() && InheritedNamedSlotSelectionMode == EInheritedNamedSlotSelectionMode::Disabled)
	{
		bIsEnabled = false;
	}

	return SNew(STableRow<TSharedPtr<FItem>>, OwnerTable)
		   .IsEnabled(bIsEnabled)
		[
			SNew(SHorizontalBox)
			// Widget icon
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4, 3, 0, 3)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(GetIconBrush(Item))
				.ToolTipText(this, &SReadOnlyHierarchyView::GetIconToolTipText, Item)
			]
			// Name of the widget
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(Item->Widget.Get() == nullptr ? FCoreStyle::GetDefaultFontStyle("Bold", 10) : FCoreStyle::Get().GetFontStyle("NormalFont"))
				.Text(this, &SReadOnlyHierarchyView::GetItemText, Item)
				.ToolTipText(this, &SReadOnlyHierarchyView::GetWidgetToolTipText, Item)
				.HighlightText_Lambda([this]() { return SearchFilter->GetRawFilterText(); })
			]
		];
}

void SReadOnlyHierarchyView::GetFilterStringsForItem(TSharedPtr<FItem> Item, TArray<FString>& OutStrings) const
{
	if (const UWidget* Widget = Item->Widget.Get())
	{
		OutStrings.Add(Widget->GetName());
		OutStrings.Add(Widget->GetLabelTextWithMetadata().ToString());
	}
	else if (const UWidgetBlueprint* Blueprint = Item->WidgetBlueprint.Get())
	{
		OutStrings.Add(Blueprint->GetName());
	}
	else
	{
		const FName NamedSlotName = Item->InheritedNamedSlotName;
		if (!NamedSlotName.IsNone())
		{
			OutStrings.Add(NamedSlotName.ToString());
		}
	}
}

void SReadOnlyHierarchyView::SetSelectedWidget(FName WidgetName)
{
	if (TSharedPtr<FItem> Found = FindItem(RootWidgets, WidgetName))
	{
		TreeView->SetSelection(Found);
	}
	else
	{
		TreeView->ClearSelection();
	}
}

TArray<FName> SReadOnlyHierarchyView::GetSelectedWidgets() const
{
	TArray<TSharedPtr<FItem>> SelectedItems = TreeView->GetSelectedItems();

	TArray<FName> SelectedNames;
	Algo::TransformIf(SelectedItems, SelectedNames, 
		[this](const TSharedPtr<FItem>& Item)
		{
			return !GetItemName(Item).IsNone();
		},
		[this](const TSharedPtr<FItem>& Item)
		{
			return GetItemName(Item);
		});

	return SelectedNames;
}

void SReadOnlyHierarchyView::ClearSelection()
{
	TreeView->ClearSelection();
}

void SReadOnlyHierarchyView::GetItemChildren(TSharedPtr<FItem> Item, TArray<TSharedPtr<FItem>>& OutChildren) const
{
	OutChildren.Append(Item->Children);
}

void SReadOnlyHierarchyView::BuildWidgetChildren(const UWidget* Widget, TSharedPtr<FItem> Parent)
{ 
	// if we don't have a ShowOnly filter, or this widget is in it, create the item
	if (ShowOnly.Num() == 0 || ShowOnly.Contains(Widget->GetFName()))
	{
		TSharedRef<FItem> WidgetItem = MakeShared<FItem>(Widget);
		if (Parent.IsValid())
		{
			Parent->Children.Add(WidgetItem);
		}
		else
		{
			RootWidgets.Add(WidgetItem);
		}

		Parent = WidgetItem;
	}

	// Search for content within a named slot that we need to recurse into.
	// We still want to iterate over its children even if we didn't create an item for this widget.
	if (const INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(Widget))
	{
		TArray<FName> SlotNames;
		NamedSlotHost->GetSlotNames(SlotNames);

		for (FName SlotName : SlotNames)
		{
			if (const UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName))
			{
				BuildWidgetChildren(SlotContent, Parent);
			}
		}
	}

	// If the current widget is a panel, then we want to iterate through its children.
	// We still want to iterate over its children even if we didn't create an item for this widget.
	if (const UPanelWidget* PanelWidget = Cast<UPanelWidget>(Widget))
	{
		for (const UWidget* Child : PanelWidget->GetAllChildren())
		{
			BuildWidgetChildren(Child, Parent);
		}
	}
}

void SReadOnlyHierarchyView::RebuildTree()
{
	const UWidgetBlueprint* WidgetBP = WidgetBlueprint.Get();
	if (WidgetBP == nullptr)
	{
		return;
	}
	
	// if we don't have a ShowOnly filter, or this widget blueprint is in it, create the root item
	TSharedPtr<FItem> RootItem;
	if (ShowOnly.Num() == 0 || ShowOnly.Contains(WidgetBP->GetFName()))
	{
		RootItem = MakeShared<FItem>(WidgetBP);
		RootWidgets.Add(RootItem);
	}

	if (const UWidget* RootWidget = WidgetBP->WidgetTree->RootWidget.Get())
	{
		BuildWidgetChildren(RootWidget, RootItem);
	}

	// Also show content placed in named slots inherited from a parent blueprint,
	// regardless of whether this blueprint has its own root widget.
	for (const FName& SlotName : WidgetBP->GetInheritedAvailableNamedSlots())
	{
		TSharedRef<FItem> NamedSlotItem = MakeShared<FItem>(SlotName);
		if (RootItem.IsValid())
		{
			RootItem->Children.Add(NamedSlotItem);
		}
		else
		{
			RootWidgets.Add(NamedSlotItem);
		}

		if (const UWidget* SlotContent = WidgetBP->WidgetTree->GetContentForSlot(SlotName))
		{
			BuildWidgetChildren(SlotContent, NamedSlotItem);
		}
	}

	if (bExpandAll)
	{
		ExpandAll();
	}
}

void SReadOnlyHierarchyView::ExpandAll()
{
	for (const TSharedPtr<FItem>& Item : FilteredRootWidgets)
	{
		SetItemExpansionRecursive(Item, true);
	}
}

TSharedPtr<SReadOnlyHierarchyView::FItem> SReadOnlyHierarchyView::FindItem(const TArray<TSharedPtr<SReadOnlyHierarchyView::FItem>>& RootItems, FName Name) const
{
	TArray<TSharedPtr<FItem>> Items;
	Items.Append(RootItems);

	for (int32 Idx = 0; Idx < Items.Num(); ++Idx)
	{
		TSharedPtr<FItem> Item = Items[Idx];

		if (const UWidgetBlueprint* WidgetBP = Item->WidgetBlueprint.Get())
		{
			if (WidgetBP->GetFName() == Name)
			{
				return Item;
			}
		}
		else if (const UWidget* Widget = Item->Widget.Get())
		{
			if (Widget->GetFName() == Name)
			{
				return Item;
			}
		}
		else 
		{
			const FName NamedSlotName = Item->InheritedNamedSlotName;
			if (!NamedSlotName.IsNone() && NamedSlotName == Name)
			{
				return Item;
			}
		}

		GetItemChildren(Item, Items);
	}

	return TSharedPtr<FItem>();
}

#undef LOCTEXT_NAMESPACE
