// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_WidgetAnimationEvent.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "WidgetBlueprint.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/WidgetAnimationDelegateBinding.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_WidgetAnimationEvent)

#define LOCTEXT_NAMESPACE "UK2Node_WidgetAnimationEvent"

UK2Node_WidgetAnimationEvent::UK2Node_WidgetAnimationEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bInternalEvent = true;

	EventReference.SetExternalDelegateMember(FName(TEXT("OnWidgetAnimationPlaybackStatusChanged__DelegateSignature")));
}

void UK2Node_WidgetAnimationEvent::Initialize(const UWidgetBlueprint* InSourceBlueprint, UWidgetAnimation* InAnimation, EWidgetAnimationEvent InAction)
{
	SourceWidgetBlueprint = InSourceBlueprint;

	AnimationPropertyName = InAnimation->GetMovieScene()->GetFName();
	Action = InAction;

	MarkDirty();
}

void UK2Node_WidgetAnimationEvent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	MarkDirty();
}

void UK2Node_WidgetAnimationEvent::MarkDirty()
{
	CachedNodeTitle.MarkDirty();

	CustomFunctionName = FName(*FString::Printf(TEXT("WidgetAnimationEvt_%s_%s"), *AnimationPropertyName.ToString(), *GetName()));
}

FText UK2Node_WidgetAnimationEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(CachedNodeTitle.IsOutOfDate(this))
	{
		FText ActionText = UEnum::GetDisplayValueAsText(TEXT("UMG.EWidgetAnimationEvent"), Action);

		FFormatNamedArguments Args;
		Args.Add(TEXT("ActionName"), ActionText);
		Args.Add(TEXT("AnimationName"), FText::FromName(AnimationPropertyName));

		if (UserTag == NAME_None)
		{
			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("AnimationBoundEvent_Title", "Animation {ActionName} ({AnimationName})"), Args), this);
		}
		else
		{
			Args.Add(TEXT("UserTag"), FText::FromName(UserTag));

			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("AnimationBoundEventWithName_Title", "Animation {ActionName} Tag:{UserTag} ({AnimationName})"), Args), this);
		}
	}

	return CachedNodeTitle;
}

FText UK2Node_WidgetAnimationEvent::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(LOCTEXT("AnimationBoundEvent_Tooltip", "Called when the corresponding animation event fires.  Can also have a tag configured to only be called under certain conditions."), this);
	}
	return CachedTooltip;
}

void UK2Node_WidgetAnimationEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
}

UClass* UK2Node_WidgetAnimationEvent::GetDynamicBindingClass() const
{
	return UWidgetAnimationDelegateBinding::StaticClass();
}

void UK2Node_WidgetAnimationEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UWidgetAnimationDelegateBinding* WidgetAnimationBindingObject = CastChecked<UWidgetAnimationDelegateBinding>(BindingObject);

	FBlueprintWidgetAnimationDelegateBinding Binding;
	Binding.Action = Action;
	Binding.AnimationToBind = AnimationPropertyName;
	Binding.FunctionNameToBind = CustomFunctionName;
	Binding.UserTag = UserTag;
	
	WidgetAnimationBindingObject->WidgetAnimationDelegateBindings.Add(Binding);
}

void UK2Node_WidgetAnimationEvent::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	if (InOldVarName == AnimationPropertyName && InVariableClass->IsChildOf(InBlueprint->GeneratedClass))
	{
		Modify();
		AnimationPropertyName = InNewVarName;
	}
}

bool UK2Node_WidgetAnimationEvent::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bIsCompatible = false;
	
	// Find the Blueprint that owns the target graph
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	if (Blueprint != nullptr)
	{
		bIsCompatible = Blueprint->IsA<UWidgetBlueprint>();
	}

	UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(TargetGraph->GetSchema());
	bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(TargetGraph) : false;
	bIsCompatible &= !bIsConstructionScript;

	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

bool UK2Node_WidgetAnimationEvent::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	if (Filter.Context.Graphs.Num() > 0)
	{
		if (Filter.Context.Blueprints.Num() > 0)
		{
			UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(Filter.Context.Blueprints[0]);
			check(WidgetBlueprint);

			if (SourceWidgetBlueprint == WidgetBlueprint)
			{
				return false;
			}
		}
	}

	return true;
}

void UK2Node_WidgetAnimationEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, const UWidgetBlueprint* InSourceBlueprint, UWidgetAnimation* Key, EWidgetAnimationEvent InAction)
	{
		UK2Node_WidgetAnimationEvent* InputNode = CastChecked<UK2Node_WidgetAnimationEvent>(NewNode);
		InputNode->Initialize(InSourceBlueprint, Key, InAction);
	};

	const UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(ActionRegistrar.GetActionKeyFilter());
	if (WidgetBlueprint && ActionRegistrar.IsOpenForRegistration(WidgetBlueprint))
	{
		for (UWidgetAnimation* WidgetAnimation : WidgetBlueprint->Animations)
		{
			auto Spawner = [&](EWidgetAnimationEvent InAction) {
				UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
				check(NodeSpawner != nullptr);

				NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, WidgetBlueprint, WidgetAnimation, InAction);
				ActionRegistrar.AddBlueprintAction(WidgetBlueprint, NodeSpawner);
			};

			Spawner(EWidgetAnimationEvent::Started);
			Spawner(EWidgetAnimationEvent::Finished);
		}
	}
}

FText UK2Node_WidgetAnimationEvent::GetMenuCategory() const
{
	static TMap<FName, FNodeTextCache> CachedCategories;

	const FName KeyCategory = TEXT("WidgetAnimations");
	const FText SubCategoryDisplayName = LOCTEXT("EventsCategory", "Widget Animation Events");
	FNodeTextCache& NodeTextCache = CachedCategories.FindOrAdd(KeyCategory);

	if (NodeTextCache.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		NodeTextCache.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, SubCategoryDisplayName), this);
	}

	return NodeTextCache;
}

FBlueprintNodeSignature UK2Node_WidgetAnimationEvent::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(AnimationPropertyName.ToString());
	NodeSignature.AddKeyValue(UEnum::GetValueAsString(TEXT("UMG.EWidgetAnimationEvent"), Action));
	NodeSignature.AddKeyValue(UserTag.ToString());

	return NodeSignature;
}

#undef LOCTEXT_NAMESPACE
