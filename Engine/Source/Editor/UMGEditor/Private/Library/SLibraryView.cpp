// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/SLibraryView.h"
#include "Library/SLibraryViewModel.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "WidgetBlueprint.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "SAssetView.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "DragDrop/WidgetTemplateDragDropOp.h"

#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateBlueprintClass.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"

#include "Settings/ContentBrowserSettings.h"
#include "IContentBrowserSingleton.h"
#include "WidgetBlueprintEditorUtils.h"

#include "Library/SLibraryView.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::Editor::ContentBrowser
{
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = false;
		UE_CALL_ONCE([&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				bIsNewStyleEnabled = CVar->GetBool();
			}
		});

		return bIsNewStyleEnabled;
	}
}

FText SLibraryViewItem::GetFavoriteToggleToolTipText() const
{
	if (GetFavoritedState() == ECheckBoxState::Checked)
	{
		return LOCTEXT("Unfavorite", "Click to remove this widget from your favorites.");
	}
	return LOCTEXT("Favorite", "Click to add this widget to your favorites.");
}

ECheckBoxState SLibraryViewItem::GetFavoritedState() const
{
	if (WidgetViewModel->IsFavorite())
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

void SLibraryViewItem::OnFavoriteToggled(ECheckBoxState InNewState)
{
	if (InNewState == ECheckBoxState::Checked)
	{
		//Add to favorites
		WidgetViewModel->AddToFavorites();
	}
	else
	{
		//Remove from favorites
		WidgetViewModel->RemoveFromFavorites();
	}
}

EVisibility SLibraryViewItem::GetFavoritedStateVisibility() const
{
	return GetFavoritedState() == ECheckBoxState::Checked || IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
}

void SLibraryViewItem::Construct(const FArguments& InArgs, TSharedPtr<FWidgetTemplateViewModel> InWidgetViewModel)
{
	WidgetViewModel = InWidgetViewModel;

	ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTip(WidgetViewModel->Template->GetToolTip())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[	
				SNew(SCheckBox)
				.ToolTipText(this, &SLibraryViewItem::GetFavoriteToggleToolTipText)
				.IsChecked(this, &SLibraryViewItem::GetFavoritedState)
				.OnCheckStateChanged(this, &SLibraryViewItem::OnFavoriteToggled)
				.Style(FAppStyle::Get(), "UMGEditor.Library.FavoriteToggleStyle")
				.Visibility(this, &SLibraryViewItem::GetFavoritedStateVisibility)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(WidgetViewModel->Template->GetIcon())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InWidgetViewModel->GetName())
				.HighlightText(InArgs._HighlightText)
			]
		];
}

TSharedRef<SWidget> SLibraryView::ConstructViewOptions()
{
	FMenuBuilder ViewOptions(true, nullptr);

	{
		ViewOptions.BeginSection("AssetViewType", LOCTEXT("ViewTypeHeading", "View Type"));
		ViewOptions.AddMenuEntry(
			LOCTEXT("TileViewOption", "Tiles"),
			LOCTEXT("TileViewOptionToolTip", "View assets as tiles in a grid."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLibraryView::SetCurrentViewTypeFromMenu, EAssetViewType::Tile),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLibraryView::IsCurrentViewType, EAssetViewType::Tile)
			),
			"TileView",
			EUserInterfaceActionType::RadioButton
		);

		ViewOptions.AddMenuEntry(
			LOCTEXT("ListViewOption", "List"),
			LOCTEXT("ListViewOptionToolTip", "View assets in a list with thumbnails."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLibraryView::SetCurrentViewTypeFromMenu, EAssetViewType::List),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLibraryView::IsCurrentViewType, EAssetViewType::List)
			),
			"ListView",
			EUserInterfaceActionType::RadioButton
		);
		ViewOptions.EndSection();
	}

	{
		ViewOptions.BeginSection("AssetThumbnails", LOCTEXT("ThumbnailsHeading", "Thumbnails"));

		auto CreateThumbnailSizeSubMenu = [this](FMenuBuilder& SubMenu)
		{
			for (int32 EnumValue = (int32)EThumbnailSize::Tiny; EnumValue < (int32)EThumbnailSize::MAX; ++EnumValue)
			{
				if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
				{
					if ((EThumbnailSize)EnumValue == EThumbnailSize::XLarge)
					{
						continue;
					}
				}

				SubMenu.AddMenuEntry(
					SAssetView::ThumbnailSizeToDisplayName((EThumbnailSize)EnumValue),
					FText::GetEmpty(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SLibraryView::OnThumbnailSizeChanged, (EThumbnailSize)EnumValue),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &SLibraryView::IsThumbnailSizeChecked, (EThumbnailSize)EnumValue)
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		};

		ViewOptions.AddSubMenu(
			LOCTEXT("ThumbnailSize", "Thumbnail Size"),
			LOCTEXT("ThumbnailSizeToolTip", "Adjust the size of thumbnails."),
			FNewMenuDelegate::CreateLambda(CreateThumbnailSizeSubMenu)
		);
		ViewOptions.EndSection();
	}

	return ViewOptions.MakeWidget();
}

void SLibraryView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;

	UBlueprint* BP = InBlueprintEditor->GetBlueprintObj();
	LibraryViewModel = InBlueprintEditor->GetLibraryViewModel();

	// Register to the update of the viewmodel.
	LibraryViewModel->OnUpdating.AddRaw(this, &SLibraryView::OnViewModelUpdating);
	LibraryViewModel->OnUpdated.AddRaw(this, &SLibraryView::OnViewModelUpdated);

	WidgetFilter = MakeShared<WidgetViewModelDelegateFilter>(
		WidgetViewModelDelegateFilter::FPredicate::CreateSP(this, &SLibraryView::HandleFilterWidgetView));

	FilterHandler = MakeShared<LibraryFilterHandler>();
	FilterHandler->SetFilter(WidgetFilter.Get());
	FilterHandler->SetRootItems(&(LibraryViewModel->GetWidgetViewModels()), &TreeWidgetViewModels);
	FilterHandler->SetGetChildrenDelegate(LibraryFilterHandler::FOnGetChildren::CreateRaw(this, &SLibraryView::OnGetChildren));

	SAssignNew(WidgetTemplatesView, STreeView< TSharedPtr<FWidgetViewModel> >)
		.SelectionMode(ESelectionMode::SingleToggle)
		.OnGenerateRow(this, &SLibraryView::OnGenerateWidgetTemplateLibrary)
		.OnGetChildren(FilterHandler.ToSharedRef(), &LibraryFilterHandler::OnGetFilteredChildren)
		.OnMouseButtonClick(this, &SLibraryView::WidgetLibrary_OnClick)
		.TreeItemsSource(&TreeWidgetViewModels);

	FilterHandler->SetTreeView(WidgetTemplatesView.Get());
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(4.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("SearchTemplates", "Search Library"))
				.OnTextChanged(this, &SLibraryView::OnSearchChanged)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SComboButton)
				.ContentPadding(0.0f)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.HasDownArrow(false)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
				]
				.MenuContent()
				[
					ConstructViewOptions()
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(0.0f)
			[
				SNew(SScrollBox)
				.ScrollBarPadding(FMargin(2.0f, 0.0f))
				+SScrollBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.Padding(0.0f)
					[
						WidgetTemplatesView.ToSharedRef()
					]
				]
			]
		]
	];

	bRefreshRequested = true;
	LastViewType = EAssetViewType::Type::Tile;
	LastThumbnailSize = EThumbnailSize::Small;

	LibraryViewModel->Update();
	LoadItemExpansion();
}

SLibraryView::~SLibraryView()
{
	// Unregister to the update of the viewmodel.
	LibraryViewModel->OnUpdating.RemoveAll(this);
	LibraryViewModel->OnUpdated.RemoveAll(this);

	// If the filter is enabled, disable it before saving the expanded items since
	// filtering expands all items by default.
	if (FilterHandler->GetIsEnabled())
	{
		FilterHandler->SetIsEnabled(false);
		FilterHandler->RefreshAndFilterTree();
	}

	SaveItemExpansion();
}

void SLibraryView::OnSearchChanged(const FText& InFilterText)
{
	bRefreshRequested = true;
	FilterHandler->SetIsEnabled(!InFilterText.IsEmpty());
	LibraryViewModel->SetSearchText(InFilterText);
}

void SLibraryView::LoadItemExpansion()
{
	// Restore the expansion state of the widget groups.
	for ( TSharedPtr<FWidgetViewModel>& ViewModel : LibraryViewModel->GetWidgetViewModels())
	{
		bool IsExpanded;
		if ( GConfig->GetBool(TEXT("WidgetTemplatesExpanded"), *ViewModel->GetName().ToString(), IsExpanded, GEditorPerProjectIni) && IsExpanded )
		{
			WidgetTemplatesView->SetItemExpansion(ViewModel, true);
		}
	}
}

void SLibraryView::SaveItemExpansion()
{
	// Restore the expansion state of the widget groups.
	for ( TSharedPtr<FWidgetViewModel>& ViewModel : LibraryViewModel->GetWidgetViewModels() )
	{
		const bool IsExpanded = WidgetTemplatesView->IsItemExpanded(ViewModel);
		GConfig->SetBool(TEXT("WidgetTemplatesExpanded"), *ViewModel->GetName().ToString(), IsExpanded, GEditorPerProjectIni);
	}
}

bool SLibraryView::HandleFilterWidgetView(TSharedPtr<FWidgetViewModel> WidgetViewModel)
{
	return WidgetViewModel->HasFilteredChildTemplates();
}

void SLibraryView::SetCurrentViewTypeFromMenu(EAssetViewType::Type ViewType)
{
	LastViewType = ViewType;

	for (TSharedPtr<FWidgetViewModel>& ViewModel : LibraryViewModel->GetWidgetTemplateListViewModels())
	{
		TSharedPtr<FWidgetTemplateListViewModel> TemplateListViewModel = StaticCastSharedPtr<FWidgetTemplateListViewModel>(ViewModel);
		TemplateListViewModel->SetViewType(ViewType);
	}
}

bool SLibraryView::IsCurrentViewType(EAssetViewType::Type ViewType)
{
	return LastViewType == ViewType;
}

void SLibraryView::OnThumbnailSizeChanged(EThumbnailSize ThumbnailSize)
{
	LastThumbnailSize = ThumbnailSize;

	for (TSharedPtr<FWidgetViewModel>& ViewModel : LibraryViewModel->GetWidgetTemplateListViewModels())
	{
		TSharedPtr<FWidgetTemplateListViewModel> TemplateListViewModel = StaticCastSharedPtr<FWidgetTemplateListViewModel>(ViewModel);
		TemplateListViewModel->SetThumbnailSize(ThumbnailSize);
	}
}

bool SLibraryView::IsThumbnailSizeChecked(EThumbnailSize ThumbnailSize)
{
	return LastThumbnailSize == ThumbnailSize;
}

void SLibraryView::OnGetChildren(TSharedPtr<FWidgetViewModel> Item, TArray< TSharedPtr<FWidgetViewModel> >& Children)
{
	return Item->GetChildren(Children);
}

TSharedRef<ITableRow> SLibraryView::OnGenerateWidgetTemplateLibrary(TSharedPtr<FWidgetViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return Item->BuildRow(OwnerTable);
}

void SLibraryView::OnViewModelUpdating()
{
	// Save the old expanded items temporarily
	WidgetTemplatesView->GetExpandedItems(ExpandedItems);
}

void SLibraryView::OnViewModelUpdated()
{
	bRefreshRequested = true;

	// Restore the expansion state
	for (TSharedPtr<FWidgetViewModel>& ExpandedItem : ExpandedItems)
	{
		for (TSharedPtr<FWidgetViewModel>& ViewModel : LibraryViewModel->GetWidgetViewModels())
		{
			if (ViewModel->GetName().EqualTo(ExpandedItem->GetName()) || ViewModel->ShouldForceExpansion())
			{
				WidgetTemplatesView->SetItemExpansion(ViewModel, true);
			}
		}
	}
	ExpandedItems.Reset();
}

void SLibraryView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRefreshRequested)
	{
		bRefreshRequested = false;
		FilterHandler->RefreshAndFilterTree();
	}
}

void SLibraryView::WidgetLibrary_OnClick(TSharedPtr<FWidgetViewModel> SelectedItem)
{
	if (!SelectedItem.IsValid())
	{
		return;
	}

	// If it's a category, toggle it
	if (SelectedItem->IsCategory())
	{
		if (TSharedPtr<FWidgetHeaderViewModel> CategoryHeader = StaticCastSharedPtr<FWidgetHeaderViewModel>(SelectedItem))
		{
			if (TSharedPtr<ITableRow> TableRow = WidgetTemplatesView->WidgetFromItem(CategoryHeader))
			{
				TableRow->ToggleExpansion();
			}
		}
		return;
	}
}

#undef LOCTEXT_NAMESPACE
