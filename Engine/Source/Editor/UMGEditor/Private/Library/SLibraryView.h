// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "WidgetBlueprintEditor.h"
#include "Misc/DelegateFilter.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Views/TreeFilterHandler.h"
#include "IContentBrowserSingleton.h"

class FWidgetTemplate;
class UWidgetBlueprint;
class SLibraryView;
class FLibraryViewModel;
class FWidgetViewModel;
class FWidgetTemplateViewModel;

/** Widget used to show a single row of the Library and Library favorite panel. */
class SLibraryViewItem : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLibraryViewItem) {}

	/** The current text to highlight */
	SLATE_ATTRIBUTE(FText, HighlightText)

	SLATE_END_ARGS()

	/**
	* Constructs this widget
	*
	* @param InArgs    Declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetTemplateViewModel> InWidgetViewModel);

private:
	/**
	 * Retrieves tooltip that describes the current favorited state 
	 *
	 * @return Text describing what this toggle will do when you click on it.
	 */
	FText GetFavoriteToggleToolTipText() const;

	/**
	 * Checks on the associated action's favorite state, and returns a
	 * corresponding checkbox state to match.
	 *
	 * @return ECheckBoxState::Checked if the associated action is already favorited, ECheckBoxState::Unchecked if not.
	 */
	ECheckBoxState GetFavoritedState() const;

	/**
	 * Triggers when the user clicks this toggle, adds or removes the associated
	 * action to the user's favorites.
	 *
	 * @param  InNewState	The new state that the user set the checkbox to.
	 */
	void OnFavoriteToggled(ECheckBoxState InNewState);

	EVisibility GetFavoritedStateVisibility() const;

private:
	TSharedPtr<FWidgetTemplateViewModel> WidgetViewModel;
};

/** */
class SLibraryView : public SCompoundWidget
{
public:
	typedef TDelegateFilter<TSharedPtr<FWidgetViewModel>> WidgetViewModelDelegateFilter;

public:
	SLATE_BEGIN_ARGS( SLibraryView ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~SLibraryView();
	
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** On item clicked in palette view */
	void WidgetLibrary_OnClick(TSharedPtr<FWidgetViewModel> SelectedItem);

private:

	/** Builds settings menu. */
	TSharedRef<SWidget> ConstructViewOptions();

	void OnGetChildren(TSharedPtr<FWidgetViewModel> Item, TArray< TSharedPtr<FWidgetViewModel> >& Children);
	TSharedRef<ITableRow> OnGenerateWidgetTemplateLibrary(TSharedPtr<FWidgetViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when the filter text is changed. */
	void OnSearchChanged(const FText& InFilterText);

	void OnViewModelUpdating();
	void OnViewModelUpdated();

private:
	/** Load the expansion state for the TreeView */
	void LoadItemExpansion();

	/** Save the expansion state for the TreeView */
	void SaveItemExpansion();

	/** Callback to determine if a particular library view passes filtering */
	bool HandleFilterWidgetView(TSharedPtr<FWidgetViewModel> WidgetViewModel);

	/** Callback to set library asset view type */
	void SetCurrentViewTypeFromMenu(EAssetViewType::Type ViewType);

	/** Callback to determine library asset view type */
	bool IsCurrentViewType(EAssetViewType::Type ViewType);

	/** Callback to set library asset thumbnail size */
	void OnThumbnailSizeChanged(EThumbnailSize InThumbnailSize);

	/** Callback to determine library asset thumbnail size */
	bool IsThumbnailSizeChecked(EThumbnailSize InThumbnailSize);

	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<FLibraryViewModel> LibraryViewModel;

	/** Handles filtering the Library based on an IFilter. */
	typedef TreeFilterHandler<TSharedPtr<FWidgetViewModel>> LibraryFilterHandler;
	TSharedPtr<LibraryFilterHandler> FilterHandler;

	typedef TArray<TSharedPtr<FWidgetViewModel>> ViewModelsArray;
	/** The root view models which are actually displayed by the TreeView which will be managed by the TreeFilterHandler. */
	ViewModelsArray TreeWidgetViewModels;

	TSharedPtr< STreeView< TSharedPtr<FWidgetViewModel> > > WidgetTemplatesView;

	/** The search box used to update the filter text */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** The filter instance which is used by the TreeFilterHandler to filter the TreeView. */
	TSharedPtr<WidgetViewModelDelegateFilter> WidgetFilter;

	/** Expended Items in the Tree view */
	TSet<TSharedPtr<FWidgetViewModel>> ExpandedItems;

	/** Set to true to force a refresh of the treeview */
	bool bRefreshRequested;

	/** Are editor widgets supported. */
	bool bAllowEditorWidget;

	/** Last view type set by user */
	EAssetViewType::Type LastViewType;

	/** Last view type set by user */
	EThumbnailSize LastThumbnailSize;

};
