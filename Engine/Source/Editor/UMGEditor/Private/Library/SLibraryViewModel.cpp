// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/SLibraryViewModel.h"
#include "Library/SLibraryView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "WidgetBlueprint.h"
#include "Editor.h"
#include "FrontendFilters.h"
#include "SAssetView.h"
#include "AssetTextFilter.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "DragDrop/WidgetTemplateDragDropOp.h"

#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateBlueprintClass.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "WidgetBlueprintEditorUtils.h"

#include "IContentBrowserDataModule.h"
#include "Settings/ContentBrowserSettings.h"
#include "Settings/WidgetDesignerSettings.h"
#include "WidgetPaletteFavorites.h"
#include "Library/SLibraryViewModel.h"

#define LOCTEXT_NAMESPACE "UMG"

void FLibraryViewModel::BuildWidgetTemplateCategory(FString& Category, TArray<TSharedPtr<FWidgetTemplate>>& Templates, TArray<FString>& FavoritesList)
{
	//This function do not handle Favorites. So we empty the Favorite Array. See FLibraryViewModel::BuildWidgetTemplateCategory to implement favorite cleaning.
	FavoritesList.Empty();

	TSharedPtr<FWidgetHeaderViewModel> Header = MakeShareable(new FWidgetHeaderViewModel());
	Header->GroupName = FText::FromString(Category);

	TSharedPtr<FWidgetTemplateListViewModel> TemplateViewModel = MakeShared<FWidgetTemplateListViewModel>();
	TemplateViewModel->ConstructListView(Templates);
	Header->Children.Add(TemplateViewModel);
	WidgetTemplateListViewModels.Add(TemplateViewModel);

	Header->Children.Sort([](const TSharedPtr<FWidgetViewModel>& L, const TSharedPtr<FWidgetViewModel>& R) { return R->GetName().CompareTo(L->GetName()) > 0; });

	AddHeader(Header);
}

void FLibraryViewModel::SetSearchText(const FText& InSearchText)
{
	FWidgetCatalogViewModel::SetSearchText(InSearchText);

	for (TSharedPtr<FWidgetViewModel>& WidgetTemplateListViewModel : WidgetTemplateListViewModels)
	{
		TSharedPtr<FWidgetTemplateListViewModel> WidgetLibrary = StaticCastSharedPtr<FWidgetTemplateListViewModel>(WidgetTemplateListViewModel);
		WidgetLibrary->SetSearchText(InSearchText);
	}
}

void FLibraryViewModel::BuildWidgetList()
{
	WidgetTemplateListViewModels.Reset();
	FWidgetCatalogViewModel::BuildWidgetList();
}

FWidgetTemplateListViewModel::FWidgetTemplateListViewModel()
{
}

void FWidgetTemplateListViewModel::ConstructListView(TArray<TSharedPtr<FWidgetTemplate>> InTemplates)
{
	Templates = InTemplates;

	if (!TemplatesFilter)
	{
		NumAssets = 0;
		NumClasses = 0;

		// Generate filter text
		bool bHasFilters = false;
		TStringBuilder<2048> FilterString;
		for (TSharedPtr<FWidgetTemplate> Template : Templates)
		{
			if (TSharedPtr<FWidgetTemplateClass> TemplateClass = StaticCastSharedPtr<FWidgetTemplateClass>(Template))
			{
				FString TemplateString;
				FString TemplatePath;

				if (TemplateClass->GetWidgetAssetData().IsValid())
				{
					TemplateString = TemplateClass->GetWidgetAssetData().AssetName.ToString();
					NumAssets++;
				}
				else if (UClass* WidgetClass = TemplateClass->GetWidgetClass().Get())
				{
					TemplateString = WidgetClass->GetFName().ToString();
					NumClasses++;
				}

				if (!TemplateString.IsEmpty())
				{
					TemplateString.RemoveFromEnd(TEXT("_C"));
					FilterString += bHasFilters ? "|+" : "+";
					FilterString += TemplateString;
					bHasFilters = false;
				}
			}
		}

		// Provide a dummy invalid virtual path to make sure nothing tries to enumerate root "/"
		LibraryContentSources = MakeShared<FAssetViewContentSources>(FName(TEXT("/UMGWidgetTemplateListViewModel")));
		// Disable any enumerate of virtual path folders
		LibraryContentSources->bIncludeVirtualPaths = false;
		// Supply a custom list of source items to display
		LibraryContentSources->OnEnumerateCustomSourceItemDatas.BindSP(this, &FWidgetTemplateListViewModel::EnumerateCustomSourceItemDatas);

		CachedLowercaseWidgetFilter = FString(FilterString.ToString()).ToLower();

		SearchFilter = MakeShared<FAssetTextFilter>();

		TemplatesFilter = MakeShared<FAssetFilterCollectionType>();
	}

	if (!AssetViewPtr)
	{
		AssetViewPtr = SNew(SAssetView)
			.InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAssets | EContentBrowserItemCategoryFilter::IncludeClasses)
			.InitialContentSources(*LibraryContentSources)
			.InitialThumbnailSize(EThumbnailSize::Small)
			.FrontendFilters(TemplatesFilter)
			.TextFilter(SearchFilter)
			.ForceShowEngineContent(true)
			.ForceShowPluginContent(true)
			.ForceHideScrollbar(true)
			.ShowTypeInTileView(false)
			.ShowViewOptions(false)
			.HighlightedText(this, &FWidgetTemplateListViewModel::GetSearchText)
			;
	}

	AssetViewPtr->RequestSlowFullListRefresh();
}

bool FWidgetTemplateListViewModel::EnumerateCustomSourceItemDatas(TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	SourceItemPaths.Reset();
	SourceItemPaths.Reserve(NumAssets);

	TArray<UObject*> ClassObjects;
	ClassObjects.Reserve(NumClasses);

	for (TSharedPtr<FWidgetTemplate> Template : Templates)
	{
		if (TSharedPtr<FWidgetTemplateClass> TemplateClass = StaticCastSharedPtr<FWidgetTemplateClass>(Template))
		{
			if (TemplateClass->GetWidgetAssetData().IsValid())
			{
				SourceItemPaths.Add(FContentBrowserItemPath(TemplateClass->GetWidgetAssetData().PackageName, EContentBrowserPathType::Internal));
			}
			else if (UClass* WidgetClass = TemplateClass->GetWidgetClass().Get())
			{
				ClassObjects.Add(WidgetClass);
			}
		}
	}
	
	UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();

	if (!SourceItemPaths.IsEmpty())
	{
		if (!ContentBrowserDataSubsystem->EnumerateItemsAtPaths(SourceItemPaths, EContentBrowserItemTypeFilter::IncludeFiles, InCallback))
		{
			return false;
		}
	}

	if (!ClassObjects.IsEmpty())
	{
		if (!ContentBrowserDataSubsystem->EnumerateItemsForObjects(ClassObjects, InCallback))
		{
			return false;
		}
	}

	return true;
}

FText FWidgetTemplateListViewModel::GetName() const
{
	return FText();
}

bool FWidgetTemplateListViewModel::IsTemplate() const
{
	return false;
}

void FWidgetTemplateListViewModel::GetFilterStrings(TArray<FString>& OutStrings) const
{
}

bool FWidgetTemplateListViewModel::HasFilteredChildTemplates() const
{
	FText SearchText = SearchFilter->GetRawFilterText();
	return SearchText.IsEmpty() || CachedLowercaseWidgetFilter.Contains(SearchText.ToString().ToLower());
}

TSharedRef<ITableRow> FWidgetTemplateListViewModel::BuildRow(const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FWidgetViewModel>>, OwnerTable)
		.Padding(2.0f)
		.ShowSelection(false)
		.ShowWires(false)
		.Style(FAppStyle::Get(), "UMGEditor.LibraryView")
		[
			AssetViewPtr.ToSharedRef()
		];
}

void FWidgetTemplateListViewModel::SetViewType(EAssetViewType::Type ViewType)
{
	if (AssetViewPtr)
	{
		AssetViewPtr->SetCurrentViewType(ViewType);
	}
}

void FWidgetTemplateListViewModel::SetThumbnailSize(EThumbnailSize ThumbnailSize)
{
	if (AssetViewPtr)
	{
		AssetViewPtr->SetCurrentThumbnailSize(ThumbnailSize);
	}
}

void FWidgetTemplateListViewModel::SetSearchText(const FText& InSearchText)
{
	if (SearchFilter && AssetViewPtr)
	{
		SearchFilter->SetRawFilterText(InSearchText);

		AssetViewPtr->SetUserSearching(!InSearchText.IsEmpty());
	}
}

FText FWidgetTemplateListViewModel::GetSearchText() const
{
	if (SearchFilter)
	{
		return SearchFilter->GetRawFilterText();
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
