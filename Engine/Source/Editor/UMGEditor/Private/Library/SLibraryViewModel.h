// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetBlueprintEditor.h"
#include "AssetRegistry/AssetData.h"

#include "IContentBrowserSingleton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Palette/SPaletteViewModel.h"

class FWidgetTemplate;
class FWidgetBlueprintEditor;
class UWidgetBlueprint;
class SAssetView;
class SView;

class FWidgetTemplateListViewModel : public FWidgetViewModel
{
public:
	FWidgetTemplateListViewModel();

	/** Builds list view for given templates */
	void ConstructListView(TArray<TSharedPtr<FWidgetTemplate>> InTemplates);

	//~ Begin FWidgetViewModel Interface
	virtual FText GetName() const override;
	virtual bool IsTemplate() const override;
	virtual void GetFilterStrings(TArray<FString>& OutStrings) const override;
	virtual bool HasFilteredChildTemplates() const override;
	virtual TSharedRef<ITableRow> BuildRow(const TSharedRef<STableViewBase>& OwnerTable) override;
	//~ End FWidgetViewModel Interface

	void SetViewType(EAssetViewType::Type ViewType);

	void SetThumbnailSize(EThumbnailSize ThumbnailSize);

	void SetSearchText(const FText& InSearchText);
	FText GetSearchText() const;

	TArray<TSharedPtr<FWidgetTemplate>> Templates;
	TSharedPtr<FAssetFilterCollectionType> TemplatesFilter;
private:

	bool EnumerateCustomSourceItemDatas(TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);

	/** The asset view widget */
	TSharedPtr<SAssetView> AssetViewPtr;

	/** Filter we forward our search text to for the asset view */
	TSharedPtr<class FAssetTextFilter> SearchFilter;

	/** Source data for the asset view */
	TSharedPtr<struct FAssetViewContentSources> LibraryContentSources;

	/** Source item path storage for asset view, accessed via delegate */
	TArray<FContentBrowserItemPath> SourceItemPaths;

	/** Lowercase string representing widget text filter */
	FString CachedLowercaseWidgetFilter;

	/** Number of widget assets in view */
	uint32 NumAssets = 0;

	/** Number of widget classes in view */
	uint32 NumClasses = 0;
};

class FLibraryViewModel : public FWidgetCatalogViewModel
{
public:
	FLibraryViewModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor) : FWidgetCatalogViewModel(InBlueprintEditor) { }

	//~ Begin FWidgetCatalogViewModel Interface
	virtual void BuildWidgetTemplateCategory(FString& Category, TArray<TSharedPtr<FWidgetTemplate>>& Templates, TArray<FString>& FavoritesList) override;
	virtual void SetSearchText(const FText& InSearchText) override;
	//~ End FWidgetCatalogViewModel Interface

	ViewModelsArray& GetWidgetTemplateListViewModels() { return WidgetTemplateListViewModels; }

protected:
	//~ Begin FWidgetCatalogViewModel Interface
	virtual void BuildWidgetList() override;
	//~ End FWidgetCatalogViewModel Interface
	
	/** Widget template list view models associated with each root. */
	ViewModelsArray WidgetTemplateListViewModels;
};

