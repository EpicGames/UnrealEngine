// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/SWidgetDesignerNavigation.h"

#include "Async/Async.h"
#include "Blueprint/UserWidget.h"
#include "DesignerExtension.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "IHasDesignerExtensibility.h"
#include "INavigationEventSimulationView.h"
#include "ISlateReflectorModule.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UObject/StructOnScope.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprintEditorUtils.h"

#include "Application/SlateWindowHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SVirtualWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SWidgetDesignerNavigation)


#define LOCTEXT_NAMESPACE "UMG"

namespace WidgetDesignerNavigation
{
	class FNavigationExtensionFactory;
	class FNavigationExtension;

	static const FName SlateReflectorModuleName("SlateReflector");
	static TWeakPtr<FNavigationExtensionFactory> NavigationExtensionFactory;

	/** Extension factory that also knows which editor did consctructed the extension */
	class FNavigationExtensionFactory : public IDesignerExtensionFactory, public TSharedFromThis<FNavigationExtensionFactory>
	{
	public:
		FNavigationExtensionFactory() = default;
		virtual ~FNavigationExtensionFactory() = default;

		virtual TSharedRef<FDesignerExtension> CreateDesignerExtension() const
		{
			return StaticCastSharedRef<FDesignerExtension>(MakeShared<FNavigationExtension>());
		}

		void PreviewContentChanged(const TSharedRef<FNavigationExtension> Extension, TSharedRef<SWidget> NewContent)
		{
			int32 FoundIndex = IndexOfItem(Extension);
			if (Items.IsValidIndex(FoundIndex))
			{
				if (Items[FoundIndex].WidgetDesigner)
				{
					Items[FoundIndex].WidgetDesigner->HandleWidgetPreviewUpdated(NewContent);
				}
			}
		}

		void Paint(const TSharedRef<const FNavigationExtension> Extension, IUMGDesigner* Designer, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId)
		{
			int32 FoundIndex = IndexOfItem(Extension);
			if (Items.IsValidIndex(FoundIndex))
			{
				if (Items[FoundIndex].WidgetDesigner)
				{
					Items[FoundIndex].WidgetDesigner->HandlePaint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
				}
			}
		}

		void AddExtension(UWidgetBlueprint* WidgetBlueprint, TSharedRef<FNavigationExtension> Extension)
		{
			int32 FoundIndex = IndexOfItem(WidgetBlueprint);
			if (!Items.IsValidIndex(FoundIndex))
			{
				FoundIndex = Items.AddDefaulted();
				Items[FoundIndex].WidgetBlueprint = WidgetBlueprint;
			}
			Items[FoundIndex].Extension = Extension;
		}

		void RemoveExtension(TSharedRef<FNavigationExtension> Extension)
		{
			int32 FoundIndex = IndexOfItem(Extension);
			if (Items.IsValidIndex(FoundIndex))
			{
				if (Items[FoundIndex].WidgetDesigner)
				{
					Items[FoundIndex].Extension.Reset();
				}
				else
				{
					Items.RemoveAtSwap(FoundIndex);
				}
			}
		}

		void AddDesigner(UWidgetBlueprint* WidgetBlueprint, SWidgetDesignerNavigation* WidgetDesigner)
		{
			int32 FoundIndex = IndexOfItem(WidgetBlueprint);
			if (!Items.IsValidIndex(FoundIndex))
			{
				FoundIndex = Items.AddDefaulted();
				Items[FoundIndex].WidgetBlueprint = WidgetBlueprint;
			}
			Items[FoundIndex].WidgetDesigner = WidgetDesigner;
		}

		void RemoveDesigner(SWidgetDesignerNavigation* WidgetDesigner)
		{
			int32 FoundIndex = IndexOfItem(WidgetDesigner);
			if (Items.IsValidIndex(FoundIndex))
			{
				if (Items[FoundIndex].Extension.IsValid())
				{
					Items[FoundIndex].WidgetDesigner = nullptr;
				}
				else
				{
					Items.RemoveAtSwap(FoundIndex);
				}
			}
		}

	private:
		struct FItem
		{
			TWeakObjectPtr<UWidgetBlueprint> WidgetBlueprint;
			TWeakPtr<FNavigationExtension> Extension;
			SWidgetDesignerNavigation* WidgetDesigner = nullptr;
		};

		int32 IndexOfItem(const UWidgetBlueprint* WidgetBlueprint) const
		{
			return Items.IndexOfByPredicate([WidgetBlueprint](const FItem& Item) { return Item.WidgetBlueprint == WidgetBlueprint; });
		}

		int32 IndexOfItem(const TSharedRef<const FNavigationExtension>& Extension) const
		{
			return Items.IndexOfByPredicate([Extension](const FItem& Item) { return Item.Extension == Extension; });
		}

		int32 IndexOfItem(const SWidgetDesignerNavigation* Designer) const
		{
			return Items.IndexOfByPredicate([Designer](const FItem& Item) { return Item.WidgetDesigner == Designer; });
		}

		TArray<FItem> Items;
	};


	class FNavigationExtension : public FDesignerExtension
	{
	private:
		using Super = FDesignerExtension;

	public:
		FNavigationExtension()
		{
			ExtensionId = FName(TEXT("NavigationSimulation"));
		}

		virtual void Initialize(IUMGDesigner* InDesigner, UWidgetBlueprint* InBlueprint) override
		{
			Super::Initialize(InDesigner, InBlueprint);

			TSharedPtr<FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
			if (ensure(FactoryPinned))
			{
				FactoryPinned->AddExtension(InBlueprint, SharedThis(this));
			}
		}

		virtual void Uninitialize() override
		{
			TSharedPtr<FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
			if (ensure(FactoryPinned))
			{
				FactoryPinned->RemoveExtension(SharedThis(this));
			}
			Super::Uninitialize();
		}

		virtual void PreviewContentChanged(TSharedRef<SWidget> NewContent) override
		{
			TSharedPtr<FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
			if (ensure(FactoryPinned))
			{
				FactoryPinned->PreviewContentChanged(SharedThis(this), NewContent);
			}
		}

		virtual void Paint(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
		{
			TSharedPtr<FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
			if (ensure(FactoryPinned))
			{
				TSharedRef<const FNavigationExtension> Shared = SharedThis<FNavigationExtension>(this);
				FactoryPinned->Paint(Shared, Designer, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			}
		}
	};
}


TSharedRef<IDesignerExtensionFactory> SWidgetDesignerNavigation::MakeDesignerExtension()
{
	TSharedPtr<WidgetDesignerNavigation::FNavigationExtensionFactory> Pinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
	if (Pinned)
	{
		ensure(false); // this should only be called only once by the module
		return StaticCastSharedRef<IDesignerExtensionFactory>(Pinned.ToSharedRef());
	}

	TSharedRef<WidgetDesignerNavigation::FNavigationExtensionFactory> Temp = MakeShareable(new WidgetDesignerNavigation::FNavigationExtensionFactory);
	WidgetDesignerNavigation::NavigationExtensionFactory = Temp;
	return StaticCastSharedRef<IDesignerExtensionFactory>(Temp);
}

void SWidgetDesignerNavigation::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;
	SimulationWidgetRequested = 0;
	bIsInSelection = false;

	TSharedPtr<WidgetDesignerNavigation::FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
	if (ensure(FactoryPinned))
	{
		FactoryPinned->AddDesigner(GetBlueprint(), this);
	}

	ISlateReflectorModule& ReflectorModule = FModuleManager::LoadModuleChecked<ISlateReflectorModule>(WidgetDesignerNavigation::SlateReflectorModuleName);
	FNavigationEventSimulationViewArgs ViewArgs;
	ViewArgs.OnWidgetSelected = FSimpleWidgetDelegate::CreateSP(this, &SWidgetDesignerNavigation::HandleSelectWidget);
	ViewArgs.OnNavigateToWidget = FSimpleWidgetDelegate::CreateSP(this, &SWidgetDesignerNavigation::HandleSelectWidget);
	NavigationEventSimulationView = ReflectorModule.CreateNavigationEventSimulationView(ViewArgs);

	BlueprintEditor.Pin()->OnSelectedWidgetsChanged.AddRaw(this, &SWidgetDesignerNavigation::HandleEditorSelectionChanged);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	FStructureDetailsViewArgs StructureDetailsViewArgs;
	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FNavigationSimulationArguments::StaticStruct(), (uint8*)&SimulationArguments);
	TSharedRef<IStructureDetailsView> DetailView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, StructOnScope);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.FillHeight(1.f)
			[
				NavigationEventSimulationView.ToSharedRef()
			]
			
			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.AutoHeight()
			[
				DetailView->GetWidget().ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.AutoHeight()
			[
				SNew(SButton)
				.ContentPadding(3.0f)
				.OnClicked(this, &SWidgetDesignerNavigation::HandleRefreshClicked)
				.IsEnabled(this, &SWidgetDesignerNavigation::HandleRefreshEnabled)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Refresh)
					]

					+ SHorizontalBox::Slot()
					.Padding(4.f, 0.f, 0.f, 0.f)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Refresh", "Refresh"))
					]
				]
			]
		]
	];
}

SWidgetDesignerNavigation::~SWidgetDesignerNavigation()
{
	TSharedPtr<WidgetDesignerNavigation::FNavigationExtensionFactory> FactoryPinned = WidgetDesignerNavigation::NavigationExtensionFactory.Pin();
	if (ensure(FactoryPinned))
	{
		FactoryPinned->RemoveDesigner(this);
	}

	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin())
	{
		BlueprintEditorPinned->OnSelectedWidgetsChanged.RemoveAll(this);
	}
}

UWidgetBlueprint* SWidgetDesignerNavigation::GetBlueprint() const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin();
	return BlueprintEditorPinned ? CastChecked<UWidgetBlueprint>(BlueprintEditorPinned->GetBlueprintObj()) : nullptr;
}

void SWidgetDesignerNavigation::HandleWidgetPreviewUpdated(TSharedRef<SWidget> NewContent)
{
	++SimulationWidgetRequested;

	TWeakPtr<SWidgetDesignerNavigation> AsWeak = SharedThis(this);
	TWeakPtr<SWidget> WeakRequest = NewContent;
	AsyncTask(ENamedThreads::GameThread, [AsWeak, WeakRequest]()
		{
			if (TSharedPtr<SWidgetDesignerNavigation> AsPinned = AsWeak.Pin())
			{
				--(AsPinned->SimulationWidgetRequested);
				if (AsPinned->SimulationWidgetRequested == 0)
				{
					if (TSharedPtr<SWidget> RequestPin = WeakRequest.Pin())
					{
						AsPinned->RunNewSimulation(RequestPin.ToSharedRef());
					}
					else
					{
						AsPinned->ClearSimulationResult();
					}
				}
			}
		});
}

void SWidgetDesignerNavigation::RunNewSimulation(TSharedRef<SWidget> NewContent)
{
	if (!FSlateApplication::IsInitialized() || !FApp::CanEverRender())
	{
		LastSimulationContent.Reset();
		return;
	}

	LastSimulationContent = NewContent;

	// Create a virtual window to create a hit test grid
	const FVector2D WidgetSize{ 1024, 1024 };
	const float Scale = 1.f;
	const float DeltaTime = 0.1f;

	TSharedPtr<SWidget> OldParent = NewContent->GetParentWidget();
	TSharedRef<SVirtualWindow> VirtualWindow = SNew(SVirtualWindow)
		.Size(WidgetSize);
	VirtualWindow->AddMetadata<FReflectionMetaData>(MakeShared<FReflectionMetaData>(TEXT("Simulation Window"), nullptr, nullptr, nullptr));
	VirtualWindow->SetContent(NewContent);

	FWidgetBlueprintEditorUtils::UpdateHittestGrid(VirtualWindow->GetHittestGrid(), VirtualWindow, Scale, WidgetSize, DeltaTime);

	// Find the content inside the virtual window
	FWidgetPath FoundWidgetPath;
	{
		TArray<TSharedRef<SWindow>> SearchWindow;
		SearchWindow.Add(VirtualWindow);
		FSlateWindowHelper::FindPathToWidget(SearchWindow, NewContent, FoundWidgetPath);
	}

	// Run the simulation
	TArray<FSlateNavigationEventSimulator::FSimulationResult> SimulationResult;
	if (FoundWidgetPath.IsValid())
	{
		FSlateNavigationEventSimulator* Simulator = FModuleManager::GetModulePtr<ISlateReflectorModule>(WidgetDesignerNavigation::SlateReflectorModuleName)->GetNavigationEventSimulator();
		check(Simulator);
		if (SimulationArguments.bOverrideUINavigation)
		{
			SimulationResult = Simulator->SimulateForEachWidgets(FoundWidgetPath, SimulationArguments.UserIndex, SimulationArguments.NavigationGenesis, SimulationArguments.UINavigation);
		}
		else
		{
			SimulationResult = Simulator->SimulateForEachWidgets(FoundWidgetPath, SimulationArguments.UserIndex, SimulationArguments.NavigationGenesis, FSlateNavigationEventSimulator::ENavigationStyle::FourCardinalDirections);
		}
	}

	NavigationEventSimulationView->SetSimulationResult(SimulationResult);

	// Reset the old parent
	if (OldParent.IsValid())
	{
		NewContent->AssignParentWidget(OldParent);
	}
}

void SWidgetDesignerNavigation::ClearSimulationResult()
{
	TArray<FSlateNavigationEventSimulator::FSimulationResult> SimulationResult;
	NavigationEventSimulationView->SetSimulationResult(SimulationResult);
}

void SWidgetDesignerNavigation::HandleEditorSelectionChanged()
{
	if (bIsInSelection)
	{
		return;
	}

	TGuardValue<bool> Tmp(bIsInSelection, true);
	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin())
	{
		const TSet<FWidgetReference>& SelectedWidgetReferences = BlueprintEditorPinned->GetSelectedWidgets();
		if (SelectedWidgetReferences.Num() > 0)
		{
			auto It = SelectedWidgetReferences.CreateConstIterator();
			const FWidgetReference& WidgetReference = *It;
			NavigationEventSimulationView->SelectWidget(WidgetReference.GetPreviewSlate());
		}
		else
		{
			NavigationEventSimulationView->SelectWidget(TSharedPtr<SWidget>());
		}
	}
}

void SWidgetDesignerNavigation::HandleSelectWidget(TWeakPtr<const SWidget> WeakWidget)
{
	if (bIsInSelection)
	{
		return;
	}

	TGuardValue<bool> Tmp(bIsInSelection, true);
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin();
	TSharedPtr<const SWidget> WidgetPinned = WeakWidget.Pin();
	if (BlueprintEditorPinned && WidgetPinned)
	{
		if (UUserWidget* PreviewUserWidget = BlueprintEditorPinned->GetPreview())
		{
			if (UWidget* UMGWidget = PreviewUserWidget->GetWidgetHandle(ConstCastSharedRef<SWidget>(WidgetPinned.ToSharedRef())))
			{
				FWidgetReference Reference = BlueprintEditorPinned->GetReferenceFromPreview(UMGWidget);
				if (Reference.IsValid())
				{
					TSet<FWidgetReference> WidgetReferences;
					WidgetReferences.Add(Reference);
					BlueprintEditorPinned->SelectWidgets(WidgetReferences, false);
				}
			}
		}
	}
}

void SWidgetDesignerNavigation::HandlePaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId)
{
	if (SimulationArguments.bShowPreview)
	{
		if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin())
		{
			if (UUserWidget* PreviewUserWidget = BlueprintEditorPinned->GetPreview())
			{
				NavigationEventSimulationView->PaintSimuationResult(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			}
		}
	}
}

FReply SWidgetDesignerNavigation::HandleRefreshClicked()
{
	if (SimulationWidgetRequested == 0)
	{
		if (TSharedPtr<SWidget> NewConent = LastSimulationContent.Pin())
		{
			HandleWidgetPreviewUpdated(NewConent.ToSharedRef());
		}
	}
	return FReply::Handled();
}

bool SWidgetDesignerNavigation::HandleRefreshEnabled() const
{
	return LastSimulationContent.IsValid();
}

#undef LOCTEXT_NAMESPACE
