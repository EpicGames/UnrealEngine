// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SWidgetDesignerNavigation.generated.h"

class IDesignerExtensionFactory;
class INavigationEventSimulationView;
class UWidgetBlueprint;
class FWidgetBlueprintEditor;

namespace WidgetDesignerNavigation
{
	class FNavigationExtensionFactory;
}

USTRUCT()
struct FNavigationSimulationArguments
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Simulation Options")
	int32 UserIndex = 0;

	UPROPERTY(EditAnywhere, Category = "Simulation Options")
	ENavigationGenesis NavigationGenesis = ENavigationGenesis::Controller;

	UPROPERTY(EditAnywhere, Category = "Simulation Options", meta = (EditCondition = "bOverrideUINavigation"))
	EUINavigation UINavigation = EUINavigation::Left;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (InlineEditConditionToggle))
	bool bOverrideUINavigation = false;
	
	UPROPERTY(EditAnywhere, Category = "Display")
	bool bShowPreview = true;
};

/**
 */
class SWidgetDesignerNavigation : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWidgetDesignerNavigation){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~SWidgetDesignerNavigation();

	static TSharedRef<IDesignerExtensionFactory> MakeDesignerExtension();

private:
	UWidgetBlueprint* GetBlueprint() const;
	void RunNewSimulation(TSharedRef<SWidget> NewContent);
	void ClearSimulationResult();

	void HandleWidgetPreviewUpdated(TSharedRef<SWidget> NewContent);
	void HandleEditorSelectionChanged();
	void HandleSelectWidget(TWeakPtr<const SWidget> Widget);
	void HandlePaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId);
	FReply HandleRefreshClicked();
	bool HandleRefreshEnabled() const;

private:
	/** Cached pointer to the blueprint editor. */
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;

	/** The simulation result view. */
	TSharedPtr<INavigationEventSimulationView> NavigationEventSimulationView;

	/** A simulation is requested. */
	TAtomic<int32> SimulationWidgetRequested;

	/** Last widget the simulation was executed from. */
	TWeakPtr<SWidget> LastSimulationContent;

	/** Simulation Arguments */
	FNavigationSimulationArguments SimulationArguments;

	/** In selection */
	bool bIsInSelection;

	friend WidgetDesignerNavigation::FNavigationExtensionFactory;
};
