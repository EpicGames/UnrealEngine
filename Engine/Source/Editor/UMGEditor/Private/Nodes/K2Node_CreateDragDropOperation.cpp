// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/K2Node_CreateDragDropOperation.h"

#include "K2Node_CallFunction.h"
#include "Blueprint/DragDropOperation.h"

#include "Blueprint/WidgetBlueprintLibrary.h"
#include "EditorCategoryUtils.h"
#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CreateDragDropOperation)

#define LOCTEXT_NAMESPACE "UMG"

UK2Node_CreateDragDropOperation::UK2Node_CreateDragDropOperation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("DragDropNodeTooltip", "Creates a new drag drop operation");
}

FText UK2Node_CreateDragDropOperation::GetBaseNodeTitle() const
{
	return LOCTEXT("CreateDragDropWidget_BaseTitle", "Create Drag & Drop Operation");
}

FText UK2Node_CreateDragDropOperation::GetNodeTitleFormat() const
{
	return LOCTEXT("CreateDragDropWidget", "Create {ClassName}");
}

UClass* UK2Node_CreateDragDropOperation::GetClassPinBaseClass() const
{
	return UDragDropOperation::StaticClass();
}

FText UK2Node_CreateDragDropOperation::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::UserInterface);
}

FName UK2Node_CreateDragDropOperation::GetCornerIcon() const
{
	return TEXT("Graph.Replication.ClientEvent");
}

void UK2Node_CreateDragDropOperation::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* ClassPin = GetClassPin();
	if ( ClassPin->DefaultObject == NULL )
	{
		ClassPin->DefaultObject = UDragDropOperation::StaticClass();

		UClass* UseSpawnClass = GetClassToSpawn();
		if ( UseSpawnClass != NULL )
		{
			CreatePinsForClass(UseSpawnClass);
		}
	}
}

void UK2Node_CreateDragDropOperation::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	static FName Create_FunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetBlueprintLibrary, CreateDragDropOperation);
	static FName OperationClass_ParamName(TEXT("OperationClass"));

	UK2Node_CreateDragDropOperation* CreateOpNode = this;
	UEdGraphPin* SpawnNodeExec = CreateOpNode->GetExecPin();
	UEdGraphPin* SpawnClassPin = CreateOpNode->GetClassPin();
	UEdGraphPin* SpawnNodeThen = CreateOpNode->GetThenPin();
	UEdGraphPin* SpawnNodeResult = CreateOpNode->GetResultPin();

	if (!SpawnNodeExec || !SpawnClassPin || !SpawnNodeThen || !SpawnNodeResult)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("CreateDragDropOperation_InternalError", "Invalid Drag/Drop node @@").ToString(), CreateOpNode);
		CreateOpNode->BreakAllNodeLinks();
		return;
	}

	UClass* SpawnClass = ( SpawnClassPin != NULL ) ? Cast<UClass>(SpawnClassPin->DefaultObject) : NULL;
	//if ( ( 0 == SpawnClassPin->LinkedTo.Num() ) && ( NULL == SpawnClass ) )
	//{
	//	CompilerContext.MessageLog.Error(*LOCTEXT("CreateWidgetNodeMissingClass_Error", "Create Drag/Drop node @@ must have a class specified.").ToString(), CreateOpNode);
	//	// we break exec links so this is the only error we get, don't want the CreateWidget node being considered and giving 'unexpected node' type warnings
	//	CreateOpNode->BreakAllNodeLinks();
	//	return;
	//}

	//////////////////////////////////////////////////////////////////////////
	// create 'UWidgetBlueprintLibrary::CreateDragDropOperation' call node
	UK2Node_CallFunction* CallCreateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(CreateOpNode, SourceGraph);
	CallCreateNode->FunctionReference.SetExternalMember(Create_FunctionName, UWidgetBlueprintLibrary::StaticClass());
	CallCreateNode->AllocateDefaultPins();

	UEdGraphPin* CallCreateExec = CallCreateNode->GetExecPin();
	UEdGraphPin* CallCreateOperationClassPin = CallCreateNode->FindPinChecked(OperationClass_ParamName);
	UEdGraphPin* CallCreateResult = CallCreateNode->GetReturnValuePin();

	// Move 'exec' connection from create widget node to 'UWidgetBlueprintLibrary::CreateDragDropOperation'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeExec, *CallCreateExec);

	if ( SpawnClassPin->LinkedTo.Num() > 0 )
	{
		// Copy the 'blueprint' connection from the spawn node to 'UWidgetBlueprintLibrary::CreateDragDropOperation'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateOperationClassPin);
	}
	else
	{
		// Copy blueprint literal onto 'UWidgetBlueprintLibrary::CreateDragDropOperation' call 
		CallCreateOperationClassPin->DefaultObject = SpawnClass;
	}

	// Move result connection from spawn node to 'UWidgetBlueprintLibrary::CreateDragDropOperation'
	CallCreateResult->PinType = SpawnNodeResult->PinType; // Copy type so it uses the right actor subclass
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////
	// create 'set var' nodes

	// Get 'result' pin from 'begin spawn', this is the actual actor we want to set properties on
	UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes(CompilerContext, SourceGraph, CallCreateNode, CreateOpNode, CallCreateResult, GetClassToSpawn(), CallCreateOperationClassPin);

	// Move 'then' connection from create widget node to the last 'then'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *LastThen);

	// Break any links to the expanded node
	CreateOpNode->BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
