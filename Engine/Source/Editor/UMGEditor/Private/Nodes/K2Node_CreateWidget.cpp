// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/K2Node_CreateWidget.h"

#if WITH_EDITOR
	#include "GameFramework/PlayerController.h"
#endif // WITH_EDITOR
#include "EdGraphSchema_K2.h"
#include "K2Node_CallFunction.h"
#include "Blueprint/UserWidget.h"

#include "Blueprint/WidgetBlueprintLibrary.h"
#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"
#include "EditorCategoryUtils.h"
#include "Widgets/SGraphNodeCreateWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CreateWidget)


#define LOCTEXT_NAMESPACE "UMG"

struct FK2Node_CreateWidgetHelper
{
	static const FName OwningPlayerPinName;
};

const FName FK2Node_CreateWidgetHelper::OwningPlayerPinName(TEXT("OwningPlayer"));

UK2Node_CreateWidget::UK2Node_CreateWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Creates a new widget");
}

void UK2Node_CreateWidget::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// OwningPlayer pin
	UEdGraphPin* OwningPlayerPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, APlayerController::StaticClass(), FK2Node_CreateWidgetHelper::OwningPlayerPinName);
	SetPinToolTip(*OwningPlayerPin, LOCTEXT("OwningPlayerPinDescription", "The player that 'owns' the widget."));
}

FText UK2Node_CreateWidget::GetBaseNodeTitle() const
{
	return LOCTEXT("CreateWidget_BaseTitle", "Create Widget");
}

FText UK2Node_CreateWidget::GetNodeTitleFormat() const
{
	return LOCTEXT("CreateWidget", "Create {ClassName} Widget");
}

UClass* UK2Node_CreateWidget::GetClassPinBaseClass() const
{
	return UUserWidget::StaticClass();
}

FText UK2Node_CreateWidget::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::UserInterface);
}

FName UK2Node_CreateWidget::GetCornerIcon() const
{
	return TEXT("Graph.Replication.ClientEvent");
}

UEdGraphPin* UK2Node_CreateWidget::GetOwningPlayerPin() const
{
	UEdGraphPin* Pin = FindPin(FK2Node_CreateWidgetHelper::OwningPlayerPinName);
	check(Pin == NULL || Pin->Direction == EGPD_Input);
	return Pin;
}

bool UK2Node_CreateWidget::IsSpawnVarPin(UEdGraphPin* Pin) const
{
	return( Super::IsSpawnVarPin(Pin) &&
		Pin->PinName != FK2Node_CreateWidgetHelper::OwningPlayerPinName );
}

void UK2Node_CreateWidget::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	static const FName Create_FunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetBlueprintLibrary, Create);
	static const FName WorldContextObject_ParamName(TEXT("WorldContextObject"));
	static const FName WidgetType_ParamName(TEXT("WidgetType"));
	static const FName OwningPlayer_ParamName(TEXT("OwningPlayer"));

	UK2Node_CreateWidget* CreateWidgetNode = this;
	UEdGraphPin* SpawnNodeExec = CreateWidgetNode->GetExecPin();
	UEdGraphPin* SpawnWorldContextPin = CreateWidgetNode->GetWorldContextPin();
	UEdGraphPin* SpawnOwningPlayerPin = CreateWidgetNode->GetOwningPlayerPin();
	UEdGraphPin* SpawnClassPin = CreateWidgetNode->GetClassPin();
	UEdGraphPin* SpawnNodeThen = CreateWidgetNode->GetThenPin();
	UEdGraphPin* SpawnNodeResult = CreateWidgetNode->GetResultPin();

	UClass* SpawnClass = ( SpawnClassPin != NULL ) ? Cast<UClass>(SpawnClassPin->DefaultObject) : NULL;
	if ( !SpawnClassPin || ((0 == SpawnClassPin->LinkedTo.Num()) && (NULL == SpawnClass)))
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("CreateWidgetNodeMissingClass_Error", "Spawn node @@ must have a class specified.").ToString(), CreateWidgetNode);
		// we break exec links so this is the only error we get, don't want the CreateWidget node being considered and giving 'unexpected node' type warnings
		CreateWidgetNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'UWidgetBlueprintLibrary::Create' call node
	UK2Node_CallFunction* CallCreateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(CreateWidgetNode, SourceGraph);
	CallCreateNode->FunctionReference.SetExternalMember(Create_FunctionName, UWidgetBlueprintLibrary::StaticClass());
	CallCreateNode->AllocateDefaultPins();

	// store off the class to spawn before we mutate pin connections:
	UClass* ClassToSpawn = GetClassToSpawn();

	UEdGraphPin* CallCreateExec = CallCreateNode->GetExecPin();
	UEdGraphPin* CallCreateWorldContextPin = CallCreateNode->FindPinChecked(WorldContextObject_ParamName);
	UEdGraphPin* CallCreateWidgetTypePin = CallCreateNode->FindPinChecked(WidgetType_ParamName);
	UEdGraphPin* CallCreateOwningPlayerPin = CallCreateNode->FindPinChecked(OwningPlayer_ParamName);
	UEdGraphPin* CallCreateResult = CallCreateNode->GetReturnValuePin();

	// Move 'exec' connection from create widget node to 'UWidgetBlueprintLibrary::Create'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeExec, *CallCreateExec);

	if ( SpawnClassPin->LinkedTo.Num() > 0 )
	{
		// Copy the 'blueprint' connection from the spawn node to 'UWidgetBlueprintLibrary::Create'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateWidgetTypePin);
	}
	else
	{
		// Copy blueprint literal onto 'UWidgetBlueprintLibrary::Create' call
		CallCreateWidgetTypePin->DefaultObject = SpawnClass;
	}

	// Copy the world context connection from the spawn node to 'UWidgetBlueprintLibrary::Create' if necessary
	if ( SpawnWorldContextPin )
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnWorldContextPin, *CallCreateWorldContextPin);
	}

	// Copy the 'Owning Player' connection from the spawn node to 'UWidgetBlueprintLibrary::Create'
	CompilerContext.MovePinLinksToIntermediate(*SpawnOwningPlayerPin, *CallCreateOwningPlayerPin);

	// Move result connection from spawn node to 'UWidgetBlueprintLibrary::Create'
	CallCreateResult->PinType = SpawnNodeResult->PinType; // Copy type so it uses the right actor subclass
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////
	// create 'set var' nodes

	// Get 'result' pin from 'begin spawn', this is the actual actor we want to set properties on
	UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes(CompilerContext, SourceGraph, CallCreateNode, CreateWidgetNode, CallCreateResult, ClassToSpawn, CallCreateWidgetTypePin);

	// Move 'then' connection from create widget node to the last 'then'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *LastThen);

	// Break any links to the expanded node
	CreateWidgetNode->BreakAllNodeLinks();
}

TSharedPtr<SGraphNode> UK2Node_CreateWidget::CreateVisualWidget()
{
	return SNew(SGraphNodeCreateWidget, this);
}

#undef LOCTEXT_NAMESPACE
