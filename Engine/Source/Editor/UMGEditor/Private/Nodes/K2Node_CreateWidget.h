// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "K2Node_ConstructObjectFromClass.h"
#include "K2Node_CreateWidget.generated.h"

#define UE_API UMGEDITOR_API

class UEdGraph;

UCLASS(MinimalAPI)
class UK2Node_CreateWidget : public UK2Node_ConstructObjectFromClass
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FName GetCornerIcon() const override;
	//~ End UK2Node Interface.

	/** Get the owning player pin */
	UE_API UEdGraphPin* GetOwningPlayerPin() const;

protected:
	/** Gets the default node title when no class is selected */
	UE_API virtual FText GetBaseNodeTitle() const override;
	/** Gets the node title when a class has been selected. */
	UE_API virtual FText GetNodeTitleFormat() const override;
	/** Gets base class to use for the 'class' pin.  UObject by default. */
	UE_API virtual UClass* GetClassPinBaseClass() const override;
	/**  */
	UE_API virtual bool IsSpawnVarPin(UEdGraphPin* Pin) const override;
};

#undef UE_API
