// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PlayAnimation.h"
#include "Animation/WidgetAnimationPlayCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PlayAnimation)

#define LOCTEXT_NAMESPACE "UMG"

UK2Node_PlayAnimation::UK2Node_PlayAnimation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetAnimationPlayCallbackProxy, CreatePlayAnimationProxyObject);
	ProxyFactoryClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
	ProxyClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
}

UK2Node_PlayAnimationTimeRange::UK2Node_PlayAnimationTimeRange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetAnimationPlayCallbackProxy, CreatePlayAnimationTimeRangeProxyObject);
	ProxyFactoryClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
	ProxyClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
}

UK2Node_PlayAnimation2::UK2Node_PlayAnimation2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetAnimationPlayCallbackProxy, NewPlayAnimationProxyObject);
	ProxyFactoryClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
	ProxyClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
}

UK2Node_PlayAnimationTimeRange2::UK2Node_PlayAnimationTimeRange2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UWidgetAnimationPlayCallbackProxy, NewPlayAnimationTimeRangeProxyObject);
	ProxyFactoryClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
	ProxyClass = UWidgetAnimationPlayCallbackProxy::StaticClass();
}

#undef LOCTEXT_NAMESPACE
