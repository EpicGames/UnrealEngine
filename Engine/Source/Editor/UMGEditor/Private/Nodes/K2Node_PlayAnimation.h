// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_PlayAnimation.generated.h"

UCLASS(MinimalAPI)
class UK2Node_PlayAnimation : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_PlayAnimationTimeRange : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_PlayAnimation2 : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_PlayAnimationTimeRange2 : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

