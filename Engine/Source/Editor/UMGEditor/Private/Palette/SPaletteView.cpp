// Copyright Epic Games, Inc. All Rights Reserved.

#include "Palette/SPaletteView.h"
#include "Palette/SPaletteViewModel.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "WidgetBlueprint.h"
#include "Editor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScrollBorder.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR



#include "DragDrop/WidgetTemplateDragDropOp.h"

#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateBlueprintClass.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"

#include "Settings/ContentBrowserSettings.h"
#include "WidgetBlueprintEditorUtils.h"




#define LOCTEXT_NAMESPACE "UMG"

FText SPaletteViewItem::GetFavoriteToggleToolTipText() const
{
	if (GetFavoritedState() == ECheckBoxState::Checked)
	{
		return LOCTEXT("Unfavorite", "Click to remove this widget from your favorites.");
	}
	return LOCTEXT("Favorite", "Click to add this widget to your favorites.");
}

ECheckBoxState SPaletteViewItem::GetFavoritedState() const
{
	if (WidgetViewModel->IsFavorite())
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

void SPaletteViewItem::OnFavoriteToggled(ECheckBoxState InNewState)
{
	if (InNewState == ECheckBoxState::Checked)
	{
		//Add to favorites
		WidgetViewModel->AddToFavorites();
	}
	else
	{
		//Remove from favorites
		WidgetViewModel->RemoveFromFavorites();
	}
}

EVisibility SPaletteViewItem::GetFavoritedStateVisibility() const
{
	return GetFavoritedState() == ECheckBoxState::Checked || IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
}

void SPaletteViewItem::Construct(const FArguments& InArgs, TSharedPtr<FWidgetTemplateViewModel> InWidgetViewModel)
{
	WidgetViewModel = InWidgetViewModel;

	ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTip(WidgetViewModel->Template->GetToolTip())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[	
				SNew(SCheckBox)
				.ToolTipText(this, &SPaletteViewItem::GetFavoriteToggleToolTipText)
				.IsChecked(this, &SPaletteViewItem::GetFavoritedState)
				.OnCheckStateChanged(this, &SPaletteViewItem::OnFavoriteToggled)
				.Style(FAppStyle::Get(), "UMGEditor.Palette.FavoriteToggleStyle")
				.Visibility(this, &SPaletteViewItem::GetFavoritedStateVisibility)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(WidgetViewModel->Template->GetIcon())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InWidgetViewModel->GetName())
				.HighlightText(InArgs._HighlightText)
			]
		];
}

FReply SPaletteViewItem::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return WidgetViewModel->Template->OnDoubleClicked();
};

void SPaletteView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;

	UBlueprint* BP = InBlueprintEditor->GetBlueprintObj();
	PaletteViewModel = InBlueprintEditor->GetPaletteViewModel();

	// Register to the update of the viewmodel.
	PaletteViewModel->OnUpdating.AddRaw(this, &SPaletteView::OnViewModelUpdating);
	PaletteViewModel->OnUpdated.AddRaw(this, &SPaletteView::OnViewModelUpdated);

	WidgetFilter = MakeShareable(new WidgetViewModelTextFilter(
		WidgetViewModelTextFilter::FItemToStringArray::CreateSP(this, &SPaletteView::GetWidgetFilterStrings)));

	FilterHandler = MakeShareable(new PaletteFilterHandler());
	FilterHandler->SetFilter(WidgetFilter.Get());
	FilterHandler->SetRootItems(&(PaletteViewModel->GetWidgetViewModels()), &TreeWidgetViewModels);
	FilterHandler->SetGetChildrenDelegate(PaletteFilterHandler::FOnGetChildren::CreateRaw(this, &SPaletteView::OnGetChildren));

	SAssignNew(WidgetTemplatesView, STreeView< TSharedPtr<FWidgetViewModel> >)
		.SelectionMode(ESelectionMode::SingleToggle)
		.OnGenerateRow(this, &SPaletteView::OnGenerateWidgetTemplateItem)
		.OnGetChildren(FilterHandler.ToSharedRef(), &PaletteFilterHandler::OnGetFilteredChildren)
		.OnSelectionChanged(this, &SPaletteView::WidgetPalette_OnSelectionChanged)
		.OnMouseButtonClick(this, &SPaletteView::WidgetPalette_OnClick)
		.TreeItemsSource(&TreeWidgetViewModels);
		

	FilterHandler->SetTreeView(WidgetTemplatesView.Get());

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(4)
		.AutoHeight()
		[
			SAssignNew(SearchBoxPtr, SSearchBox)
			.HintText(LOCTEXT("SearchPalette", "Search Palette"))
			.OnTextChanged(this, &SPaletteView::OnSearchChanged)
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, WidgetTemplatesView.ToSharedRef())
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(0)
				[
					WidgetTemplatesView.ToSharedRef()
				]
			]
		]
	];

	bRefreshRequested = true;

	PaletteViewModel->Update();
	LoadItemExpansion();
}

SPaletteView::~SPaletteView()
{
	// Unregister to the update of the viewmodel.
	PaletteViewModel->OnUpdating.RemoveAll(this);
	PaletteViewModel->OnUpdated.RemoveAll(this);

	// If the filter is enabled, disable it before saving the expanded items since
	// filtering expands all items by default.
	if (FilterHandler->GetIsEnabled())
	{
		FilterHandler->SetIsEnabled(false);
		FilterHandler->RefreshAndFilterTree();
	}

	SaveItemExpansion();
}

void SPaletteView::OnSearchChanged(const FText& InFilterText)
{
	bRefreshRequested = true;
	FilterHandler->SetIsEnabled(!InFilterText.IsEmpty());
	WidgetFilter->SetRawFilterText(InFilterText);
	SearchBoxPtr->SetError(WidgetFilter->GetFilterErrorText());
	PaletteViewModel->SetSearchText(InFilterText);
}

void SPaletteView::WidgetPalette_OnClick(TSharedPtr<FWidgetViewModel> SelectedItem)
{
	if (!SelectedItem.IsValid())
	{
		return;
	}

	// If it's a category, toggle it
	if (SelectedItem->IsCategory())
	{
		if (TSharedPtr<FWidgetHeaderViewModel> CategoryHeader = StaticCastSharedPtr<FWidgetHeaderViewModel>(SelectedItem))
		{
			if (TSharedPtr<ITableRow> TableRow = WidgetTemplatesView->WidgetFromItem(CategoryHeader))
			{
				TableRow->ToggleExpansion();
			}
		}
		return;
	}
}

void SPaletteView::WidgetPalette_OnSelectionChanged(TSharedPtr<FWidgetViewModel> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (!SelectedItem.IsValid()) 
	{
		return;
	}

	// Reset the selected
	BlueprintEditor.Pin()->SetSelectedTemplate(nullptr);
	BlueprintEditor.Pin()->SetSelectedUserWidget(FAssetData());

	// If it's not a template, return
	if (!SelectedItem->IsTemplate())
	{
		return;
	}

	TSharedPtr<FWidgetTemplateViewModel> SelectedTemplate = StaticCastSharedPtr<FWidgetTemplateViewModel>(SelectedItem);
	if (SelectedTemplate.IsValid())
	{
		TSharedPtr<FWidgetTemplateClass> TemplateClass = StaticCastSharedPtr<FWidgetTemplateClass>(SelectedTemplate->Template);
		if (TemplateClass.IsValid())
		{
			if (TemplateClass->GetWidgetClass().IsValid())
			{
				BlueprintEditor.Pin()->SetSelectedTemplate(TemplateClass->GetWidgetClass());
			}
			else
			{
				TSharedPtr<FWidgetTemplateBlueprintClass> UserCreatedTemplate = StaticCastSharedPtr<FWidgetTemplateBlueprintClass>(TemplateClass);
				if (UserCreatedTemplate.IsValid())
				{
					// Then pass in the asset data of selected widget
					FAssetData UserCreatedWidget = UserCreatedTemplate->GetWidgetAssetData();
					BlueprintEditor.Pin()->SetSelectedUserWidget(UserCreatedWidget);
				}
			}
		}
	}
}

TSharedPtr<FWidgetTemplate> SPaletteView::GetSelectedTemplateWidget() const
{
	TArray<TSharedPtr<FWidgetViewModel>> SelectedTemplates = WidgetTemplatesView.Get()->GetSelectedItems();
	if (SelectedTemplates.Num() == 1)
	{
		TSharedPtr<FWidgetTemplateViewModel> TemplateViewModel = StaticCastSharedPtr<FWidgetTemplateViewModel>(SelectedTemplates[0]);
		if (TemplateViewModel.IsValid())
		{
			return TemplateViewModel->Template;
		}
	}
	return nullptr;
}

void SPaletteView::LoadItemExpansion()
{
	// Restore the expansion state of the widget groups.
	for ( TSharedPtr<FWidgetViewModel>& ViewModel : PaletteViewModel->GetWidgetViewModels())
	{
		bool IsExpanded;
		if ( GConfig->GetBool(TEXT("WidgetTemplatesExpanded"), *ViewModel->GetName().ToString(), IsExpanded, GEditorPerProjectIni) && IsExpanded )
		{
			WidgetTemplatesView->SetItemExpansion(ViewModel, true);
		}
	}
}

void SPaletteView::SaveItemExpansion()
{
	// Restore the expansion state of the widget groups.
	for ( TSharedPtr<FWidgetViewModel>& ViewModel : PaletteViewModel->GetWidgetViewModels() )
	{
		const bool IsExpanded = WidgetTemplatesView->IsItemExpanded(ViewModel);
		GConfig->SetBool(TEXT("WidgetTemplatesExpanded"), *ViewModel->GetName().ToString(), IsExpanded, GEditorPerProjectIni);
	}
}

void SPaletteView::OnGetChildren(TSharedPtr<FWidgetViewModel> Item, TArray< TSharedPtr<FWidgetViewModel> >& Children)
{
	return Item->GetChildren(Children);
}

TSharedRef<ITableRow> SPaletteView::OnGenerateWidgetTemplateItem(TSharedPtr<FWidgetViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return Item->BuildRow(OwnerTable);
}

void SPaletteView::OnViewModelUpdating()
{
	// Save the old expanded items temporarily
	WidgetTemplatesView->GetExpandedItems(ExpandedItems);
}

void SPaletteView::OnViewModelUpdated()
{
	bRefreshRequested = true;

	// Restore the expansion state
	for (TSharedPtr<FWidgetViewModel>& ExpandedItem : ExpandedItems)
	{
		for (TSharedPtr<FWidgetViewModel>& ViewModel : PaletteViewModel->GetWidgetViewModels())
		{
			if (ViewModel->GetName().EqualTo(ExpandedItem->GetName()) || ViewModel->ShouldForceExpansion())
			{
				WidgetTemplatesView->SetItemExpansion(ViewModel, true);
			}
		}
	}
	ExpandedItems.Reset();
}

void SPaletteView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRefreshRequested)
	{
		bRefreshRequested = false;
		FilterHandler->RefreshAndFilterTree();
	}
}

void SPaletteView::GetWidgetFilterStrings(TSharedPtr<FWidgetViewModel> WidgetViewModel, TArray<FString>& OutStrings)
{
	WidgetViewModel->GetFilterStrings(OutStrings);
}

#undef LOCTEXT_NAMESPACE
