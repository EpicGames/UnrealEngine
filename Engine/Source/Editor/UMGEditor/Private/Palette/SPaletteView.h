// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "WidgetBlueprintEditor.h"
#include "Misc/TextFilter.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Views/TreeFilterHandler.h"

class FWidgetTemplate;
class UWidgetBlueprint;
class SPaletteView;
class FPaletteViewModel;
class FWidgetViewModel;
class FWidgetTemplateViewModel;

/** Widget used to show a single row of the Palette and Palette favorite panel. */
class SPaletteViewItem : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPaletteViewItem) {}

	/** The current text to highlight */
	SLATE_ATTRIBUTE(FText, HighlightText)

	SLATE_END_ARGS()

	/**
	* Constructs this widget
	*
	* @param InArgs    Declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetTemplateViewModel> InWidgetViewModel);

private:
	/**
	 * Retrieves tooltip that describes the current favorited state 
	 *
	 * @return Text describing what this toggle will do when you click on it.
	 */
	FText GetFavoriteToggleToolTipText() const;

	/**
	 * Checks on the associated action's favorite state, and returns a
	 * corresponding checkbox state to match.
	 *
	 * @return ECheckBoxState::Checked if the associated action is already favorited, ECheckBoxState::Unchecked if not.
	 */
	ECheckBoxState GetFavoritedState() const;

	/**
	 * Triggers when the user clicks this toggle, adds or removes the associated
	 * action to the user's favorites.
	 *
	 * @param  InNewState	The new state that the user set the checkbox to.
	 */
	void OnFavoriteToggled(ECheckBoxState InNewState);

	EVisibility GetFavoritedStateVisibility() const;

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:
	TSharedPtr<FWidgetTemplateViewModel> WidgetViewModel;
};

/** */
class SPaletteView : public SCompoundWidget
{
public:
	typedef TTextFilter<TSharedPtr<FWidgetViewModel>> WidgetViewModelTextFilter;

public:
	SLATE_BEGIN_ARGS( SPaletteView ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~SPaletteView();
	
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Gets the text currently displayed in the search box. */
	FText GetSearchText() const;

	/** On item clicked in palette view */
	void WidgetPalette_OnClick(TSharedPtr<FWidgetViewModel> SelectedItem);

	/** On Selected template widget changed in palette view */
	void WidgetPalette_OnSelectionChanged(TSharedPtr<FWidgetViewModel> SelectedItem, ESelectInfo::Type SelectInfo);

	/** Gets the selected template widget in palette */
	TSharedPtr<FWidgetTemplate> GetSelectedTemplateWidget() const;

private:

	void OnGetChildren(TSharedPtr<FWidgetViewModel> Item, TArray< TSharedPtr<FWidgetViewModel> >& Children);
	TSharedRef<ITableRow> OnGenerateWidgetTemplateItem(TSharedPtr<FWidgetViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when the filter text is changed. */
	void OnSearchChanged(const FText& InFilterText);

	void OnViewModelUpdating();
	void OnViewModelUpdated();

private:
	/** Load the expansion state for the TreeView */
	void LoadItemExpansion();

	/** Save the expansion state for the TreeView */
	void SaveItemExpansion();

	/** Gets an array of strings used for filtering/searching the specified widget. */
	void GetWidgetFilterStrings(TSharedPtr<FWidgetViewModel> WidgetViewModel, TArray<FString>& OutStrings);

	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<FPaletteViewModel> PaletteViewModel;

	/** Handles filtering the palette based on an IFilter. */
	typedef TreeFilterHandler<TSharedPtr<FWidgetViewModel>> PaletteFilterHandler;
	TSharedPtr<PaletteFilterHandler> FilterHandler;

	typedef TArray<TSharedPtr<FWidgetViewModel>> ViewModelsArray;
	/** The root view models which are actually displayed by the TreeView which will be managed by the TreeFilterHandler. */
	ViewModelsArray TreeWidgetViewModels;

	TSharedPtr< STreeView< TSharedPtr<FWidgetViewModel> > > WidgetTemplatesView;

	/** The search box used to update the filter text */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** The filter instance which is used by the TreeFilterHandler to filter the TreeView. */
	TSharedPtr<WidgetViewModelTextFilter> WidgetFilter;

	/** Expended Items in the Tree view */
	TSet<TSharedPtr<FWidgetViewModel>> ExpandedItems;

	/** Set to true to force a refresh of the treeview */
	bool bRefreshRequested;

	/** Are editor widgets supported. */
	bool bAllowEditorWidget;
};
