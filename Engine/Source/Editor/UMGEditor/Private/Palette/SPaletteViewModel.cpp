// Copyright Epic Games, Inc. All Rights Reserved.

#include "Palette/SPaletteViewModel.h"
#include "Palette/SPaletteView.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "WidgetBlueprint.h"
#include "Editor.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "EditorClassUtils.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "DragDrop/WidgetTemplateDragDropOp.h"

#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateBlueprintClass.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Misc/NamePermissionList.h"

#include "Settings/ContentBrowserSettings.h"
#include "Settings/WidgetDesignerSettings.h"
#include "WidgetEditingProjectSettings.h"
#include "WidgetPaletteFavorites.h"

#define LOCTEXT_NAMESPACE "UMG"

FWidgetTemplateViewModel::FWidgetTemplateViewModel()
	: FavortiesViewModel(nullptr),
	bIsFavorite(false)
{
}

FText FWidgetTemplateViewModel::GetName() const
{
	return Template->Name;
}

bool FWidgetTemplateViewModel::IsTemplate() const
{
	return true;
}

void FWidgetTemplateViewModel::GetFilterStrings(TArray<FString>& OutStrings) const
{
	Template->GetFilterStrings(OutStrings);
}

TSharedRef<ITableRow> FWidgetTemplateViewModel::BuildRow(const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FWidgetViewModel>>, OwnerTable)
		.Padding(2.0f)
		.OnDragDetected(this, &FWidgetTemplateViewModel::OnDraggingWidgetTemplateItem)
		[
			SNew(SPaletteViewItem, SharedThis(this))
			.HighlightText(FavortiesViewModel, &FWidgetCatalogViewModel::GetSearchText)
		];
}

FReply FWidgetTemplateViewModel::OnDraggingWidgetTemplateItem(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().BeginDragDrop(FWidgetTemplateDragDropOp::New(Template));
}

void FWidgetTemplateViewModel::AddToFavorites()
{
	bIsFavorite = true;
	FavortiesViewModel->AddToFavorites(this);
}

void FWidgetTemplateViewModel::RemoveFromFavorites()
{
	bIsFavorite = false;
	FavortiesViewModel->RemoveFromFavorites(this);
}

TSharedRef<ITableRow> FWidgetHeaderViewModel::BuildRow(const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FWidgetViewModel>>, OwnerTable)
		.Style(FAppStyle::Get(), "UMGEditor.PaletteHeader")
		.Padding(5.0f)
		.ShowSelection(false)
		[
			SNew(STextBlock)
			.TransformPolicy(ETextTransformPolicy::ToUpper)
			.Text(GroupName)
			.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
		];
}

void FWidgetHeaderViewModel::GetChildren(TArray< TSharedPtr<FWidgetViewModel> >& OutChildren)
{
	for (TSharedPtr<FWidgetViewModel>& Child : Children)
	{
		OutChildren.Add(Child);
	}
}

FWidgetCatalogViewModel::FWidgetCatalogViewModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: bRebuildRequested(true)
{
	BlueprintEditor = InBlueprintEditor;

	FavoriteHeader = MakeShareable(new FWidgetHeaderViewModel());
	FavoriteHeader->GroupName = LOCTEXT("Favorites", "Favorites");
}

void FWidgetCatalogViewModel::RegisterToEvents()
{
	// Register for events that can trigger a palette rebuild
	GEditor->OnBlueprintReinstanced().AddRaw(this, &FWidgetCatalogViewModel::OnBlueprintReinstanced);
	FEditorDelegates::OnAssetsDeleted.AddSP(this, &FWidgetCatalogViewModel::HandleOnAssetsDeleted);
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddSP(this, &FWidgetCatalogViewModel::OnReloadComplete);

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FWidgetCatalogViewModel::OnObjectsReplaced);

	// Register for favorite list update to handle the case where a favorite is added in another window of the UMG Designer
	UWidgetPaletteFavorites* Favorites = GetDefault<UWidgetDesignerSettings>()->Favorites;
	Favorites->OnFavoritesUpdated.AddSP(this, &FWidgetCatalogViewModel::OnFavoritesUpdated);
}

FWidgetCatalogViewModel::~FWidgetCatalogViewModel()
{
	GEditor->OnBlueprintReinstanced().RemoveAll(this);
	FEditorDelegates::OnAssetsDeleted.RemoveAll(this);
	FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	UWidgetPaletteFavorites* Favorites = GetDefault<UWidgetDesignerSettings>()->Favorites;
	Favorites->OnFavoritesUpdated.RemoveAll(this);
}

void FWidgetCatalogViewModel::Update()
{
	if (bRebuildRequested)
	{
		OnUpdating.Broadcast();
		BuildWidgetList();
		bRebuildRequested = false;
		OnUpdated.Broadcast();
	}
}

UWidgetBlueprint* FWidgetCatalogViewModel::GetBlueprint() const
{
	if (BlueprintEditor.IsValid())
	{
		UBlueprint* BP = BlueprintEditor.Pin()->GetBlueprintObj();
		return Cast<UWidgetBlueprint>(BP);
	}

	return NULL;
}

void FWidgetCatalogViewModel::BuildWidgetList()
{
	// Clear the current list of view models and categories
	WidgetViewModels.Reset();
	WidgetTemplateCategories.Reset();

	// Generate a list of templates
	BuildClassWidgetList();

	// Clear the Favorite section
	bool bHasFavorites = FavoriteHeader->Children.Num() != 0;
	FavoriteHeader->Children.Reset();
	
	// Build ViewModel and clean the Favorite list if needed
	{
		// Copy of the list of favorites to be able to do some cleanup in the real list
		UWidgetPaletteFavorites* FavoritesPalette = GetDefault<UWidgetDesignerSettings>()->Favorites;
		TArray<FString> FavoritesList = FavoritesPalette->GetFavorites();

		// For each entry in the category create a view model for the widget template
		for (auto& Entry : WidgetTemplateCategories)
		{
			BuildWidgetTemplateCategory(Entry.Key, Entry.Value, FavoritesList);
		}

		// Remove all Favorites that may be left in the list.Typically happening when the list of favorite contains widget that were deleted since the last opening.
		for (const FString& FavoriteName : FavoritesList)
		{
			FavoritesPalette->Remove(FavoriteName);
		}
	}

	// Sort the view models by name
	WidgetViewModels.Sort([] (TSharedPtr<FWidgetViewModel> L, TSharedPtr<FWidgetViewModel> R) { return R->GetName().CompareTo(L->GetName()) > 0; });

	// Add the Favorite section at the top
	if (FavoriteHeader->Children.Num() != 0)
	{
		// We force expansion of the favorite header when we add favorites for the first time.
		FavoriteHeader->SetForceExpansion(!bHasFavorites);
		FavoriteHeader->Children.Sort([](TSharedPtr<FWidgetViewModel> L, TSharedPtr<FWidgetViewModel> R) { return R->GetName().CompareTo(L->GetName()) > 0; });
		WidgetViewModels.Insert(FavoriteHeader, 0);
	}
	
	// Take the Advanced Section, and put it at the end.
	TSharedPtr<FWidgetViewModel>* advancedSectionPtr = WidgetViewModels.FindByPredicate([](TSharedPtr<FWidgetViewModel> widget) {return widget->GetName().CompareTo(LOCTEXT("Advanced", "Advanced")) == 0; });
	if (advancedSectionPtr)
	{
		TSharedPtr<FWidgetViewModel> advancedSection = *advancedSectionPtr;
		WidgetViewModels.Remove(advancedSection);
		WidgetViewModels.Push(advancedSection);
	}
}

void FWidgetCatalogViewModel::BuildClassWidgetList()
{
	const UClass* ActiveWidgetBlueprintClass = GetBlueprint()->GeneratedClass;
	FName ActiveWidgetBlueprintClassName = ActiveWidgetBlueprintClass->GetFName();
	TSharedPtr<FWidgetBlueprintEditor> PinnedBPEditor = BlueprintEditor.Pin();

	if (!PinnedBPEditor)
	{
		return;
	}

	// Locate all UWidget classes from code and loaded widget BPs
	TArray<UClass*> Widgets;
	GetDerivedClasses(UWidget::StaticClass(), Widgets);
	TSet<FName> ProcessedPackages;
	for (UClass* WidgetClass : Widgets)
	{
		const bool bIsSameClass = WidgetClass->GetFName() == ActiveWidgetBlueprintClassName;
		if (bIsSameClass)
		{
			continue;
		}
		
		// even if we reject the widget, consider its package processed:
		ProcessedPackages.Add(WidgetClass->GetPackage()->GetFName());
		if (!FWidgetBlueprintEditorUtils::IsUsableWidgetClass(WidgetClass, PinnedBPEditor.ToSharedRef()))
		{
			continue;
		}

		if (WidgetClass->HasAnyClassFlags(CLASS_HideDropDown | CLASS_Hidden))
		{
			continue;
		}

		if (WidgetClass->IsChildOf(UUserWidget::StaticClass()))
		{
			AddWidgetTemplate(MakeShared<FWidgetTemplateBlueprintClass>(FAssetData(WidgetClass), WidgetClass));
		}
		else
		{
			// For UWidget
			AddWidgetTemplate(MakeShared<FWidgetTemplateClass>(WidgetClass));
		}
	}

	// Locate all UWidget BP assets, include loaded and unloaded. Only parsed the unloaded.
	const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (!AssetRegistryModule)
	{
		return;
	}
	
	TArray<FAssetData> AllWidgetClassAssetData;
	UAssetRegistryHelpers::GetDerivedClassAssetData(
		{
			UWidget::StaticClass()->GetClassPathName()
		},
		AllWidgetClassAssetData
	);
	
	// for a large working set with thousands of entries in AllWidgetClassAssetData
	// this loop costs us 100ms - acceptable for editor performance. Much of the
	// time is spent finding the native parent class, which we could probably do
	// more efficiently.
	for (const FAssetData& BPAssetData : AllWidgetClassAssetData)
	{
		const bool bIsSameClass = BPAssetData.AssetName == ActiveWidgetBlueprintClassName;
		if (bIsSameClass)
		{
			continue;
		}

		// Was already parsed by the GetDerivedClasses(UWidget::StaticClass())
		if (ProcessedPackages.Contains(BPAssetData.PackageName))
		{
			continue;
		}

		ProcessedPackages.Add(BPAssetData.PackageName);
		TValueOrError<FWidgetBlueprintEditorUtils::FUsableWidgetClassResult, void> Usable = FWidgetBlueprintEditorUtils::IsUsableWidgetClass(BPAssetData, PinnedBPEditor.ToSharedRef());
		if (Usable.HasError())
		{
			continue;
		}

		if ((Usable.GetValue().AssetClassFlags & (CLASS_Hidden | CLASS_HideDropDown)) != 0)
		{
			continue;
		}		

		if (Usable.GetValue().NativeParentClass->IsChildOf(UUserWidget::StaticClass()))
		{
			AddWidgetTemplate(MakeShared<FWidgetTemplateBlueprintClass>(BPAssetData, nullptr));
		}
		else
		{
			AddWidgetTemplate(MakeShared<FWidgetTemplateClass>(BPAssetData, nullptr));
		}
	}

	TArray<FAssetData> AllGeneratedBPsAssetData; // if it's a widget already compiled

	// this logic will take >500ms in a large project, but only if bUseEditorConfigPaletteFiltering is enabled
	// It likely can be replaced with a call to UAssetRegistryHelpers::GetDerivedClassAssetData
	// unfortunately at least for loose cooked classes that function does not work, so i'm leaving 
	// this in place for users relying on bUseEditorConfigPaletteFiltering
	if (FWidgetBlueprintEditorUtils::GetRelevantSettings(BlueprintEditor)->bUseEditorConfigPaletteFiltering)
	{
		const IAssetRegistry& AssetRegistry = AssetRegistryModule->Get();
		AssetRegistry.GetAssetsByClass(UBlueprintGeneratedClass::StaticClass()->GetClassPathName(), AllGeneratedBPsAssetData, true);
	}

	for (const FAssetData& BPAssetData : AllGeneratedBPsAssetData)
	{
		const bool bIsSameClass = BPAssetData.AssetName == ActiveWidgetBlueprintClassName;
		if (bIsSameClass)
		{
			continue;
		}

		// Was already parsed by the TObjectIterator<UClass>
		if (BPAssetData.IsAssetLoaded())
		{
			continue;
		}

		TValueOrError<FWidgetBlueprintEditorUtils::FUsableWidgetClassResult, void> Usable = FWidgetBlueprintEditorUtils::IsUsableWidgetClass(BPAssetData, PinnedBPEditor.ToSharedRef());
		if (Usable.HasError())
		{
			continue;
		}

		if ((Usable.GetValue().AssetClassFlags & (CLASS_Hidden | CLASS_HideDropDown)) != 0)
		{
			continue;
		}

		AddWidgetTemplate(MakeShared<FWidgetTemplateBlueprintClass>(BPAssetData, nullptr));
	}
}

void FWidgetCatalogViewModel::AddHeader(TSharedPtr<FWidgetHeaderViewModel>& Header)
{
	WidgetViewModels.Add(Header);
}

void FWidgetCatalogViewModel::AddToFavoriteHeader(TSharedPtr<FWidgetTemplateViewModel>& Favorite)
{
	if (FavoriteHeader)
	{
		FavoriteHeader->Children.Add(Favorite);
	}
}

void FWidgetCatalogViewModel::AddWidgetTemplate(TSharedPtr<FWidgetTemplate> Template)
{
	FString Category = *Template->GetCategory().ToString();

	// Hide user specific categories
	const TArray<FString>& CategoriesToHide = FWidgetBlueprintEditorUtils::GetRelevantSettings(BlueprintEditor)->CategoriesToHide;
	
	for (const FString& CategoryName : CategoriesToHide)
	{
		if (Category == CategoryName)
		{
			return;
		}
	}
	WidgetTemplateArray& Group = WidgetTemplateCategories.FindOrAdd(Category);
	Group.Add(Template);
}

void FWidgetCatalogViewModel::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
}

void FWidgetCatalogViewModel::OnBlueprintReinstanced()
{
	bRebuildRequested = true;
}

void FWidgetCatalogViewModel::OnFavoritesUpdated()
{
	bRebuildRequested = true;
}

void FWidgetCatalogViewModel::OnReloadComplete(EReloadCompleteReason Reason)
{
	bRebuildRequested = true;
}

void FWidgetCatalogViewModel::HandleOnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses)
{
	for (const UClass* DeletedAssetClass : DeletedAssetClasses)
	{
		if ((DeletedAssetClass == nullptr) || DeletedAssetClass->IsChildOf(UWidgetBlueprint::StaticClass()))
		{
			bRebuildRequested = true;
		}
	}
}

void FPaletteViewModel::BuildWidgetTemplateCategory(FString& Category, TArray<TSharedPtr<FWidgetTemplate>>& Templates, TArray<FString>& FavoritesList)
{
	TSharedPtr<FWidgetHeaderViewModel> Header = MakeShareable(new FWidgetHeaderViewModel());
	Header->GroupName = FText::FromString(Category);
	for (auto& Template : Templates)
	{
		TSharedPtr<FWidgetTemplateViewModel> TemplateViewModel = MakeShareable(new FWidgetTemplateViewModel());
		TemplateViewModel->Template = Template;
		TemplateViewModel->FavortiesViewModel = this;
		Header->Children.Add(TemplateViewModel);

		// If it's a favorite, we also add it to the Favorite section
		int32 index = FavoritesList.Find(Template->Name.ToString());
		if (index != INDEX_NONE)
		{
			TemplateViewModel->SetFavorite();

			// We have to create a second copy of the ViewModel for the treeview has it doesn't support to have the same element twice.
			TSharedPtr<FWidgetTemplateViewModel> FavoriteTemplateViewModel = MakeShareable(new FWidgetTemplateViewModel());
			FavoriteTemplateViewModel->Template = Template;
			FavoriteTemplateViewModel->FavortiesViewModel = this;
			FavoriteTemplateViewModel->SetFavorite();

			AddToFavoriteHeader(FavoriteTemplateViewModel);

			// Remove the favorite from the temporary list
			FavoritesList.RemoveAt(index);
		}

	}
	
	Header->Children.Sort([](const TSharedPtr<FWidgetViewModel>& L, const TSharedPtr<FWidgetViewModel>& R) { return R->GetName().CompareTo(L->GetName()) > 0; });

	AddHeader(Header);
}

void FWidgetCatalogViewModel::AddToFavorites(const FWidgetTemplateViewModel* WidgetTemplateViewModel)
{
	UWidgetPaletteFavorites* Favorites = GetDefault<UWidgetDesignerSettings>()->Favorites;
	Favorites->Add(WidgetTemplateViewModel->GetName().ToString());
}

void FWidgetCatalogViewModel::RemoveFromFavorites(const FWidgetTemplateViewModel* WidgetTemplateViewModel)
{
	UWidgetPaletteFavorites* Favorites = GetDefault<UWidgetDesignerSettings>()->Favorites;
	Favorites->Remove(WidgetTemplateViewModel->GetName().ToString());
}

#undef LOCTEXT_NAMESPACE
