// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetBlueprintEditor.h"
#include "AssetRegistry/AssetData.h"

#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FWidgetTemplate;
class FWidgetBlueprintEditor;
class UWidgetBlueprint;
class SPaletteView;
class FWidgetCatalogViewModel;

/** View model for the items in the widget template list */
class FWidgetViewModel : public TSharedFromThis<FWidgetViewModel>
{
public:
	virtual ~FWidgetViewModel() = default;

	virtual FText GetName() const = 0;

	virtual bool IsTemplate() const = 0;

	virtual bool IsCategory() const
	{
		return false;
	}

	/** @param OutStrings - Returns an array of strings used for filtering/searching this item. */
	virtual void GetFilterStrings(TArray<FString>& OutStrings) const = 0;

	virtual bool HasFilteredChildTemplates() const
	{
		return false;
	}

	virtual TSharedRef<ITableRow> BuildRow(const TSharedRef<STableViewBase>& OwnerTable) = 0;

	virtual void GetChildren(TArray< TSharedPtr<FWidgetViewModel> >& OutChildren)
	{
	}

	/** Return true if the widget is a favorite */
	virtual bool IsFavorite() const { return false; }

	/** Set the favorite flag */
	virtual void SetFavorite()
	{
	}

	virtual bool ShouldForceExpansion() const { return false; }
};

class FWidgetTemplateViewModel : public FWidgetViewModel
{
public:
	FWidgetTemplateViewModel();

	virtual FText GetName() const override;

	virtual bool IsTemplate() const override;

	virtual void GetFilterStrings(TArray<FString>& OutStrings) const override;

	virtual TSharedRef<ITableRow> BuildRow(const TSharedRef<STableViewBase>& OwnerTable) override;

	FReply OnDraggingWidgetTemplateItem(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Add the widget template to the list of favorites */
	void AddToFavorites();

	/** Remove the widget template from the list of favorites */
	void RemoveFromFavorites();

	/** Return true if the widget is a favorite */
	virtual bool IsFavorite() const override { return bIsFavorite; }

	/** Set the favorite flag */
	virtual void SetFavorite() override { bIsFavorite = true; }

	TSharedPtr<FWidgetTemplate> Template;
	FWidgetCatalogViewModel* FavortiesViewModel;
private:
	/** True is the widget is a favorite. It's keep as a state to prevent a search in the favorite list. */
	bool bIsFavorite;
};

class FWidgetHeaderViewModel : public FWidgetViewModel
{
public:
	virtual ~FWidgetHeaderViewModel() = default;

	virtual FText GetName() const override
	{
		return GroupName;
	}

	virtual bool IsTemplate() const override
	{
		return false;
	}

	virtual bool IsCategory() const override
	{
		return true;
	}

	virtual void GetFilterStrings(TArray<FString>& OutStrings) const override
	{
		// Headers should never be included in filtering to avoid showing a header with all of
		// it's widgets filtered out, so return an empty filter string.
	}

	virtual bool HasFilteredChildTemplates() const override
	{
		for (const TSharedPtr<FWidgetViewModel>& Child : Children)
		{
			if (Child && Child->HasFilteredChildTemplates())
			{
				return true;
			}
		}
		return false;
	}

	virtual TSharedRef<ITableRow> BuildRow(const TSharedRef<STableViewBase>& OwnerTable) override;

	virtual void GetChildren(TArray< TSharedPtr<FWidgetViewModel> >& OutChildren) override;

	virtual bool ShouldForceExpansion() const { return bForceExpansion; }

	void SetForceExpansion(bool bInForceExpansion) { bForceExpansion = bInForceExpansion; }

	FText GroupName;
	TArray< TSharedPtr<FWidgetViewModel> > Children;

private:

	bool bForceExpansion = false;
};

class FWidgetCatalogViewModel : public TSharedFromThis<FWidgetCatalogViewModel>
{
public:

	DECLARE_MULTICAST_DELEGATE(FOnUpdating)
	DECLARE_MULTICAST_DELEGATE(FOnUpdated)

public:
	FWidgetCatalogViewModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~FWidgetCatalogViewModel();

	/** Register the View Model to events that should trigger a update */
	void RegisterToEvents();

	/** Update the view model if needed and returns true if it did. */
	void Update();

	/** Returns true if the view model needs to be updated */
	bool NeedUpdate() const { return bRebuildRequested; }

	/** Add the widget template to the list of favorites */
	static void AddToFavorites(const FWidgetTemplateViewModel* WidgetTemplateViewModel);

	/** Remove the widget template to the list of favorites */
	static void RemoveFromFavorites(const FWidgetTemplateViewModel* WidgetTemplateViewModel);

	typedef TArray< TSharedPtr<FWidgetViewModel> > ViewModelsArray;
	ViewModelsArray& GetWidgetViewModels() { return WidgetViewModels; }

	/** Fires before the view model is updated */
	FOnUpdating OnUpdating;

	/** Fires after the view model is updated */
	FOnUpdated OnUpdated;

	virtual void SetSearchText(const FText& InSearchText) { SearchText = InSearchText; }
	FText GetSearchText() const { return SearchText; }

protected:
	virtual void BuildWidgetList();

	void AddHeader(TSharedPtr<FWidgetHeaderViewModel>& Header);
	void AddToFavoriteHeader(TSharedPtr<FWidgetTemplateViewModel>& Favorite);

private:
	FWidgetCatalogViewModel() {};

	UWidgetBlueprint* GetBlueprint() const;
	virtual void BuildWidgetTemplateCategory(FString& Category, TArray<TSharedPtr<FWidgetTemplate>>& Templates, TArray<FString>& FavoritesList) = 0;
	void BuildClassWidgetList();

	void AddWidgetTemplate(TSharedPtr<FWidgetTemplate> Template);

	/** Called when a Blueprint is recompiled and live objects are swapped out for replacements */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Requests a rebuild of the widget list if a widget blueprint was compiled */
	void OnBlueprintReinstanced();

	/** Called when the favorite list is changed */
	void OnFavoritesUpdated();

	/** Requests a rebuild of the widget list */
	void OnReloadComplete(EReloadCompleteReason Reason);

	/** Requests a rebuild of the widget list if a widget blueprint was deleted */
	void HandleOnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses);
	
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;

	typedef TArray<TSharedPtr<FWidgetTemplate>> WidgetTemplateArray;
	TMap<FString, WidgetTemplateArray> WidgetTemplateCategories;

	/** The source root view models for the tree. */
	ViewModelsArray WidgetViewModels;
	   
	/** Controls rebuilding the list of spawnable widgets */
	bool bRebuildRequested;

	TSharedPtr<FWidgetHeaderViewModel> FavoriteHeader;

	FText SearchText;
};

class FPaletteViewModel : public FWidgetCatalogViewModel
{
public:
	FPaletteViewModel(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor) : FWidgetCatalogViewModel(InBlueprintEditor) { }

	//~ Begin FWidgetCatalogViewModel Interface
	virtual void BuildWidgetTemplateCategory(FString& Category, TArray<TSharedPtr<FWidgetTemplate>>& Templates, TArray<FString>& FavoritesList) override;
	//~ End FWidgetCatalogViewModel Interface
};

