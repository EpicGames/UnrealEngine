// Copyright Epic Games, Inc. All Rights Reserved.

#include "Preview/PreviewMode.h"
#include "Blueprint/UserWidget.h"

namespace UE::UMG::Editor
{

void FPreviewMode::SetSelectedObject(TArray<TWeakObjectPtr<UObject>> Objects)
{
	SelectedObjects = MoveTemp(Objects);
	SelectedObjectChangedDelegate.Broadcast();
}

void FPreviewMode::SetSelectedObject(TArrayView<UObject*> Objects)
{
	SelectedObjects.Reset(Objects.Num());
	for (UObject* Obj : Objects)
	{
		SelectedObjects.Emplace(Obj);
	}
	SelectedObjectChangedDelegate.Broadcast();
}

TArray<UObject*> FPreviewMode::GetSelectedObjectList() const
{
	TArray<UObject*> Result;
	Result.Reset(SelectedObjects.Num());
	for (TWeakObjectPtr<UObject> Obj : SelectedObjects)
	{
		if (UObject* Ptr = Obj.Get())
		{
			Result.Add(Ptr);
		}
	}
	return Result;
}

void FPreviewMode::SetPreviewWidget(UUserWidget* Widget)
{
	UUserWidget* Previous = PreviewWidget.Get();
	if (Previous != Widget)
	{
		PreviewWidget = Widget;
		PreviewWidgetChangedDelegate.Broadcast();
		if (SelectedObjects.ContainsByPredicate([Previous](const TWeakObjectPtr<UObject>& Other){ return Other.Get() == Previous; }))
		{
			SelectedObjects.Reset();
			SelectedObjectChangedDelegate.Broadcast();
		}
	}
}

} // namespace UE::UMG
