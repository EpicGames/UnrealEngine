// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPreviewDetails.h"

#include "Preview/PreviewMode.h"
#include "INotifyFieldValueChanged.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "WidgetBlueprintEditor.h"

#include "IDetailsView.h"

namespace UE::UMG::Editor
{

void SPreviewDetails::Construct(const FArguments& Args, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	WeakEditor = InBlueprintEditor;

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;

	DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

	TSharedPtr<FPreviewMode> Context = InBlueprintEditor && InBlueprintEditor->GetPreviewMode() ? InBlueprintEditor->GetPreviewMode() : TSharedPtr<FPreviewMode>();
	if (Context)
	{
		Context->OnSelectedObjectChanged().AddSP(this, &SPreviewDetails::HandleSelectedObjectChanged);
	}
	HandleSelectedObjectChanged();

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}


void SPreviewDetails::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{

	for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++Index)
	{
		if (const UObject* Object = PropertyChangedEvent.GetObjectBeingEdited(Index))
		{
			if (PropertyThatChanged->GetActiveMemberNode())
			{
				FProperty* Property = PropertyThatChanged->GetActiveMemberNode()->GetValue();
				if (Property && Object->GetClass()->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
				{
					TScriptInterface<INotifyFieldValueChanged> Interface = const_cast<UObject*>(Object);
					UE::FieldNotification::FFieldId FieldId = Interface->GetFieldNotificationDescriptor().GetField(Object->GetClass(), Property->GetFName());
					if (FieldId.IsValid())
					{
						Interface->BroadcastFieldValueChanged(FieldId);
					}
				}
			}
		}
	}
}


void SPreviewDetails::HandleSelectedObjectChanged()
{
	TSharedPtr<FWidgetBlueprintEditor> Editor = WeakEditor.Pin();
	if (!Editor)
	{
		return;
	}

	TSharedPtr<FPreviewMode> Context = Editor && Editor->GetPreviewMode() ? Editor->GetPreviewMode() : TSharedPtr<FPreviewMode>();
	if (Context)
	{
		TArray<UObject*> Objects = Context->GetSelectedObjectList();
		if (Objects.Num() == 0)
		{
			if (Context->GetPreviewWidget())
			{
				Objects.Add(Context->GetPreviewWidget());
			}
		}
		DetailsView->SetObjects(Objects);
	}
	else
	{
		DetailsView->SetObject(nullptr);
	}
}

} // namespace
