// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class FWidgetBlueprintEditor;

namespace UE::UMG::Editor
{

class SPreviewDetails : public SCompoundWidget, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SPreviewDetails) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);

	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

private:
	void HandleSelectedObjectChanged();

private:
	TWeakPtr<FWidgetBlueprintEditor> WeakEditor;
	TSharedPtr<IDetailsView> DetailsView;
};

} // namespace
