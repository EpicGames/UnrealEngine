// Copyright Epic Games, Inc. All Rights Reserved.

#include "Preview/SWidgetPreview.h"

#include "Blueprint/UserWidget.h"
#include "Engine/World.h"

#include "Preview/PreviewMode.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

namespace UE::UMG::Editor
{

void SWidgetPreview::Construct(const FArguments& Args, TSharedPtr<FWidgetBlueprintEditor> InWidgetBlueprintEditor)
{
	WeakEditor = InWidgetBlueprintEditor;

	TSharedPtr<SWidget> SlateWidget;

	UWidgetBlueprint* WidgetBlueprint = InWidgetBlueprintEditor->GetWidgetBlueprintObj();
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (WidgetBlueprint && WidgetBlueprint->GeneratedClass && World)
	{
		if (UWidgetBlueprintGeneratedClass* WidgetClass = Cast<UWidgetBlueprintGeneratedClass>(WidgetBlueprint->GeneratedClass))
		{
			CreatedWidget.Reset(CreateWidget(World, WidgetClass, WidgetClass->GetFName()));
			if (CreatedWidget.Get())
			{
				CreatedWidget->SetDesignerFlags(EWidgetDesignFlags::Previewing);
				SlateWidget = CreatedWidget->TakeWidget();
			}
		}
	}

	if (InWidgetBlueprintEditor->GetPreviewMode())
	{
		InWidgetBlueprintEditor->GetPreviewMode()->SetPreviewWidget(CreatedWidget.Get());
	}
	//Editor->GetDebugStuff()->SetPreviewWidget(CreateWidget);
	// if widget compiled, (it could be the parent), need to clean this up like with widget blutility

	ChildSlot
	[
		SlateWidget.IsValid() ? SlateWidget.ToSharedRef() : SNullWidget::NullWidget
	];
}

SWidgetPreview::~SWidgetPreview()
{
	if (TSharedPtr<FWidgetBlueprintEditor> Editor = WeakEditor.Pin())
	{
		if (Editor->GetPreviewMode())
		{
			Editor->GetPreviewMode()->SetPreviewWidget(nullptr);
		}
	}
}

} // namespace UE::UMG
