// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FWidgetBlueprintEditor;
class UPlayer;
class UUserWidget;

namespace UE::UMG::Editor
{

class SWidgetPreview : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWidgetPreview) {}
	SLATE_END_ARGS()

	~SWidgetPreview();

	void Construct(const FArguments& Args, TSharedPtr<FWidgetBlueprintEditor> InWidgetBlueprintEditor);

private:

	TWeakPtr<FWidgetBlueprintEditor> WeakEditor;
	TStrongObjectPtr<UUserWidget> CreatedWidget;
};

} // namespace UE::UMG