// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGEditorProjectSettings.h"
#include "WidgetBlueprint.h"
#include "WidgetCompilerRule.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include "Blueprint/UserWidget.h"
#include "Components/CanvasPanel.h"	
#include "Components/GridPanel.h"
#include "Components/HorizontalBox.h"
#include "Components/VerticalBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UMGEditorProjectSettings)

UUMGEditorProjectSettings::UUMGEditorProjectSettings()
{
	Version = 0;
	CurrentVersion = 1;
	bShowWidgetsFromEngineContent = false;
	bShowWidgetsFromDeveloperContent = true;

	bEnableMakeVariable = true;
	bEnableWidgetAnimationEditor = true;
	bEnablePaletteWindow = true;
	bEnableLibraryWindow = true;
	bEnableHierarchyWindow = true;
	bEnableBindWidgetWindow = true;
	bEnableNavigationSimulationWindow = true;
	bEnableUIComponentsProperty = true;

	bUseEditorConfigPaletteFiltering = false;
	bUseUserWidgetParentClassViewerSelector = true;
	bUseUserWidgetParentDefaultClassViewerSelector = true;

	// For compatibility with the UMG Widget Previewer this must be set to true.
	// However, we do not wish to change the value on "legacy" widgets, so this will only affect newly created ones.
	bCanCallInitializedWithoutPlayerContext = true;

	bUseWidgetTemplateSelector = false;
	CommonRootWidgetClasses = {
		UHorizontalBox::StaticClass(),
		UVerticalBox::StaticClass(),
		UGridPanel::StaticClass(),
		UCanvasPanel::StaticClass()
	};
	DefaultRootWidget = nullptr;
	FavoriteWidgetParentClasses.Add(UUserWidget::StaticClass());
}

#if WITH_EDITOR

FText UUMGEditorProjectSettings::GetSectionText() const
{
	return NSLOCTEXT("UMG", "WidgetDesignerTeamSettingsName", "Widget Designer (Team)");
}

FText UUMGEditorProjectSettings::GetSectionDescription() const
{
	return NSLOCTEXT("UMG", "WidgetDesignerTeamSettingsDescription", "Configure options for the Widget Designer that affect the whole team.");
}

#endif

#if WITH_EDITOR
void UUMGEditorProjectSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	DirectoryCompilerOptions.StableSort([](const FDirectoryWidgetCompilerOptions& A, const FDirectoryWidgetCompilerOptions& B) {
		return A.Directory.Path < B.Directory.Path;
	});

	// If there's a change, we should scan for widgets currently in the error or warning state and mark them as dirty
	// so they get recompiled next time we PIE.  Don't mark all widgets dirty, or we're in for a very large recompile.
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		for (TObjectIterator<UWidgetBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
		{
			UWidgetBlueprint* Blueprint = *BlueprintIt;
			if (Blueprint->Status == BS_Error || Blueprint->Status == BS_UpToDateWithWarnings)
			{
				Blueprint->Status = BS_Dirty;
			}
		}
	}
}
#endif
