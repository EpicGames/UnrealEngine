// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/WidgetDesignerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetDesignerSettings)

UWidgetDesignerSettings::UWidgetDesignerSettings()
{

	Favorites = CreateDefaultSubobject<UWidgetPaletteFavorites>(TEXT("WidgetPaletteFavorites"));

	CategoryName = TEXT("ContentEditors");

	GridSnapEnabled = true;
	GridSnapSize = 4;
	bShowOutlines = true;
	bExecutePreConstructEvent = true;
	bRespectLocks = true;
	CreateOnCompile = EDisplayOnCompile::DoC_ErrorsOrWarnings;
	DismissOnCompile = EDisplayOnCompile::DoC_ErrorsOrWarnings;
	DefaultPreviewResolution = FUintVector2(1280, 720);
}

#if WITH_EDITOR

FText UWidgetDesignerSettings::GetSectionText() const
{
	return NSLOCTEXT("UMG", "WidgetDesignerSettingsName", "Widget Designer");
}

FText UWidgetDesignerSettings::GetSectionDescription() const
{
	return NSLOCTEXT("UMG", "WidgetDesignerSettingsDescription", "Configure options for the Widget Designer.");
}

#endif
