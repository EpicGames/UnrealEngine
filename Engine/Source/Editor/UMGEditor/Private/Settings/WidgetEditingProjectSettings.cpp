// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditingProjectSettings.h"
#include "WidgetBlueprint.h"
#include "WidgetCompilerRule.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include "Blueprint/UserWidget.h"
#include "Components/CanvasPanel.h"	
#include "Components/GridPanel.h"
#include "Components/HorizontalBox.h"
#include "Components/VerticalBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetEditingProjectSettings)

UWidgetEditingProjectSettings::UWidgetEditingProjectSettings()
{
	Version = 0;
	CurrentVersion = 1;

	bEnableMakeVariable = true;
	bEnableWidgetAnimationEditor = true;
	bEnablePaletteWindow = true;
	bEnableLibraryWindow = true;
	bEnableHierarchyWindow = true;
	bEnableBindWidgetWindow = true;
	bEnableNavigationSimulationWindow = true;
	bEnableUIComponentsProperty = true;

	bUseEditorConfigPaletteFiltering = false;
	bUseUserWidgetParentClassViewerSelector = true;
	bUseUserWidgetParentDefaultClassViewerSelector = true;

	DefaultWidgetAnimationFrameRate = 20;

	bUseWidgetTemplateSelector = false;
}

bool UWidgetEditingProjectSettings::CompilerOption_AllowBlueprintTick(const class UWidgetBlueprint* WidgetBlueprint) const
{
	return GetFirstCompilerOption(WidgetBlueprint, &FWidgetCompilerOptions::bAllowBlueprintTick, true);
}

bool UWidgetEditingProjectSettings::CompilerOption_AllowBlueprintPaint(const class UWidgetBlueprint* WidgetBlueprint) const
{
	return GetFirstCompilerOption(WidgetBlueprint, &FWidgetCompilerOptions::bAllowBlueprintPaint, true);
}

EPropertyBindingPermissionLevel UWidgetEditingProjectSettings::CompilerOption_PropertyBindingRule(const class UWidgetBlueprint* WidgetBlueprint) const
{
	return GetFirstCompilerOption(WidgetBlueprint, &FWidgetCompilerOptions::PropertyBindingRule, EPropertyBindingPermissionLevel::Allow);
}

TArray<UWidgetCompilerRule*> UWidgetEditingProjectSettings::CompilerOption_Rules(const class UWidgetBlueprint* WidgetBlueprint) const
{
	TArray<UWidgetCompilerRule*> Rules;
	GetCompilerOptionsForWidget(WidgetBlueprint, [&Rules](const FWidgetCompilerOptions& Options) {
		for (const TSoftClassPtr<UWidgetCompilerRule>& RuleClassPtr : Options.Rules)
		{
			// The compiling rule may not be loaded yet in early loading phases, we'll
			// just have to skip the rules in those cases.
			RuleClassPtr.LoadSynchronous(LOAD_Quiet);
			if (RuleClassPtr)
			{
				if (UWidgetCompilerRule* Rule = RuleClassPtr->GetDefaultObject<UWidgetCompilerRule>())
				{
					Rules.Add(Rule);
				}
			}
		}
		return false;
	});
	return Rules;
}

FNamePermissionList& UWidgetEditingProjectSettings::GetAllowedPaletteCategories()
{
	return AllowedPaletteCategories;
}

const FNamePermissionList& UWidgetEditingProjectSettings::GetAllowedPaletteCategories() const
{
	return AllowedPaletteCategories;
}

FPathPermissionList& UWidgetEditingProjectSettings::GetAllowedPaletteWidgets()
{
	return AllowedPaletteWidgets;
}

const FPathPermissionList& UWidgetEditingProjectSettings::GetAllowedPaletteWidgets() const
{
	return AllowedPaletteWidgets;
}

void UWidgetEditingProjectSettings::GetCompilerOptionsForWidget(const UWidgetBlueprint* WidgetBlueprint, TFunctionRef<bool(const FWidgetCompilerOptions&)> Operator) const
{
	FString AssetPath = WidgetBlueprint->GetOutermost()->GetName();
	FSoftObjectPath SoftObjectPath = WidgetBlueprint->GetPathName();
	
	// Don't apply the rules to the engine widgets.
	if (AssetPath.StartsWith(TEXT("/Engine")))
	{
		return;
	}

	for (int32 DirectoryIndex = DirectoryCompilerOptions.Num() - 1; DirectoryIndex >= 0; DirectoryIndex--)
	{
		const FDirectoryWidgetCompilerOptions& CompilerOptions = DirectoryCompilerOptions[DirectoryIndex];

		const FString& DirectoryPath = CompilerOptions.Directory.Path;
		if (!DirectoryPath.IsEmpty())
		{
			if (AssetPath.StartsWith(DirectoryPath))
			{
				const bool bIgnoreWidget = CompilerOptions.IgnoredWidgets.ContainsByPredicate([&SoftObjectPath](const TSoftObjectPtr<UWidgetBlueprint>& IgnoredWidget) {
					return IgnoredWidget.ToSoftObjectPath() == SoftObjectPath;
				});

				if (bIgnoreWidget)
				{
					continue;
				}

				if (Operator(CompilerOptions.Options))
				{
					return;
				}
			}
		}
	}

	Operator(DefaultCompilerOptions);
}

void UWidgetEditingProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (Version < CurrentVersion)
	{
		for (int32 FromVersion = Version + 1; FromVersion <= CurrentVersion; FromVersion++)
		{
			PerformUpgradeStepForVersion(FromVersion);
		}

		Version = CurrentVersion;
	}
}

void UWidgetEditingProjectSettings::PerformUpgradeStepForVersion(int32 ForVersion)
{
}
