// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateVectorArtDataFactory.h"

#include "Misc/AssertionMacros.h"
#include "Slate/SlateVectorArtData.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateVectorArtDataFactory)

class FFeedbackContext;
class UClass;
class UObject;

USlateVectorArtDataFactory::USlateVectorArtDataFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USlateVectorArtData::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* USlateVectorArtDataFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class == USlateVectorArtData::StaticClass());

	USlateVectorArtData* SlateVectorArtData = NewObject<USlateVectorArtData>(InParent, Name, Flags);

	return SlateVectorArtData;
}
