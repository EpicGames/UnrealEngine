// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/AnimationTabSummoner.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieScene.h"
#include "Animation/WidgetAnimation.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "Blueprint/WidgetTree.h"

#include "UMGEditorActions.h"
#include "UMGEditorProjectSettings.h"
#include "UMGStyle.h"
#include "Widgets/Input/SSearchBox.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/TextFilter.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "SPositiveActionButton.h"
#include "GraphEditorActions.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Engine/MemberReference.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Editor
{
	extern bool bDisplayWidgetVariableGuids;
}

const FName FAnimationTabSummoner::TabID(TEXT("Animations"));
const FName FAnimationTabSummoner::WidgetAnimSequencerDrawerID(TEXT("WidgetAnimSequencer"));

FAnimationTabSummoner::FAnimationTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor, bool bInIsDrawerTab)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
		, bIsDrawerTab(bInIsDrawerTab)
{
	TabLabel = LOCTEXT("AnimationsTabLabel", "Animations");
	TabIcon = FSlateIcon(FUMGStyle::GetStyleSetName(), "Animations.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("Animations_ViewMenu_Desc", "Animations");
	ViewMenuTooltip = LOCTEXT("Animations_ViewMenu_ToolTip", "Opens a tab to manage animations");
}


bool VerifyAnimationRename( FWidgetBlueprintEditor& BlueprintEditor, UWidgetAnimation* Animation, FString NewAnimationName, FText& OutErrorMessage )
{
	if (NewAnimationName != SlugStringForValidName(NewAnimationName))
	{
		FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS;
		FString CurrentInvalidCharacter;
		FString FoundInvalidCharacters;

		// Create a string with all invalid characters found to output with the error message.
		for (int32 StringIndex = 0; StringIndex < InvalidCharacters.Len(); ++StringIndex)
		{
			CurrentInvalidCharacter = InvalidCharacters.Mid(StringIndex, 1);

			if (NewAnimationName.Contains(CurrentInvalidCharacter))
			{
				FoundInvalidCharacters += CurrentInvalidCharacter;
			}
		}
		
		OutErrorMessage = FText::Format(LOCTEXT("NameContainsInvalidCharacters", "The object name may not contain the following characters:  {0}"), FText::FromString(FoundInvalidCharacters));
		return false;
	}

	UWidgetBlueprint* Blueprint = BlueprintEditor.GetWidgetBlueprintObj();
	if (Blueprint && FindObject<UWidgetAnimation>( Blueprint, *NewAnimationName, EFindObjectFlags::ExactClass ) )
	{
		OutErrorMessage = LOCTEXT( "NameInUseByAnimation", "An animation with this name already exists" );
		return false;
	}

	FName NewAnimationNameAsName( *NewAnimationName );
	if ( Blueprint && Blueprint->WidgetTree->FindWidget<UWidget>( NewAnimationNameAsName ) != nullptr )
	{
		OutErrorMessage = LOCTEXT( "NameInUseByWidget", "A widget with this name already exists" );
		return false;
	}

	UUserWidget* PreviewWidget = BlueprintEditor.GetPreview();
	FName FunctionName(*NewAnimationName);
	if (PreviewWidget && PreviewWidget->FindFunction(FunctionName))
	{
		OutErrorMessage = LOCTEXT("NameInUseByFunction", "A function with this name already exists");
		return false;
	}

	// Check for BindWidgetAnim property
	if (Blueprint)
	{
		FProperty* Property = Blueprint->ParentClass->FindPropertyByName(NewAnimationNameAsName);
		if (Property && FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(Property))
		{
			return true;
		}
	}

	FKismetNameValidator Validator( Blueprint );
	EValidatorResult ValidationResult = Validator.IsValid( NewAnimationName );

	if ( ValidationResult != EValidatorResult::Ok )
	{
		FString ErrorString = FKismetNameValidator::GetErrorString( NewAnimationName, ValidationResult );
		OutErrorMessage = FText::FromString( ErrorString );
		return false;
	}

	return true;
}


struct FWidgetAnimationListItem
{
	FWidgetAnimationListItem(UWidgetAnimation* InAnimation, bool bInRenameRequestPending = false, bool bInNewAnimation = false )
		: Animation(InAnimation)
		, bRenameRequestPending(bInRenameRequestPending)
		, bNewAnimation( bInNewAnimation )
	{}

	UWidgetAnimation* Animation;
	bool bRenameRequestPending;
	bool bNewAnimation;
};

/**
 * This drag drop operation allows us to move around animations in the widget tree
 */
class FWidgetAnimationDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWidgetAnimationDragDropOp, FDecoratedDragDropOp)

	/** The template to create an instance */
	TSharedPtr<FWidgetAnimationListItem> ListItem;

	/** Constructs the drag drop operation */
	static TSharedRef<FWidgetAnimationDragDropOp> New(const TSharedPtr<FWidgetAnimationListItem>& InListItem, FText InDragText)
	{
		TSharedRef<FWidgetAnimationDragDropOp> Operation = MakeShared<FWidgetAnimationDragDropOp>();
		Operation->ListItem = InListItem;
		Operation->DefaultHoverText = InDragText;
		Operation->CurrentHoverText = InDragText;
		Operation->Construct();

		return Operation;
	}
};

typedef SListView<TSharedPtr<FWidgetAnimationListItem> > SWidgetAnimationListView;

class SWidgetAnimationListItem : public STableRow<TSharedPtr<FWidgetAnimationListItem> >
{
public:
	SLATE_BEGIN_ARGS( SWidgetAnimationListItem ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, TSharedPtr<FWidgetAnimationListItem> InListItem )
	{
		ListItem = InListItem;
		BlueprintEditor = InBlueprintEditor;

		STableRow<TSharedPtr<FWidgetAnimationListItem>>::Construct(
			STableRow<TSharedPtr<FWidgetAnimationListItem>>::FArguments()
			.Padding(FMargin(3.0f, 2.0f))
			.OnDragDetected(this, &SWidgetAnimationListItem::OnDragDetected)
			.OnCanAcceptDrop(this, &SWidgetAnimationListItem::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SWidgetAnimationListItem::OnAcceptDrop)
			.Content()
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
				.Text(this, &SWidgetAnimationListItem::GetMovieSceneText)
				.ToolTipText(this, &SWidgetAnimationListItem::GetMovieSceneToolTipText)
				//.HighlightText(InArgs._HighlightText)
				.OnVerifyTextChanged(this, &SWidgetAnimationListItem::OnVerifyNameTextChanged)
				.OnTextCommitted(this, &SWidgetAnimationListItem::OnNameTextCommited)
				.IsSelected(this, &SWidgetAnimationListItem::IsSelectedExclusively)
			],
			InOwnerTableView);
	}

	void BeginRename()
	{
		InlineTextBlock->EnterEditingMode();
	}

private:
	FText GetMovieSceneText() const
	{
		if( ListItem.IsValid() )
		{
			return ListItem.Pin()->Animation->GetDisplayName();
		}

		return FText::GetEmpty();
	}

	FText GetMovieSceneToolTipText() const
	{
		if (UE::UMG::Editor::bDisplayWidgetVariableGuids && ListItem.IsValid() && BlueprintEditor.IsValid())
		{
			const UWidgetAnimation* WidgetAnimation = ListItem.Pin()->Animation;
			const UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();
			if (WidgetAnimation && Blueprint)
			{
				static const FTextFormat VariableGUIDFormat = LOCTEXT("WidgetVariableGUIDFormat", "GUID: {0}");
				const FGuid VariableGuid = Blueprint->WidgetVariableNameToGuidMap.FindRef(WidgetAnimation->GetFName());
				return FText::Format(VariableGUIDFormat, FText::FromString(VariableGuid.ToString(EGuidFormats::DigitsWithHyphens)));
			}
		}

		return FText::GetEmpty();
	}

	bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
	{
		UWidgetAnimation* Animation = ListItem.Pin()->Animation;

		const FName NewName = *InText.ToString().Left(NAME_SIZE - 1);

		if ( Animation->GetFName() != NewName )
		{
			TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();
			return Editor.IsValid() && VerifyAnimationRename( *Editor, Animation, NewName.ToString(), OutErrorMessage );
		}

		return true;
	}

	void OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
	{
		UWidgetAnimation* WidgetAnimation = ListItem.Pin()->Animation;
		UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();

		// Name has already been checked in VerifyAnimationRename
		const FName NewFName = *InText.ToString();
		const FName OldFName = WidgetAnimation->GetFName();

		FObjectPropertyBase* ExistingProperty = CastField<FObjectPropertyBase>(Blueprint->ParentClass->FindPropertyByName(NewFName));
		const bool bBindWidgetAnim = ExistingProperty && FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(ExistingProperty) && ExistingProperty->PropertyClass->IsChildOf(UWidgetAnimation::StaticClass());

		const bool bValidName = !OldFName.IsEqual(NewFName) && !InText.IsEmpty();
		const bool bCanRename = (bValidName || bBindWidgetAnim);

		const bool bNewAnimation = ListItem.Pin()->bNewAnimation;
		if (bCanRename)
		{
			const FString NewNameStr = NewFName.ToString();
			const FString OldNameStr = OldFName.ToString();

			FText TransactionName = bNewAnimation ? LOCTEXT("NewAnimation", "New Animation") : LOCTEXT("RenameAnimation", "Rename Animation");
			{
				const FScopedTransaction Transaction(TransactionName);
				WidgetAnimation->Modify();
				WidgetAnimation->GetMovieScene()->Modify();

				WidgetAnimation->SetDisplayLabel(InText.ToString());
				WidgetAnimation->Rename(*NewNameStr);
				WidgetAnimation->GetMovieScene()->Rename(*NewNameStr);

				if (bNewAnimation)
				{
					Blueprint->Modify();
					Blueprint->Animations.Add(WidgetAnimation);
					Blueprint->OnVariableAdded(WidgetAnimation->GetFName());
					ListItem.Pin()->bNewAnimation = false;

					if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
					{
						WidgetBlueprintEditorPin->NotifyWidgetAnimListChanged();
					}
				}
				else
				{
					Blueprint->OnVariableRenamed(OldFName, NewFName);
				}
			}

			FBlueprintEditorUtils::ReplaceVariableReferences(Blueprint, OldFName, NewFName);
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
		else if (bNewAnimation)
		{
			const FScopedTransaction Transaction(LOCTEXT("NewAnimation", "New Animation"));
			Blueprint->Modify();
			Blueprint->Animations.Add(WidgetAnimation);
			Blueprint->OnVariableAdded(WidgetAnimation->GetFName());
			ListItem.Pin()->bNewAnimation = false;
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

			if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
			{
				WidgetBlueprintEditorPin->NotifyWidgetAnimListChanged();
			}
		}
	}

	/** Called whenever a drag is detected by the tree view. */
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
	{
		TSharedPtr<FWidgetAnimationListItem> ListItemPinned = ListItem.Pin();
		if (ListItemPinned.IsValid())
		{
			FText DefaultText = LOCTEXT("DefaultDragDropFormat", "Move 1 item(s)");
			return FReply::Handled().BeginDragDrop(FWidgetAnimationDragDropOp::New(ListItemPinned, DefaultText));
		}
		return FReply::Unhandled();
	}

	/** Called to determine whether a current drag operation is valid for this row. */
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FWidgetAnimationListItem> InListItem)
	{
		TSharedPtr<FWidgetAnimationDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FWidgetAnimationDragDropOp>();
		if (DragDropOp.IsValid())
		{
			if (InItemDropZone == EItemDropZone::OntoItem)
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			}
			else
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Ok"));
			}
			return InItemDropZone;
		}
		return TOptional<EItemDropZone>();
	}

	/** Called to complete a drag and drop onto this drop. */
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FWidgetAnimationListItem> InListItem)
	{
		TSharedPtr<FWidgetAnimationDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FWidgetAnimationDragDropOp>();
		if (DragDropOp.IsValid() 
			&& DragDropOp->ListItem.IsValid() && DragDropOp->ListItem->Animation
			&& InListItem.IsValid() && InListItem->Animation
			&& DragDropOp->ListItem->Animation != InListItem->Animation)
		{
			if (UWidgetBlueprint* Blueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj())
			{
				Blueprint->Modify();
				Blueprint->Animations.Remove(DragDropOp->ListItem->Animation);

				int32 RelativeNewIndex = Blueprint->Animations.IndexOfByKey(InListItem->Animation);
				RelativeNewIndex += InItemDropZone == EItemDropZone::BelowItem ? 1 : 0;

				Blueprint->Animations.Insert(DragDropOp->ListItem->Animation, RelativeNewIndex);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

				if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
				{
					WidgetBlueprintEditorPin->NotifyWidgetAnimListChanged();
				}
				return FReply::Handled();
			}
		}
		return FReply::Unhandled();
	}

private:
	TWeakPtr<FWidgetAnimationListItem> ListItem;
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;
};


class SUMGAnimationList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SUMGAnimationList	) {}
	SLATE_END_ARGS()

	~SUMGAnimationList()
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
		{
			WidgetBlueprintEditorPin->OnWidgetAnimationsUpdated.RemoveAll(this);
			WidgetBlueprintEditorPin->OnSelectedAnimationChanged.RemoveAll(this);
		}
	}

	void Construct( const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor, bool bInIsDrawerTab )
	{
		BlueprintEditor = InBlueprintEditor;
		bIsDrawerTab = bInIsDrawerTab;

		InBlueprintEditor->GetOnWidgetBlueprintTransaction().AddSP( this, &SUMGAnimationList::OnWidgetBlueprintTransaction );
		InBlueprintEditor->OnEnterWidgetDesigner.AddSP(this, &SUMGAnimationList::OnEnteringDesignerMode);
		InBlueprintEditor->OnWidgetAnimationsUpdated.AddSP(this, &SUMGAnimationList::OnUpdatedAnimationList);
		InBlueprintEditor->OnSelectedAnimationChanged.AddSP(this, &SUMGAnimationList::AnimationListSelelctionSync);

		SAssignNew(AnimationListView, SWidgetAnimationListView)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SUMGAnimationList::OnGenerateWidgetForMovieScene)
			.OnItemScrolledIntoView(this, &SUMGAnimationList::OnItemScrolledIntoView)
			.OnSelectionChanged(this, &SUMGAnimationList::OnSelectionChanged)
			.OnContextMenuOpening(this, &SUMGAnimationList::OnContextMenuOpening)
			.ListItemsSource(&Animations);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding( FMargin(bIsDrawerTab ? 8.0f : 2.0f, 2.0f) )
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SSplitter)
					+ SSplitter::Slot()
					.Value(0.15f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.Padding( 2 )
						.AutoHeight()
						[
							SNew( SHorizontalBox )
							+ SHorizontalBox::Slot()
							.Padding(0)
							.VAlign( VAlign_Center )
							.AutoWidth()
							[
								SNew(SPositiveActionButton)
								.OnClicked( this, &SUMGAnimationList::OnNewAnimationClicked )
								.Text( LOCTEXT("NewAnimationButtonText", "Animation") )
							]
							+ SHorizontalBox::Slot()
							.Padding(2.0f, 0.0f)
							.VAlign( VAlign_Center )
							[
								SAssignNew(SearchBoxPtr, SSearchBox)
								.HintText(LOCTEXT("Search Animations", "Search Animations"))
								.OnTextChanged(this, &SUMGAnimationList::OnSearchChanged)
							]
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SScrollBorder, AnimationListView.ToSharedRef())
							[
								AnimationListView.ToSharedRef()
							]
						]
					]
					+ SSplitter::Slot()
					.Value(0.85f)
					[
						bIsDrawerTab ? BlueprintEditor.Pin()->CreateSequencerDrawerWidget() : BlueprintEditor.Pin()->CreateSequencerTabWidget()
					]
				]
				+SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				.Padding(FMargin(24.0, 10.0))
				[
					CreateDrawerDockButton()
				]
			]
		];

		UpdateAnimationList();

		CreateCommandList();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		FReply Reply = FReply::Unhandled();
		if( CommandList->ProcessCommandBindings( InKeyEvent ) )
		{
			Reply = FReply::Handled();
		}

		return Reply;
	}

private:

	void RefreshCurrentAnimationSelection()
	{		
		if(const UWidgetAnimation* CurrentAnim = BlueprintEditor.Pin()->RefreshCurrentAnimation())
		{
			const TSharedPtr<FWidgetAnimationListItem>* FoundListItemPtr = Animations.FindByPredicate([&](const TSharedPtr<FWidgetAnimationListItem>& ListItem) { return ListItem->Animation == CurrentAnim; });
			if (FoundListItemPtr != nullptr)
			{
				AnimationListView->SetSelection(*FoundListItemPtr);
			}
		}
	}

	void OnUpdatedAnimationList()
	{
		Animations.Empty();

		const TArray<UWidgetAnimation*>& WidgetAnimations = BlueprintEditor.Pin()->GetWidgetBlueprintObj()->Animations;

		for (UWidgetAnimation* Animation : WidgetAnimations)
		{
			Animations.Add(MakeShareable(new FWidgetAnimationListItem(Animation)));
		}

		RefreshCurrentAnimationSelection();

		AnimationListView->RequestListRefresh();
	}

	void AnimationListSelelctionSync()
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
		{
			UWidgetAnimation* CurrentSelectedAnimation = WidgetBlueprintEditorPin->GetCurrentAnimation();

			// This is to avoid looping calls to this function due to broadcast.
			for (const TSharedPtr<FWidgetAnimationListItem>& SelectedAnimItem : AnimationListView->GetSelectedItems())
			{
				if (SelectedAnimItem->Animation == CurrentSelectedAnimation)
				{
					return;
				}
			}

			// Find the list item containing the selected animation.
			for (const TSharedPtr<FWidgetAnimationListItem>& AnimItem : Animations)
			{
				if (AnimItem->Animation == CurrentSelectedAnimation)
				{
					AnimationListView->SetSelection(AnimItem);
					return;
				}
			}
			AnimationListView->ClearSelection();
		}
	}

	void UpdateAnimationList()
	{
		// There may be multiple sequencers acting as a view for our widget
		// Let the BP editor handle updates as it as aware of all possible sequencers
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin())
		{
			WidgetBlueprintEditorPin->NotifyWidgetAnimListChanged();
		}
	}

	void OnEnteringDesignerMode()
	{
		UpdateAnimationList();


		TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin();

		RefreshCurrentAnimationSelection();

		UWidgetAnimation* CurrentAnim = WidgetBlueprintEditorPin->GetCurrentAnimation();
		WidgetBlueprintEditorPin->ChangeViewedAnimation(*CurrentAnim);
	}

	void OnWidgetBlueprintTransaction()
	{
		UpdateAnimationList();

		
		TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin();
		const UWidgetAnimation* ViewedAnim = WidgetBlueprintEditorPin->RefreshCurrentAnimation();

		if(ViewedAnim)
		{
			const TSharedPtr<FWidgetAnimationListItem>* FoundListItemPtr = Animations.FindByPredicate( [&](const TSharedPtr<FWidgetAnimationListItem>& ListItem ) { return ListItem->Animation == ViewedAnim; } );

			if (FoundListItemPtr != nullptr)
			{
				AnimationListView->SetSelection(*FoundListItemPtr);
			}
		}

	}

	void OnItemScrolledIntoView( TSharedPtr<FWidgetAnimationListItem> InListItem, const TSharedPtr<ITableRow>& InWidget ) const
	{
		if( InListItem->bRenameRequestPending )
		{
			StaticCastSharedPtr<SWidgetAnimationListItem>( InWidget )->BeginRename();
			InListItem->bRenameRequestPending = false;
		}
	}

	FReply OnNewAnimationClicked()
	{
		const float InTime = 0.f;
		const float OutTime = 5.0f;

		TSharedPtr<FWidgetBlueprintEditor> Editor = BlueprintEditor.Pin();
		if (!Editor.IsValid())
		{
			return FReply::Handled();
		}

		UWidgetBlueprint* WidgetBlueprint = Editor->GetWidgetBlueprintObj();

		FString BaseName = "NewAnimation";
		UWidgetAnimation* NewAnimation = NewObject<UWidgetAnimation>(WidgetBlueprint, FName(), RF_Transactional);

		FString UniqueName = BaseName;
		int32 NameIndex = 1;
		FText Unused;
		while ( VerifyAnimationRename( *Editor, NewAnimation, UniqueName, Unused ) == false )
		{
			UniqueName = FString::Printf( TEXT( "%s_%i" ), *BaseName, NameIndex );
			NameIndex++;
		}

		const FName NewFName = FName(*UniqueName);
		NewAnimation->SetDisplayLabel( UniqueName );
		NewAnimation->Rename(*UniqueName);

		NewAnimation->MovieScene = NewObject<UMovieScene>(NewAnimation, NewFName, RF_Transactional);

		const UUMGEditorProjectSettings* UMGEditorSettings = GetDefault<UUMGEditorProjectSettings>();
		NewAnimation->MovieScene->SetDisplayRate(FFrameRate(UMGEditorSettings->DefaultWidgetAnimationFrameRate, 1));

		const FFrameTime InFrame  = InTime  * NewAnimation->MovieScene->GetTickResolution();
		const FFrameTime OutFrame = OutTime * NewAnimation->MovieScene->GetTickResolution();
		NewAnimation->MovieScene->SetPlaybackRange(TRange<FFrameNumber>(InFrame.FrameNumber, OutFrame.FrameNumber+1));
		NewAnimation->MovieScene->GetEditorData().WorkStart = InTime;
		NewAnimation->MovieScene->GetEditorData().WorkEnd   = OutTime;

		bool bRequestRename = true;
		bool bNewAnimation = true;

		int32 NewIndex = Animations.Add( MakeShareable( new FWidgetAnimationListItem( NewAnimation, bRequestRename, bNewAnimation ) ) );

		AnimationListView->RequestScrollIntoView( Animations[NewIndex] );

		return FReply::Handled();
	}

	void OnSearchChanged( const FText& InSearchText )
	{
		const TArray<UWidgetAnimation*>& WidgetAnimations = BlueprintEditor.Pin()->GetWidgetBlueprintObj()->Animations;

		if( !InSearchText.IsEmpty() )
		{
			struct Local
			{
				static void UpdateFilterStrings(UWidgetAnimation* InAnimation, OUT TArray< FString >& OutFilterStrings)
				{
					OutFilterStrings.Add(InAnimation->GetName());
				}
			};

			TTextFilter<UWidgetAnimation*> TextFilter(TTextFilter<UWidgetAnimation*>::FItemToStringArray::CreateStatic(&Local::UpdateFilterStrings));

			TextFilter.SetRawFilterText(InSearchText);
			SearchBoxPtr->SetError(TextFilter.GetFilterErrorText());

			Animations.Reset();

			for(UWidgetAnimation* Animation : WidgetAnimations)
			{
				if(TextFilter.PassesFilter(Animation))
				{
					Animations.Add(MakeShareable(new FWidgetAnimationListItem(Animation)));
				}
			}

			AnimationListView->RequestListRefresh();
		}
		else
		{
			SearchBoxPtr->SetError(FText::GetEmpty());

			// Just regenerate the whole list
			UpdateAnimationList();
		}
	}

	void OnSelectionChanged( TSharedPtr<FWidgetAnimationListItem> InSelectedItem, ESelectInfo::Type SelectionInfo )
	{
		UWidgetAnimation* WidgetAnimation;
		if( InSelectedItem.IsValid() )
		{
			WidgetAnimation = InSelectedItem->Animation;
		}
		else
		{
			WidgetAnimation = UWidgetAnimation::GetNullAnimation();
		}

		const UWidgetAnimation* CurrentWidgetAnimation = BlueprintEditor.Pin()->RefreshCurrentAnimation();
		if (WidgetAnimation != CurrentWidgetAnimation)
		{
			BlueprintEditor.Pin()->ChangeViewedAnimation( *WidgetAnimation );
		}
	}

	TSharedPtr<SWidget> OnContextMenuOpening()
	{
		FMenuBuilder MenuBuilder( true, CommandList.ToSharedRef() );

		MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder.AddMenuSeparator();
			MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferences);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void FindReferencesToSelectedAnimation()
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = BlueprintEditor.Pin())
		{
			TArray< TSharedPtr<FWidgetAnimationListItem> > SelectedAnimations = AnimationListView->GetSelectedItems();
			if (SelectedAnimations.Num() == 1)
			{
				TSharedPtr<FWidgetAnimationListItem> SelectedAnimation = SelectedAnimations[0];
				const FString VariableName = SelectedAnimation->Animation->GetName();
				
				FMemberReference MemberReference;
				MemberReference.SetSelfMember(*VariableName);
				const FString SearchTerm = MemberReference.GetReferenceSearchString(WidgetEditor->GetWidgetBlueprintObj()->SkeletonGeneratedClass);

				WidgetEditor->SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);
				WidgetEditor->SummonSearchUI(true, SearchTerm);
			}
		}
	}

	TSharedRef<ITableRow> OnGenerateWidgetForMovieScene( TSharedPtr<FWidgetAnimationListItem> InListItem, const TSharedRef< STableViewBase >& InOwnerTableView )
	{
		return SNew( SWidgetAnimationListItem, InOwnerTableView, BlueprintEditor.Pin(), InListItem );
	}

	void CreateCommandList()
	{
		CommandList = MakeShareable(new FUICommandList);

		CommandList->MapAction(
			FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &SUMGAnimationList::OnDuplicateAnimation),
			FCanExecuteAction::CreateSP(this, &SUMGAnimationList::CanExecuteContextMenuAction)
			);

		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SUMGAnimationList::OnDeleteAnimation),
			FCanExecuteAction::CreateSP(this, &SUMGAnimationList::CanExecuteContextMenuAction)
			);

		CommandList->MapAction(
			FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SUMGAnimationList::OnRenameAnimation),
			FCanExecuteAction::CreateSP(this, &SUMGAnimationList::CanExecuteContextMenuAction)
			);
		
		CommandList->MapAction(
			FGraphEditorCommands::Get().FindReferences,
			FExecuteAction::CreateSP(this, &SUMGAnimationList::FindReferencesToSelectedAnimation),
			FCanExecuteAction::CreateSP(this, &SUMGAnimationList::CanExecuteContextMenuAction)
			);

		CommandList->MapAction(FUMGEditorCommands::Get().OpenAnimDrawer,
			FExecuteAction::CreateSP(this, &SUMGAnimationList::ToggleAnimDrawer)
		);
	}

	FReply CreateDrawerDockButtonClicked()
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = BlueprintEditor.Pin())
		{
			WidgetEditor->DockInLayoutClicked();
		}

		return FReply::Handled();
	}

	TSharedRef<SWidget> CreateDrawerDockButton()
	{
		if(bIsDrawerTab)
		{
			return 
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("DockInLayout_Tooltip", "Docks animation drawer in tab."))
				.ContentPadding(FMargin(1, 0))
				.OnClicked(this, &SUMGAnimationList::CreateDrawerDockButtonClicked)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DockInLayout", "Dock in Layout"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				];
		}

		return SNullWidget::NullWidget;
	}

	bool CanExecuteContextMenuAction() const
	{
		return AnimationListView->GetNumItemsSelected() == 1 && !BlueprintEditor.Pin()->IsPlayInEditorActive();
	}

	void OnDuplicateAnimation()
	{
		TArray< TSharedPtr<FWidgetAnimationListItem> > SelectedAnimations = AnimationListView->GetSelectedItems();
		check(SelectedAnimations.Num() == 1);

		TSharedPtr<FWidgetAnimationListItem> SelectedAnimation = SelectedAnimations[0];

		UWidgetBlueprint* WidgetBlueprint = BlueprintEditor.Pin()->GetWidgetBlueprintObj();

		UWidgetAnimation* NewAnimation = 
			DuplicateObject<UWidgetAnimation>
			(
				SelectedAnimation->Animation,
				WidgetBlueprint, 
				MakeUniqueObjectName( WidgetBlueprint, UWidgetAnimation::StaticClass(), SelectedAnimation->Animation->GetFName() )
			);
	
		NewAnimation->MovieScene->Rename(*NewAnimation->GetName(), nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		NewAnimation->SetDisplayLabel(NewAnimation->GetName());

		bool bRenameRequestPending = true;
		bool bNewAnimation = true;
		int32 NewIndex = Animations.Add(MakeShareable(new FWidgetAnimationListItem(NewAnimation,bRenameRequestPending,bNewAnimation)));

		AnimationListView->RequestScrollIntoView(Animations[NewIndex]);
	}


	void OnDeleteAnimation()
	{
		TArray< TSharedPtr<FWidgetAnimationListItem> > SelectedAnimations = AnimationListView->GetSelectedItems();
		check(SelectedAnimations.Num() == 1);

		TSharedPtr<FWidgetAnimationListItem> SelectedAnimation = SelectedAnimations[0];

		TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditorPin = BlueprintEditor.Pin();

		UWidgetBlueprint* WidgetBlueprint = WidgetBlueprintEditorPin->GetWidgetBlueprintObj();

		TArray<TObjectPtr<UWidgetAnimation>>& WidgetAnimations = WidgetBlueprint->Animations;

		{
			const FScopedTransaction Transaction(LOCTEXT("DeleteAnimationTransaction", "Delete Animation"));
			WidgetBlueprint->Modify();
			const FName RemovedAnimationName = SelectedAnimation->Animation->GetFName();
			// Rename the animation and move it to the transient package to avoid collisions.
			SelectedAnimation->Animation->Rename( NULL, GetTransientPackage() );
			WidgetAnimations.Remove(SelectedAnimation->Animation);
			WidgetBlueprint->OnVariableRemoved(RemovedAnimationName);

			UpdateAnimationList();
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);

		WidgetBlueprintEditorPin->ChangeViewedAnimation(*UWidgetAnimation::GetNullAnimation());
	}

	void OnRenameAnimation()
	{
		TArray< TSharedPtr<FWidgetAnimationListItem> > SelectedAnimations = AnimationListView->GetSelectedItems();
		check( SelectedAnimations.Num() == 1 );

		TSharedPtr<FWidgetAnimationListItem> SelectedAnimation = SelectedAnimations[0];
		SelectedAnimation->bRenameRequestPending = true;

		AnimationListView->RequestScrollIntoView( SelectedAnimation );
	}

	void ToggleAnimDrawer()
	{
		if (TSharedPtr<FWidgetBlueprintEditor> WidgetEditor = BlueprintEditor.Pin())
		{
			WidgetEditor->ToggleAnimDrawer();
		}
	}


private:
	TSharedPtr<FUICommandList> CommandList;
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TSharedPtr<SWidgetAnimationListView> AnimationListView;
	TArray< TSharedPtr<FWidgetAnimationListItem> > Animations;
	TSharedPtr<SSearchBox> SearchBoxPtr;
	bool bIsDrawerTab;
};


TSharedRef<SWidget> FAnimationTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin();

	return SNew( SUMGAnimationList, BlueprintEditorPinned, bIsDrawerTab);
}

TSharedRef<SDockTab> FAnimationTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> NewTab = FWorkflowTabFactory::SpawnTab(Info);
	if (TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin())
	{
		NewTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(BlueprintEditorPinned.ToSharedRef(), &FWidgetBlueprintEditor::OnWidgetAnimTabSequencerClosed));
		BlueprintEditorPinned->OnWidgetAnimTabSequencerOpened();
	}
	return NewTab;
}

#undef LOCTEXT_NAMESPACE 
