// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WidgetBlueprintEditor.h"

class SDockTab;

struct FAnimationTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	static const FName WidgetAnimSequencerDrawerID;
	
public:
	FAnimationTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor, bool bInIsDrawerTab = false);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
	bool bIsDrawerTab;
};
