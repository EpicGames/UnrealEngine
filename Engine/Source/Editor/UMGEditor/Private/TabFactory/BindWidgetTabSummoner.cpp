// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/BindWidgetTabSummoner.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBindWidgetView.h"

#define LOCTEXT_NAMESPACE "UMG.BindWidget"

const FName FBindWidgetTabSummoner::TabID(TEXT("BindWidget"));

FBindWidgetTabSummoner::FBindWidgetTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("SlateBindWidgetTabLabel", "Bind Widgets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SlateHierarchy_ViewMenu_Desc", "Bind Widgets");
	ViewMenuTooltip = LOCTEXT("SlateHierarchy_ViewMenu_ToolTip", "Show the list of widgets that needs to be binded.");
}

TSharedRef<SWidget> FBindWidgetTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SBindWidgetView, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("BindWidget")));
}

#undef LOCTEXT_NAMESPACE 
