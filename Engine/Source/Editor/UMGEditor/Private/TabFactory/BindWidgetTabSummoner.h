// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WidgetBlueprintEditor.h"

class FWidgetBlueprintEditor;

struct FBindWidgetTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FBindWidgetTabSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
};
