// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/DesignerTabSummoner.h"
#include "Tools/ToolCompatible.h"
#include "Designer/SDesignerView.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "UMGStyle.h" 

#define LOCTEXT_NAMESPACE "UMG"

const FName FDesignerTabSummoner::ToolPaletteTabID = UAssetEditorUISubsystem::TopLeftTabID;
const FName FDesignerTabSummoner::TabID(TEXT("SlatePreview"));

static int32 bEnableWidgetDesignerTools = 1;
static FAutoConsoleVariableRef CVarEnableWidgetDesignerTools(
	TEXT("UMG.Editor.EnableWidgetDesignerTools"),
	bEnableWidgetDesignerTools,
	TEXT("Toggles processing of widget designer tools. Must be set before opening widget designer."));


FDesignerTabSummoner::FDesignerTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("DesignerTabLabel", "Designer");
	TabIcon = FSlateIcon(FUMGStyle::GetStyleSetName(), "Designer.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SlatePreview_ViewMenu_Desc", "Designer");
	ViewMenuTooltip = LOCTEXT("SlatePreview_ViewMenu_ToolTip", "Show the Designer");
}

TSharedRef<SWidget> FDesignerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	if (!bEnableWidgetDesignerTools)
	{
		return SNew(SDesignerView, BlueprintEditor.Pin())
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Designer")));
	}

	TSharedRef<TToolCompatibleMixin<SDesignerView>> DesignerView = SNew(TToolCompatibleMixin<SDesignerView>, BlueprintEditor.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Designer")));
	DesignerView->SetParentToolkit(BlueprintEditor.Pin());

	// @TODO: DarenC - Ideally this is done by SetParentToolkit, but can't seem to do that due to template issues with SharedThis.
	DesignerView->GetWidgetModeManger()->SetManagedWidget(DesignerView);
	return DesignerView;
}

#undef LOCTEXT_NAMESPACE 
