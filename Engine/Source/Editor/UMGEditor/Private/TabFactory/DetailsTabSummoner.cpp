// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/DetailsTabSummoner.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "Details/SWidgetDetailsView.h"
#define LOCTEXT_NAMESPACE "UMG"

const FName FDetailsTabSummoner::TabID(TEXT("WidgetDetails"));

FDetailsTabSummoner::FDetailsTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("WidgetDetails_TabLabel", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("WidgetDetails_ViewMenu_Desc", "Details");
	ViewMenuTooltip = LOCTEXT("WidgetDetails_ViewMenu_ToolTip", "Show the Details");
}

TSharedRef<SWidget> FDetailsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SWidgetDetailsView, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Details")));
}

#undef LOCTEXT_NAMESPACE 
