// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/HierarchyTabSummoner.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "Hierarchy/SHierarchyView.h"
#include "WidgetBlueprintEditor.h"

#define LOCTEXT_NAMESPACE "UMG"

const FName FHierarchyTabSummoner::TabID(TEXT("SlateHierarchy"));

FHierarchyTabSummoner::FHierarchyTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("SlateHierarchyTabLabel", "Hierarchy");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SlateHierarchy_ViewMenu_Desc", "Hierarchy");
	ViewMenuTooltip = LOCTEXT("SlateHierarchy_ViewMenu_ToolTip", "Show the Hierarchy");
}

TSharedRef<SWidget> FHierarchyTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SHierarchyView, BlueprintEditorPtr, BlueprintEditorPtr->GetBlueprintObj()->SimpleConstructionScript)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Hierarchy")));
}

#undef LOCTEXT_NAMESPACE 
