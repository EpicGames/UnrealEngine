// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WidgetBlueprintEditor.h"

struct FHierarchyTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FHierarchyTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
};
