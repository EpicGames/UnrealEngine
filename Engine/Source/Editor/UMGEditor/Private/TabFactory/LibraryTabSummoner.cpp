// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/LibraryTabSummoner.h"
#include "Library/SLibraryView.h"
#include "UMGStyle.h"

#define LOCTEXT_NAMESPACE "UMG"

const FName FLibraryTabSummoner::TabID(TEXT("WidgetLibrary"));

FLibraryTabSummoner::FLibraryTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("WidgetLibraryTabLabel", "Library");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderClosed");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("WidgetLibrary_ViewMenu_Desc", "Library");
	ViewMenuTooltip = LOCTEXT("WidgetLibrary_ViewMenu_ToolTip", "Show the Widget Library");
}

TSharedRef<SWidget> FLibraryTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SLibraryView, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Library")));
}

#undef LOCTEXT_NAMESPACE 
