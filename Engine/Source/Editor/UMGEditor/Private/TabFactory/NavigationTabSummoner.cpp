// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/NavigationTabSummoner.h"
#include "Navigation/SWidgetDesignerNavigation.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "UMG"

const FName FNavigationTabSummoner::TabID(TEXT("NavigationSimulation"));

FNavigationTabSummoner::FNavigationTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("NavigationSimulationTabLabel", "Navigation Simulation");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SoftwareCursor_CardinalCross");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("NavigationSimulation_ViewMenu_Desc", "Navigation Simulation");
	ViewMenuTooltip = LOCTEXT("NavigationSimulation_ViewMenu_ToolTip", "Show the Navigation Simulation");
}

TSharedRef<SWidget> FNavigationTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SWidgetDesignerNavigation, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("WidgetDesignerNavigation")));
}

#undef LOCTEXT_NAMESPACE 
