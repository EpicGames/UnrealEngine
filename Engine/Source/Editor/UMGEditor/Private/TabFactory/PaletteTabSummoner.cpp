// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/PaletteTabSummoner.h"
#include "Palette/SPaletteView.h"
#include "UMGStyle.h"

#define LOCTEXT_NAMESPACE "UMG"

const FName FPaletteTabSummoner::TabID(TEXT("WidgetTemplates"));

FPaletteTabSummoner::FPaletteTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("WidgetTemplatesTabLabel", "Palette");
	TabIcon = FSlateIcon(FUMGStyle::GetStyleSetName(), "Palette.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("WidgetTemplates_ViewMenu_Desc", "Palette");
	ViewMenuTooltip = LOCTEXT("WidgetTemplates_ViewMenu_ToolTip", "Show the Palette");
}

TSharedRef<SWidget> FPaletteTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FWidgetBlueprintEditor>(BlueprintEditor.Pin());

	return SNew(SPaletteView, BlueprintEditorPtr)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Palette")));
}

#undef LOCTEXT_NAMESPACE 
