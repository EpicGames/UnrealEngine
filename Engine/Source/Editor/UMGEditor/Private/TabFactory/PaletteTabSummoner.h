// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WidgetBlueprintEditor.h"

struct FPaletteTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FPaletteTabSummoner(TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<class FWidgetBlueprintEditor> BlueprintEditor;
};
