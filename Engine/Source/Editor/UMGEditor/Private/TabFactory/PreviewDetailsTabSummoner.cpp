// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/PreviewDetailsTabSummoner.h"

#include "UMGStyle.h"
#include "WidgetBlueprintEditor.h"
#include "Preview/SPreviewDetails.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Editor
{

const FName FPreviewDetailsTabSummoner::TabID("DebugDetails");

FPreviewDetailsTabSummoner::FPreviewDetailsTabSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
	: FWorkflowTabFactory(TabID, InBlueprintEditor)
	, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("DebugDetails_TabLabel", "Detail View");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDebugger.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DebugDetails_ViewMenu_Desc", "Detail View");
	ViewMenuTooltip = LOCTEXT("DebugDetails_ViewMenu_ToolTip", "Show the Detail View");
}

TSharedRef<SWidget> FPreviewDetailsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SPreviewDetails, BlueprintEditor.Pin());
}

} //namespace

#undef LOCTEXT_NAMESPACE 
