// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FWidgetBlueprintEditor;
class SWidget;

namespace UE::UMG::Editor
{

struct FPreviewDetailsTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FPreviewDetailsTabSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
};

} //namespace