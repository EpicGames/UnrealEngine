// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/PreviewTabSummoner.h"

#include "Preview/SWidgetPreview.h"
#include "UMGStyle.h"
#include "WidgetBlueprintEditor.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Editor
{

const FName FWidgetPreviewTabSummoner::TabID("WidgetPreview");

FWidgetPreviewTabSummoner::FWidgetPreviewTabSummoner(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
		: FWorkflowTabFactory(TabID, InBlueprintEditor)
		, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("DebugPreview_TabLabel", "Preview");
	TabIcon = FSlateIcon(FUMGStyle::GetStyleSetName(), "BlueprintDebugger.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DebugPreview_ViewMenu_Desc", "Preview");
	ViewMenuTooltip = LOCTEXT("DebugPreview_ViewMenu_ToolTip", "Show the Preview");
}

TSharedRef<SWidget> FWidgetPreviewTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SWidgetPreview, BlueprintEditor.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Preview")));
}

} // namespace

#undef LOCTEXT_NAMESPACE 
