// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetTemplate.h"

FWidgetTemplate::FWidgetTemplate()
{
}
