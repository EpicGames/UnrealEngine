// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/WidgetTemplateBlueprintClass.h"

#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "UObject/CoreRedirects.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditorUtils.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "UMGEditor"

//FWidgetTemplateBlueprintClass::FWidgetTemplateBlueprintClass(const FAssetData& InWidgetAssetData, TSubclassOf<UUserWidget> InUserWidgetClass, bool bInIsBlueprintGeneratedClass)
//	: FWidgetTemplateBlueprintClass(InWidgetAssetData, InUserWidgetClass)
//{}

FWidgetTemplateBlueprintClass::FWidgetTemplateBlueprintClass(const FAssetData& InWidgetAssetData, TSubclassOf<UUserWidget> InUserWidgetClass)
	: FWidgetTemplateClass(InWidgetAssetData, InUserWidgetClass)
{
	// Blueprints get the class type actions for their parent native class - this avoids us having to load the blueprint
	static const FTopLevelAssetPath WidgetBlueprintGeneratedClassAssetPath = UWidgetBlueprintGeneratedClass::StaticClass()->GetClassPathName();
	static const FTopLevelAssetPath BlueprintGeneratedClassAssetPath = UBlueprintGeneratedClass::StaticClass()->GetClassPathName();
	bool bClassIsUWidgetBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(InUserWidgetClass.Get()) != nullptr;
	bIsBlueprintGeneratedClass = WidgetBlueprintGeneratedClassAssetPath == InWidgetAssetData.AssetClassPath 
								|| BlueprintGeneratedClassAssetPath == InWidgetAssetData.AssetClassPath 
								|| bClassIsUWidgetBlueprintGeneratedClass;
	if (bIsBlueprintGeneratedClass && !InUserWidgetClass && InWidgetAssetData.IsValid())
	{
		FString ParentClassName;
		if (!WidgetAssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
		{
			WidgetAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
		}
		if (!ParentClassName.IsEmpty())
		{
			const FString RedirectedClassPath = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(ParentClassName)).ToString();
			CachedParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(RedirectedClassPath));
		}
	}
}

FWidgetTemplateBlueprintClass::~FWidgetTemplateBlueprintClass()
{
}

FText FWidgetTemplateBlueprintClass::GetCategory() const
{
	if (WidgetClass.Get())
	{
		return FWidgetBlueprintEditorUtils::GetPaletteCategory(WidgetClass.Get());
	}
	else
	{
		return FWidgetBlueprintEditorUtils::GetPaletteCategory(WidgetAssetData, CachedParentClass.Get());
	}
}

UWidget* FWidgetTemplateBlueprintClass::Create(UWidgetTree* Tree)
{
	// Load the blueprint asset or blueprint generated class if needed
	if (!WidgetClass.Get())
	{
		FString AssetPath = WidgetAssetData.GetObjectPathString();

		if (bIsBlueprintGeneratedClass)
		{
			WidgetClass = LoadObject<UBlueprintGeneratedClass>(nullptr, *AssetPath);
		}
		else if (UBlueprint* LoadedWidget = LoadObject<UBlueprint>(nullptr, *AssetPath))
		{
			WidgetClass = *LoadedWidget->GeneratedClass;
 		}
	}

	return FWidgetTemplateClass::CreateNamed(Tree, FName(*FBlueprintEditorUtils::GetClassNameWithoutSuffix(WidgetClass.Get())));
}

const FSlateBrush* FWidgetTemplateBlueprintClass::GetIcon() const
{
	if (CachedParentClass.IsValid() && CachedParentClass->IsChildOf(UWidget::StaticClass()) && !CachedParentClass->IsChildOf(UUserWidget::StaticClass()))
	{
		return FSlateIconFinder::FindIconBrushForClass(CachedParentClass.Get());
	}
	else
	{
		return FSlateIconFinder::FindIconBrushForClass(UUserWidget::StaticClass());
	}
	
}

TSharedRef<IToolTip> FWidgetTemplateBlueprintClass::GetToolTip() const
{
	FText Description;

	FString DescriptionStr = WidgetAssetData.GetTagValueRef<FString>( GET_MEMBER_NAME_CHECKED( UBlueprint, BlueprintDescription ) );
	if ( !DescriptionStr.IsEmpty() )
	{
		DescriptionStr.ReplaceInline( TEXT( "\\n" ), TEXT( "\n" ) );
		Description = FText::FromString( MoveTemp(DescriptionStr) );
	}
	else
	{
		Description = Name;
	}

	return IDocumentation::Get()->CreateToolTip( Description, nullptr, FString( TEXT( "Shared/Types/" ) ) + Name.ToString(), TEXT( "Class" ) );
}

FReply FWidgetTemplateBlueprintClass::OnDoubleClicked()
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset( WidgetAssetData.GetAsset() );
	return FReply::Handled();
}

bool FWidgetTemplateBlueprintClass::Supports(UClass* InClass)
{
	return InClass != nullptr && InClass->IsChildOf(UWidgetBlueprint::StaticClass());
}

#undef LOCTEXT_NAMESPACE
