// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/WidgetTemplateClass.h"
#include "Engine/Blueprint.h"

#if WITH_EDITOR
	#include "Editor.h"
#endif // WITH_EDITOR
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "UObject/CoreRedirects.h"
#include "WidgetBlueprintEditorUtils.h"

#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Blueprint/WidgetTree.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "UMGEditor"

FWidgetTemplateClass::FWidgetTemplateClass()
	: WidgetClass(nullptr)
{
	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FWidgetTemplateClass::OnObjectsReplaced);
}

FWidgetTemplateClass::FWidgetTemplateClass(TSubclassOf<UWidget> InWidgetClass)
	: WidgetClass(InWidgetClass.Get())
{
	Name = WidgetClass->GetDisplayNameText();

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FWidgetTemplateClass::OnObjectsReplaced);
}

FWidgetTemplateClass::FWidgetTemplateClass(const FAssetData& InWidgetAssetData, TSubclassOf<UWidget> InWidgetClass)
	: WidgetAssetData(InWidgetAssetData)
{
	if (InWidgetClass)
	{
		WidgetClass = *InWidgetClass;
		Name = WidgetClass->GetDisplayNameText();
	}
	else
	{
		FString AssetName = WidgetAssetData.AssetName.ToString();
		AssetName.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);
		Name = FText::FromString(FName::NameToDisplayString(AssetName, false));

		// Blueprints get the class type actions for their parent native class - this avoids us having to load the blueprint
		if (InWidgetAssetData.IsValid())
		{
			FString ParentClassName;
			if (!WidgetAssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
			{
				WidgetAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
			}
			if (!ParentClassName.IsEmpty())
			{
				const FString RedirectedClassPath = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(ParentClassName)).ToString();
				CachedParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(RedirectedClassPath));
				ensure(CachedParentClass == nullptr || CachedParentClass->IsChildOf(UWidget::StaticClass()));
			}
		}
	}
}

FWidgetTemplateClass::~FWidgetTemplateClass()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

FText FWidgetTemplateClass::GetCategory() const
{
	if (WidgetClass.Get())
	{
		return FWidgetBlueprintEditorUtils::GetPaletteCategory(WidgetClass.Get());
	}
	else
	{
		return FWidgetBlueprintEditorUtils::GetPaletteCategory(WidgetAssetData, CachedParentClass.Get());
	}
}

UWidget* FWidgetTemplateClass::Create(UWidgetTree* Tree)
{
	// Load the blueprint asset if needed
	if (!WidgetClass.Get())
	{
		FString AssetPath = WidgetAssetData.GetObjectPathString();
		UBlueprint* LoadedBP = LoadObject<UBlueprint>(nullptr, *AssetPath);
		WidgetClass = *LoadedBP->GeneratedClass;
	}

	return CreateNamed(Tree, NAME_None);
}

const FSlateBrush* FWidgetTemplateClass::GetIcon() const
{
	if (WidgetClass.IsValid())
	{
		return FSlateIconFinder::FindIconBrushForClass(WidgetClass.Get());
	}
	else if (CachedParentClass.IsValid() && CachedParentClass->IsChildOf(UWidget::StaticClass()) && !CachedParentClass->IsChildOf(UUserWidget::StaticClass()))
	{
		return FSlateIconFinder::FindIconBrushForClass(CachedParentClass.Get());
	}
	else
	{
		return FSlateIconFinder::FindIconBrushForClass(UWidget::StaticClass());
	}
}

TSharedRef<IToolTip> FWidgetTemplateClass::GetToolTip() const
{
	const FString ExcerptName = WidgetClass.IsValid() ? WidgetClass->GetName() : Name.ToString();
	return IDocumentation::Get()->CreateToolTip(GetToolTipText(), nullptr, FString(TEXT("Shared/Types/")) + ExcerptName, TEXT("Class"));
}

FText FWidgetTemplateClass::GetToolTipText() const
{
	if (WidgetClass.IsValid())
	{
		return WidgetClass->GetToolTipText();
	}

	FString DescriptionStr = WidgetAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDescription));
	if (!DescriptionStr.IsEmpty())
	{
		DescriptionStr.ReplaceInline(TEXT("\\n"), TEXT("\n"));
		return FText::FromString(MoveTemp(DescriptionStr));
	}

	return Name;
}

void FWidgetTemplateClass::GetFilterStrings(TArray<FString>& OutStrings) const
{
	FWidgetTemplate::GetFilterStrings(OutStrings);
	if (WidgetClass.IsValid())
	{
		OutStrings.Add(WidgetClass->GetName());
	}
	if (WidgetAssetData.IsValid())
	{
		OutStrings.Add(WidgetAssetData.AssetName.ToString());
	}
}

void FWidgetTemplateClass::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	UObject* const* NewObject = ReplacementMap.Find(WidgetClass.Get());
	if (NewObject)
	{
		WidgetClass = CastChecked<UClass>(*NewObject);
	}
}

UWidget* FWidgetTemplateClass::CreateNamed(class UWidgetTree* Tree, FName NameOverride)
{
	if (Tree == nullptr || !WidgetClass.IsValid() || WidgetClass.Get()->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
	{
		return nullptr;
	}

	// For inherited widget trees, we need to make sure the name is unique for the parent tree
	// We do this even if NameOverride is not set as the default name could exist in the parent tree
	if (Tree->RootWidget == nullptr)
	{
		if (const UWidgetBlueprint* WidgetBP = Cast<UWidgetBlueprint>(Tree->GetOuter()))
		{
			const UWidgetBlueprintGeneratedClass* BaseWidgetClass = Cast<UWidgetBlueprintGeneratedClass>(WidgetBP->GeneratedClass);
			const UWidgetBlueprintGeneratedClass* ParentWidgetClass = BaseWidgetClass ? BaseWidgetClass->FindWidgetTreeOwningClass() : nullptr;
			if (ParentWidgetClass != BaseWidgetClass && ParentWidgetClass != nullptr)
			{
				NameOverride = MakeUniqueObjectName(ParentWidgetClass->GetWidgetTreeArchetype(), WidgetClass.Get(), NameOverride);
			}
		}
	}

	if (NameOverride != NAME_None)
	{
		UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Tree, *NameOverride.ToString());
		if (ExistingObject != nullptr)
		{
			NameOverride = MakeUniqueObjectName(Tree, WidgetClass.Get(), NameOverride);
		}
	}

	UWidget* NewWidget = Tree->ConstructWidget<UWidget>(WidgetClass.Get(), NameOverride);
	NewWidget->CreatedFromPalette();

	return NewWidget;
}

#undef LOCTEXT_NAMESPACE
