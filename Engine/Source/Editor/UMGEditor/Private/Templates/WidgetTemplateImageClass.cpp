// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/WidgetTemplateImageClass.h"
#include "Components/Image.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture.h"

FWidgetTemplateImageClass::FWidgetTemplateImageClass(const FAssetData& InAssetData)
	: FWidgetTemplateClass(InAssetData, UImage::StaticClass())
{
}

FWidgetTemplateImageClass::~FWidgetTemplateImageClass()
{
}

UWidget* FWidgetTemplateImageClass::Create(UWidgetTree* WidgetTree)
{
	UWidget* Widget = FWidgetTemplateClass::Create(WidgetTree);

	if (Cast<UImage>(Widget) != nullptr && Supports(FindObject<UClass>(WidgetAssetData.AssetClassPath)) )
	{
		UImage* ImageWidget = Cast<UImage>(Widget);

		UObject* ImageResource = FindObject<UObject>(nullptr, *WidgetAssetData.GetObjectPathString());
		ImageWidget->SetBrushResourceObject(ImageResource);
	}

	return Widget;
}

bool FWidgetTemplateImageClass::Supports(UClass* InClass)
{
	static const UClass* SlateTextureAtlasInterface = FindObject<UClass>(nullptr, TEXT("/Script/Engine.SlateTextureAtlasInterface"));

	return InClass != nullptr && ( InClass->IsChildOf(UTexture::StaticClass())
		|| InClass->IsChildOf(UMaterialInterface::StaticClass())
		|| InClass->ImplementsInterface(SlateTextureAtlasInterface) );
}
