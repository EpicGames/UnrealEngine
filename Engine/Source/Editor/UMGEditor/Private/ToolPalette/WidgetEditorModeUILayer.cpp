// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorModeUILayer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "UMGEditorModule.h"
#include "Toolkits/IToolkit.h"
#include "TabFactory/DesignerTabSummoner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetEditorModeUILayer)

void UWidgetEditorModeUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	IUMGEditorModule& UMGEditorModule = FModuleManager::GetModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UWidgetEditorModeUISubsystem::RegisterLayoutExtensions);
}

void UWidgetEditorModeUISubsystem::Deinitialize()
{
	IUMGEditorModule& UMGEditorModule = FModuleManager::GetModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void UWidgetEditorModeUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID, ETabIdFlags::SaveLayout), ETabState::ClosedTab);
	Extender.ExtendLayout(FDesignerTabSummoner::ToolPaletteTabID, ELayoutExtensionPosition::After, NewTab);
}

FWidgetEditorModeUILayer::FWidgetEditorModeUILayer(const IToolkitHost* InToolkitHost) :
	FAssetEditorModeUILayer(InToolkitHost)
{}

void FWidgetEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();
	}
}

void FWidgetEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}
}

TSharedPtr<FWorkspaceItem> FWidgetEditorModeUILayer::GetModeMenuCategory() const
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	return MenuStructure.GetLevelEditorModesCategory();
}

