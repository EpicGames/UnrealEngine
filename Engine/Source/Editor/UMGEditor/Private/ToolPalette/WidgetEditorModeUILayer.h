// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"
#include "WidgetEditorModeUILayer.generated.h"

UCLASS()
class UWidgetEditorModeUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;
};


class FWidgetEditorModeUILayer : public FAssetEditorModeUILayer
{
public:

	FWidgetEditorModeUILayer(const IToolkitHost* InToolkitHost);

	void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	
	TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;
};


