// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIComponentClipboardExtension.h"

#include "Exporters/Exporter.h"
#include "Extensions/UIComponent.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UIComponentUtils.h"
#include "UIComponentWidgetBlueprintExtension.h"
#include "WidgetBlueprintEditorUtils.h"

void FUIComponentClipboardExtension::AppendToClipboard(const UWidget* Widget, const FExportArgs& ExportArgs)
{
	check(ExportArgs.Out);

	if (const UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		if (const UUIComponentWidgetBlueprintExtension* ComponentExtension = UUIComponentWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
		{
			for (const UUIComponent* Component : ComponentExtension->GetComponentsFor(Widget))
			{
				UUIComponentWidgetPair* ComponentWidgetPair = NewObject<UUIComponentWidgetPair>();
				ComponentWidgetPair->WidgetName = Widget->GetFName();				
				ComponentWidgetPair->Component = CastChecked<UUIComponent>(StaticDuplicateObject(Component, ComponentWidgetPair));								
				
				UExporter::ExportToOutputDevice(ExportArgs.Context, ComponentWidgetPair, ExportArgs.Exporter, *ExportArgs.Out, ExportArgs.FileType, ExportArgs.Indent, ExportArgs.PortFlags, ExportArgs.bSelectedOnly, ExportArgs.ExportRootScope);				
			}		
		}
	}
}

bool FUIComponentClipboardExtension::CanAppendToClipboard(const UWidget* Widget) const
{
	if (FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget))
	{
		return true;
	}
	return false;
}

void FUIComponentClipboardExtension::ProcessImportedText(const UWidgetBlueprint* WidgetBlueprint, const FString& TextToImport, UPackage*& TempPackage)
{
	Factory.Reinit();
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
}

bool FUIComponentClipboardExtension::CanImportFromClipboard(const UWidget* Widget) const
{
	return !Factory.IsEmpty();
}

void FUIComponentClipboardExtension::ImportDataToWidget(const UWidget* Widget, FName OldWidgetName)
{
	Factory.AddProcessedComponentsTo(Widget, OldWidgetName);
}

bool FUIComponentClipboardExtension::CanWidgetAcceptPaste(const UWidget* Widget) const
{
	constexpr bool bCanWidgetAcceptPaste = true;
	return bCanWidgetAcceptPaste;
}

FUIComponentClipboardExtension::FExtensionTextFactory::FExtensionTextFactory()
	: FCustomizableTextObjectFactory(GWarn)
{
}

bool FUIComponentClipboardExtension::FExtensionTextFactory::CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const
{
	const bool bIsComponentWidgetPair = ObjectClass->IsChildOf(UUIComponentWidgetPair::StaticClass());
	return bIsComponentWidgetPair;
}

void FUIComponentClipboardExtension::FExtensionTextFactory::ProcessConstructedObject(UObject* NewObject)
{
	check(NewObject);

	if (UUIComponentWidgetPair* ComponentWidgetPair = Cast<UUIComponentWidgetPair>(NewObject))
	{
		NewComponentWidgetPairs.Add(ComponentWidgetPair);
	}
}
void FUIComponentClipboardExtension::FExtensionTextFactory::AddProcessedComponentsTo(const UWidget* Widget, FName OldWidgetName)
{
	if (IsEmpty())
	{
		return;
	}

	UWidgetBlueprint* WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(Widget);	
	// Here we call request instead of GetExtension to make sure to create one if it doesn't exist.
	if (UUIComponentWidgetBlueprintExtension* ComponentExtension = WidgetBlueprint ? UUIComponentWidgetBlueprintExtension::RequestExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint) : nullptr)
	{
		for (const UUIComponentWidgetPair* Pair : NewComponentWidgetPairs)
		{
			UUIComponent* NewComponent = Pair->Component;
			if (NewComponent && Pair->WidgetName == OldWidgetName)
			{
				ComponentExtension->AddOrReplaceComponent(NewComponent, Widget->GetFName());
			}
		}
	}	
}