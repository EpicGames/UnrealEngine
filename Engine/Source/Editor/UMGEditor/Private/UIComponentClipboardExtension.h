// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories.h"
#include "IHasClipboardExtensibility.h"
#include "UIComponentClipboardExtension.generated.h"

class UWidget;
class UWidgetBlueprint;
class UUIComponent;

UCLASS()
class UUIComponentWidgetPair : public UObject
{
	GENERATED_BODY()
	
public:

	UUIComponentWidgetPair(){};

	UPROPERTY()
	FName WidgetName;

	UPROPERTY(instanced)
	TObjectPtr<UUIComponent> Component;
};

class FUIComponentClipboardExtension
	: public IClipboardExtension
{
	//~ Begin IClipboardExtension overrides
	virtual void AppendToClipboard(const UWidget* Widget, const FExportArgs& ExportArgs) override;
	virtual bool CanAppendToClipboard(const UWidget* Widget) const override;
	virtual void ProcessImportedText(const UWidgetBlueprint* WidgetBlueprint, const FString& TextToImport, UPackage*& TempPackage) override;
	virtual bool CanImportFromClipboard(const UWidget* Widget) const override;
	virtual void ImportDataToWidget(const UWidget* Widget, FName OldWidgetName) override;
	virtual bool CanWidgetAcceptPaste(const UWidget* Widget) const override;
	//~ End IClipboardExtension overrides

private:

	class FExtensionTextFactory : public FCustomizableTextObjectFactory
	{
	public:
		FExtensionTextFactory();

		//~ Begin FCustomizableTextObjectFactory overrides
		virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override;
		virtual void ProcessConstructedObject(UObject* NewObject) override;
		//~ End FCustomizableTextObjectFactory overrides

		void AddProcessedComponentsTo(const UWidget* Widget, FName OldWidgetName);

		void Reinit()
		{
			NewComponentWidgetPairs.Empty();
		}

		bool IsEmpty() const
		{			
			return NewComponentWidgetPairs.IsEmpty();	
		}

	private:
		TArray<UUIComponentWidgetPair*> NewComponentWidgetPairs;
	};

	FExtensionTextFactory Factory;
};
