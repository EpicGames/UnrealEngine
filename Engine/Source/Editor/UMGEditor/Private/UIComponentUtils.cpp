// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIComponentUtils.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetTree.h"
#include "UMGEditorModule.h"
#include "ClassViewerModule.h"
#include "CoreGlobals.h"
#include "UIComponentWidgetBlueprintExtension.h"
#include "Extensions/UIComponent.h"
#include "WidgetBlueprintEditor.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "UMG"

FClassViewerInitializationOptions FUIComponentUtils::CreateClassViewerInitializationOptions(const TSet<const UClass*>& ExistingComponentClasses)
{
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	TSharedPtr<FUIComponentUtils::FUIComponentClassFilter> Filter = MakeShared<FUIComponentUtils::FUIComponentClassFilter>();
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown | CLASS_Abstract;
	Filter->AllowedChildrenOfClasses.Add(UUIComponent::StaticClass());

	// Exclude classes that are in the same inheritance hierarchy as any already-present component.
	// This prevents adding both a base and derived UIComponent to the same widget.
	Filter->ExcludedHierarchyClasses = ExistingComponentClasses;

	return Options;
}

void FUIComponentUtils::OnWidgetRenamed(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const FName& OldVarName, const FName& NewVarName)
{
	// On a Widget rename in the Editor we update the Widget names in UI Components extensions	
	if (UUIComponentWidgetBlueprintExtension* Extension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		Extension->RenameWidget(OldVarName, NewVarName);
	}

	if (BlueprintEditor)
	{
		if (const UUserWidget* PreviewWidget = BlueprintEditor->GetPreview())
		{
			if (UUIComponentUserWidgetExtension* UserWidgetExtension = PreviewWidget->GetExtension<UUIComponentUserWidgetExtension>())
			{
				UserWidgetExtension->RenameWidget(OldVarName, NewVarName);
			}
		}
	}
}

void FUIComponentUtils::ReplaceComponentVariableReferences(UWidgetBlueprint* WidgetBlueprint, const FName& OldWidgetName, const FName& NewWidgetName)
{
	if (UUIComponentWidgetBlueprintExtension* Extension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		Extension->ReplaceComponentVariableReferences(OldWidgetName, NewWidgetName);
	}
}

UWidget* FUIComponentUtils::AddComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClass, const FName WidgetName, FText& OutErrorMessage)
{
	if (ComponentClass == nullptr)
	{
		OutErrorMessage = LOCTEXT("AddComponentMissingComponentClass", "Failed to create UI component: missing ComponentClass.");
		return nullptr;
	}
	const FScopedTransaction Transaction( FText::Format( LOCTEXT("AddComponentTransaction", "Add Component {0} to {1}"), FText::FromString(ComponentClass->GetName()), FText::FromName(WidgetName)));
	UUIComponentWidgetBlueprintExtension* WidgetBlueprintExtension = UUIComponentWidgetBlueprintExtension::RequestExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint);
	if (!WidgetBlueprintExtension)
	{
		OutErrorMessage = LOCTEXT("AddComponentExtensionFailed", "Failed to create UI component: request extension failed.");
		return nullptr;
	}

	UUIComponent* ComponentArchetype = WidgetBlueprintExtension->AddComponent(ComponentClass, WidgetName, OutErrorMessage);
	if (!ComponentArchetype)
	{
		return nullptr;
	}

	UUserWidget* PreviewWidget = BlueprintEditor ? BlueprintEditor->GetPreview() : nullptr;
	if (PreviewWidget)
	{
		// If the extension do not exist, we create it which will create a copy of the component we just added.
		if (UUIComponentUserWidgetExtension* UserWidgetExtension = WidgetBlueprintExtension->GetOrCreateExtension(PreviewWidget))
		{
			UserWidgetExtension->CreateAndAddComponent(ComponentArchetype, WidgetName);
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return WidgetBlueprint->WidgetTree->FindWidget(WidgetName);
}

bool FUIComponentUtils::RemoveComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClass, const FName WidgetName, FText& OutErrorMessage)
{
	if (ComponentClass == nullptr)
	{
		OutErrorMessage = LOCTEXT("RemoveComponentMissingComponentClass", "Failed to Remove UI component: missing ComponentClass.");
		return false;
	}
	
	UUIComponentWidgetBlueprintExtension* WidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint);
	if (!WidgetBlueprintExtension)
	{
		OutErrorMessage = LOCTEXT("RemoveComponentNoExtension", "No UI components found on this widget blueprint.");
		return false;
	}

	if (!WidgetBlueprintExtension->GetComponent(ComponentClass, WidgetName))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("RemoveComponentNotFound", "UI component '{0}' not found on widget '{1}'."),
			FText::FromString(ComponentClass->GetName()), FText::FromName(WidgetName));
		return false;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveComponentTransaction", "Remove Component {0} from {1}"), FText::FromString(ComponentClass->GetName()), FText::FromName(WidgetName)));
	WidgetBlueprintExtension->RemoveComponent(ComponentClass, WidgetName);

	// Also Remove it from the Preview
	if (BlueprintEditor)
	{
		if (const UUserWidget* PreviewWidget = BlueprintEditor->GetPreview())
		{
			if (UUIComponentUserWidgetExtension* UserWidgetExtension = PreviewWidget->GetExtension<UUIComponentUserWidgetExtension>())
			{
				UserWidgetExtension->RemoveComponent(ComponentClass, WidgetName);
			}
		}
	}
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return true;
}

bool FUIComponentUtils::MoveComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClassToMove, const UClass* RelativeToComponentClass, const FName WidgetName, bool bMoveAfter, FText& OutErrorMessage)
{	
	UUIComponentWidgetBlueprintExtension* WidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint);
	if (!WidgetBlueprintExtension)
	{
		OutErrorMessage = LOCTEXT("MoveComponentNoExtension", "No UI components found on this widget blueprint.");
		return false;
	}

	if (!WidgetBlueprintExtension->GetComponent(ComponentClassToMove, WidgetName))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("MoveComponentToMoveNotFound", "UI component '{0}' not found on widget '{1}'."),
			FText::FromString(ComponentClassToMove->GetName()), FText::FromName(WidgetName));
		return false;
	}

	if (!WidgetBlueprintExtension->GetComponent(RelativeToComponentClass, WidgetName))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("MoveComponentRelativeNotFound", "Relative UI component '{0}' not found on widget '{1}'."),
			FText::FromString(RelativeToComponentClass->GetName()), FText::FromName(WidgetName));
		return false;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("MoveComponentTransaction", "Move Component {0} in {1}"), FText::FromString(ComponentClassToMove->GetName()), FText::FromName(WidgetName)));
	WidgetBlueprintExtension->MoveComponent(WidgetName, ComponentClassToMove, RelativeToComponentClass, bMoveAfter);

	// Also Move it in the Preview
	if (BlueprintEditor)
	{
		if (const UUserWidget* PreviewWidget = BlueprintEditor->GetPreview())
		{
			if (UUIComponentUserWidgetExtension* UserWidgetExtension = PreviewWidget->GetExtension<UUIComponentUserWidgetExtension>())
			{
				UserWidgetExtension->MoveComponent(WidgetName, ComponentClassToMove, RelativeToComponentClass, bMoveAfter);
			}
		}
	}
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return true;
}

bool FUIComponentUtils::FUIComponentClassFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs)
{
	if (InClass->HasAnyClassFlags(DisallowedClassFlags))
	{
		return false;
	}
	if (InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) == EFilterReturn::Failed)
	{
		return false;
	}
	// Reject classes that are ancestors or descendants of any already-present component class.
	for (const UClass* ExistingClass : ExcludedHierarchyClasses)
	{
		if (InClass->IsChildOf(ExistingClass) || ExistingClass->IsChildOf(InClass))
		{
			return false;
		}
	}
	return true;
}

bool FUIComponentUtils::FUIComponentClassFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs)
{
	// Note: unloaded classes cannot be checked against ExcludedHierarchyClasses since UClass is unavailable.
	// The AddComponent guard in UUIComponentWidgetBlueprintExtension will catch any hierarchy conflicts at add time.
	return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
		&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FUIComponentUtils::AddComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClass, const FName WidgetName)
{
	FText OutErrorMessage;
	if (!AddComponent(&BlueprintEditor.Get(), BlueprintEditor->GetWidgetBlueprintObj(), ComponentClass, WidgetName, OutErrorMessage))
	{
		UE_LOGF(LogUMGEditor, Warning,  "%ls", *OutErrorMessage.ToString());
	}
}

void FUIComponentUtils::RemoveComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClass, const FName WidgetName)
{
	FText OutErrorMessage;
	if (!RemoveComponent(&BlueprintEditor.Get(), BlueprintEditor->GetWidgetBlueprintObj(), ComponentClass, WidgetName, OutErrorMessage))
	{
		UE_LOGF(LogUMGEditor, Warning,  "%ls", *OutErrorMessage.ToString());
	}
}

void FUIComponentUtils::MoveComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClassToMove, const UClass* RelativeToComponentClass, const FName WidgetName, bool bMoveAfter)
{
	FText OutErrorMessage;
	if (!MoveComponent(&BlueprintEditor.Get(), BlueprintEditor->GetWidgetBlueprintObj(), ComponentClassToMove, RelativeToComponentClass, WidgetName, bMoveAfter, OutErrorMessage))
	{
		UE_LOGF(LogUMGEditor, Warning,  "%ls", *OutErrorMessage.ToString());
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE