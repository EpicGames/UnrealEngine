// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIComponentWidgetBlueprintExtension.h"

#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentContainer.h"
#include "Extensions/UIComponentWidgetBlueprintGeneratedClassExtension.h"
#include "Extensions/UIComponentUserWidgetExtension.h"

#include "Blueprint/WidgetTree.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/ObjectInstancingGraph.h"
#include "WidgetBlueprintEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIComponentWidgetBlueprintExtension)

#define LOCTEXT_NAMESPACE "UIComponentWidgetBlueprintExtension"


const FName UUIComponentWidgetBlueprintExtension::MD_ComponentVariable = "GeneratedForComponent";

UUIComponentWidgetBlueprintExtension::UUIComponentWidgetBlueprintExtension(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ComponentContainer = CreateDefaultSubobject<UUIComponentContainer>(TEXT("ComponentContainer"));
	ComponentContainer->SetFlags(RF_Transactional | RF_ArchetypeObject);
}

UUIComponentContainer* UUIComponentWidgetBlueprintExtension::DuplicateContainer(UObject* Outer) const
{
	ensure(ComponentContainer->HasAnyFlags(RF_ArchetypeObject));
			
	FObjectInstancingGraph ObjectInstancingGraph;
	UUIComponentContainer* NewContainer = NewObject<UUIComponentContainer>(Outer, ComponentContainer->GetClass(), NAME_None, RF_Transactional, ComponentContainer, false);
	return NewContainer;
}

void UUIComponentWidgetBlueprintExtension::RemoveComponent(const UClass* ComponentClass, FName OwnerName)
{
	// Remove the Component
	if (ensure(ComponentContainer))
	{
		ComponentContainer->RemoveAllComponentsOfType(ComponentClass, OwnerName);
	}
}

void UUIComponentWidgetBlueprintExtension::MoveComponent(FName OwnerName, const UClass* ComponentClass, const UClass* RelativeToComponentClass, bool bMoveAfter)
{
	// Remove the Component
	if (ensure(ComponentContainer))
	{
		ComponentContainer->MoveComponent(OwnerName, ComponentClass, RelativeToComponentClass, bMoveAfter);
	}
}

TArray<UUIComponent*> UUIComponentWidgetBlueprintExtension::GetComponentsFor(const UWidget* Target) const
{
	TArray<UUIComponent*> Components;
	if (ensure(Target))
	{
		ComponentContainer->ForEachComponentTarget([Target, &Components](const FUIComponentTarget& ComponentTarget){
			if (ComponentTarget.GetTargetName() == Target->GetFName())			
			{
				Components.Push(ComponentTarget.GetComponent());
			}
		});
	}
	return Components;	
}

UUIComponent* UUIComponentWidgetBlueprintExtension::GetComponent(const UClass* ComponentClass, FName OwnerName) const
{
	if (ComponentContainer)
	{
		return ComponentContainer->GetComponent(ComponentClass, OwnerName);
	}
	return nullptr;
}

EDataValidationResult UUIComponentWidgetBlueprintExtension::ValidateComponents(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;

	if (!ComponentContainer)
	{
		return Result;
	}

	const UWidgetTree* WidgetTree = nullptr;
	if (const UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprint())
	{
		WidgetTree = WidgetBlueprint->WidgetTree;
	}

	ComponentContainer->ForEachComponentTarget([&](const FUIComponentTarget& ComponentTarget)
	{
		UUIComponent* Component = ComponentTarget.GetComponent();
		if (!Component)
		{
			return;
		}

		UWidget* OwnerWidget = nullptr;
		if (WidgetTree)
		{
			OwnerWidget = const_cast<FUIComponentTarget&>(ComponentTarget).Resolve(WidgetTree);
		}

		Result = CombineDataValidationResults(Result, Component->IsDataValidForOwner(OwnerWidget, Context));
	});

	return Result;
}

void UUIComponentWidgetBlueprintExtension::RenameWidget(const FName& OldVarName, const FName& NewVarName)
{
	if (ensure(ComponentContainer))
	{
		ComponentContainer->RenameWidget(OldVarName, NewVarName);
	}
}

void UUIComponentWidgetBlueprintExtension::ReplaceComponentVariableReferences(const FName& OldWidgetName, const FName& NewWidgetName)
{
	// Component variables are generated at compile time with a derived name "ComponentName_WidgetName".
	// When a widget is renamed, the generated variable name changes, but graph node references still use
	// the old compound name. This must be called after MarkBlueprintAsStructurallyModified to avoid
	// BroadcastChanged() reverting the graph fixup.
	if (ensure(ComponentContainer))
	{
		if (UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprint())
		{
			ComponentContainer->ForEachComponentTarget([&](const FUIComponentTarget& ComponentTarget)
			{
				if (ComponentTarget.GetTargetName() == NewWidgetName && ComponentTarget.GetComponent())
				{
					const FName OldCompVarName = UUIComponentContainer::GetPropertyNameForComponent(ComponentTarget.GetComponent(), OldWidgetName);
					const FName NewCompVarName = UUIComponentContainer::GetPropertyNameForComponent(ComponentTarget.GetComponent(), NewWidgetName);
					FBlueprintEditorUtils::ReplaceVariableReferences(WidgetBlueprint, OldCompVarName, NewCompVarName);
				}
			});
		}
	}
}

bool UUIComponentWidgetBlueprintExtension::VerifyContainer(UUserWidget* UserWidget) const
{
	UUIComponentUserWidgetExtension* UserWidgetExtension = UserWidget->GetExtension<UUIComponentUserWidgetExtension>();
	if (UserWidgetExtension)
	{
		UserWidgetExtension->CleanupComponents();
		
		TArray<FUIComponentTarget> Components;
		ComponentContainer->ForEachComponentTarget([&Components](const FUIComponentTarget& ComponentTarget){
			Components.Push(ComponentTarget);
		});

		while(Components.Num() > 0)
		{
			FUIComponentTarget& Target = Components[Components.Num()-1];
			if (!UserWidgetExtension->GetComponent(Target.GetComponent()->GetClass(), Target.GetTargetName()))
			{
				return false;
			}
			else
			{
				Components.RemoveAt(Components.Num()-1);
			}
		}
		return true;
	}
	return false;	
}

UUIComponentUserWidgetExtension* UUIComponentWidgetBlueprintExtension::GetOrCreateExtension(UUserWidget* PreviewWidget)
{
	UUIComponentUserWidgetExtension* UserWidgetExtension = PreviewWidget->GetExtension<UUIComponentUserWidgetExtension>();
	// If the extension do not exist, we create it which will create a copy of the component we just added.
	if (!UserWidgetExtension)
	{
		UserWidgetExtension = PreviewWidget->AddExtension<UUIComponentUserWidgetExtension>();
		UserWidgetExtension->InitializeContainer(DuplicateContainer(UserWidgetExtension));
	}
	else
	{
		VerifyContainer(PreviewWidget);
	}
	
	return UserWidgetExtension;
}

UUIComponent* UUIComponentWidgetBlueprintExtension::AddComponent(const UClass* ComponentClass, FName OwnerName, FText& OutErrorMessage)
{
	// Add the Component to the WidgetBlueprint
	check(ComponentContainer);

	// Reject if a component in the same class hierarchy already exists on this widget.
	bool bHierarchyConflict = false;
	ComponentContainer->ForEachComponentTarget([&](const FUIComponentTarget& Target)
	{
		if (Target.GetTargetName() == OwnerName && Target.GetComponent())
		{
			const UClass* ExistingClass = Target.GetComponent()->GetClass();
			if (ExistingClass->IsChildOf(ComponentClass) ||
				ComponentClass->IsChildOf(ExistingClass))
			{
				if (ExistingClass == ComponentClass)
				{
					OutErrorMessage = FText::Format(
						LOCTEXT("ComponentAlreadyPresent", "Cannot add UI component '{0}' to widget '{1}': an instance of this component is already present."),
						FText::FromString(ComponentClass->GetName()), FText::FromName(OwnerName));
				}
				else
				{
					OutErrorMessage = FText::Format(
						LOCTEXT("ComponentWithSameClassHierarchyAlreadyExist", "Cannot add UI component '{0}' to widget '{1}': a component in the same class hierarchy ('{2}') is already present. Adding both a base and derived UIComponent to the same widget is not supported."),
						FText::FromString(ComponentClass->GetName()), FText::FromName(OwnerName), FText::FromString(ExistingClass->GetName()));
				}

				bHierarchyConflict = true;
			}
		}
	});

	if (bHierarchyConflict)
	{
		return nullptr;
	}

	if (UUIComponent* NewComponent = NewObject<UUIComponent>(ComponentContainer, ComponentClass, NAME_None, RF_ArchetypeObject))
	{
		Modify();
		ComponentContainer->AddComponent(OwnerName, NewComponent);
		return NewComponent;
	}
	OutErrorMessage = FText::Format(
						LOCTEXT("NewComponentFailed", "Cannot add UI component '{0}' to widget '{1}': failed to create the UI component."),
						FText::FromString(ComponentClass->GetName()), FText::FromName(OwnerName));
	return nullptr;
}

UUIComponent* UUIComponentWidgetBlueprintExtension::AddOrReplaceComponent(UUIComponent* Component, FName OwnerName)
{
	// Add the Component to the WidgetBlueprint
	check(ComponentContainer);
	Modify();
	// If we already have a component of that type we replace it
	ComponentContainer->RemoveAllComponentsOfType(Component->GetClass(), OwnerName);
	ComponentContainer->AddComponent(OwnerName, Component);
	Component->Rename(nullptr, ComponentContainer, RF_ArchetypeObject);
	return Component;
}

void UUIComponentWidgetBlueprintExtension::HandleBeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext)
{
	Super::HandleBeginCompilation(InCreationContext);

	CompilerContext = &InCreationContext;
}

void UUIComponentWidgetBlueprintExtension::HandleCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO)
{
	// Here we handle Widget Deletion. Go through all Widgets referenced by Components and remove them if they no longer exist.
	const UWidgetTree* WidgetTree = GetWidgetBlueprint()->WidgetTree; 	
	
	if (ComponentContainer && WidgetTree)
	{	
		ComponentContainer->CleanupUIComponents(WidgetTree);
	}
}

void UUIComponentWidgetBlueprintExtension::HandlePopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
{
	ComponentContainer->ForEachComponentTarget([this, &Context](const FUIComponentTarget& ComponentTarget)
		{
			if (ensure(ComponentTarget.GetComponent()))
			{
				FBPVariableDescription ComponentVariableDesc;
				ComponentVariableDesc.VarName = UUIComponentContainer::GetPropertyNameForComponent(ComponentTarget.GetComponent(), ComponentTarget.GetTargetName());
				ComponentVariableDesc.VarGuid = FGuid::NewGuid();
				ComponentVariableDesc.VarType = FEdGraphPinType(UEdGraphSchema_K2::PC_Object, NAME_None, ComponentTarget.GetComponent()->GetClass(), EPinContainerType::None, true, FEdGraphTerminalType());
				ComponentVariableDesc.FriendlyName = ComponentTarget.GetComponent()->GetName();
				ComponentVariableDesc.PropertyFlags = (CPF_PersistentInstance | CPF_InstancedReference | CPF_BlueprintVisible | CPF_BlueprintReadOnly | CPF_Transient | CPF_RepSkip);
				ComponentVariableDesc.SetMetaData(UUIComponentWidgetBlueprintExtension::MD_ComponentVariable, TEXT("true"));
				ComponentVariableDesc.SetMetaData(FBlueprintMetadata::MD_FieldNotify, TEXT("true"));
				ComponentVariableDesc.Category = FText::FromString(TEXT("Component"));
				Context.AddGeneratedVariable(MoveTemp(ComponentVariableDesc));
			}
		});
}

void UUIComponentWidgetBlueprintExtension::HandleFinishCompilingClass(UWidgetBlueprintGeneratedClass* Class)
{
	Super::HandleFinishCompilingClass(Class);

	// If we do not have Components, do not add the extension to the Generated Class
	if (ensure(CompilerContext) && !ComponentContainer->IsEmpty())
	{
		if (UWidgetBlueprintGeneratedClass* NewWidgetGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(CompilerContext->NewClass))
		{
			UUIComponentWidgetBlueprintGeneratedClassExtension* NewExtension = NewObject<UUIComponentWidgetBlueprintGeneratedClassExtension>(NewWidgetGeneratedClass);
			CompilerContext->AddExtension(NewWidgetGeneratedClass, NewExtension);
			NewExtension->InitializeContainer(DuplicateContainer(NewExtension));
		}		
	}
}

bool UUIComponentWidgetBlueprintExtension::HandleValidateGeneratedClass(UWidgetBlueprintGeneratedClass* Class)
{
	const UWidgetBlueprintGeneratedClass* GeneratedClass = Class;
	if ( !ensure(GeneratedClass) )
	{
		return false;
	}
	const UWidgetBlueprint* Blueprint = GetWidgetBlueprint();
	if ( !ensure(Blueprint) )
	{
		return false;
	}

	// Validate with WidgetTree if all widget exists.
	if (UWidgetBlueprintGeneratedClass* NewWidgetGeneratedClass = Cast<UWidgetBlueprintGeneratedClass>(CompilerContext->NewClass))
	{
		if (const UUIComponentWidgetBlueprintGeneratedClassExtension* Extension = NewWidgetGeneratedClass->GetExtension<UUIComponentWidgetBlueprintGeneratedClassExtension>())
		{
			return Extension->VerifyAllWidgetsExists(NewWidgetGeneratedClass->GetWidgetTreeArchetype());
		}
	}
	
	return true;
}

void UUIComponentWidgetBlueprintExtension::HandleEndCompilation()
{
	CompilerContext = nullptr;
}

#undef LOCTEXT_NAMESPACE
