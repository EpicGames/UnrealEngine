// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGEditorActions.h"

#define LOCTEXT_NAMESPACE "UMGEditorCommands"

void FUMGEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateNativeBaseClass, "Create Native Base Class", "Create a native base class for this widget, using the current parent as the parent of the native class.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(ExportAsPNG, "Export as Image", "Export the current widget blueprint as .png format.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetImageAsThumbnail, "Capture and Set as Thumbnail", "Captures the current state of the widget blueprint and sets it as the thumbnail for this asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearCustomThumbnail, "Clear Thumbnail", "Removes the image used as thumbnail and enables automatic thumbnail generation.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(DismissOnCompile_ErrorsAndWarnings, "No Errors or Warnings", "Automatically dismiss compile tab when no errors or warnings occur", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DismissOnCompile_Errors, "No Errors", "Automatically dismiss compile tab when no errors occur", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DismissOnCompile_Warnings, "No Warnings", "Automatically dismiss compile tab when no warnings occur (even if there are errors)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DismissOnCompile_Never, "Never", "Never automatically dismiss the compile tab", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(CreateOnCompile_ErrorsAndWarnings, "On Errors or Warnings", "Automatically create compile tab when errors or warnings occur", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CreateOnCompile_Errors, "On Errors", "Automatically create compile tab when errors occur", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CreateOnCompile_Warnings, "On Warnings", "Automatically create compile tab when warnings occur (but not errors)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CreateOnCompile_Never, "Never", "Never automatically create the compile tab", EUserInterfaceActionType::RadioButton, FInputChord());

#if PLATFORM_MAC
	// On mac command and ctrl are automatically swapped. Command + Space is spotlight search so we use ctrl+space on mac to avoid the conflict
	UI_COMMAND(OpenAnimDrawer, "Open Animation Browser Drawer", "Opens the animation drawer from the status bar", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command | EModifierKey::Shift, EKeys::SpaceBar));
#else
	UI_COMMAND(OpenAnimDrawer, "Open Animation Browser Drawer", "Opens the animation drawer from the status bar", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::SpaceBar));
#endif
}

#undef LOCTEXT_NAMESPACE