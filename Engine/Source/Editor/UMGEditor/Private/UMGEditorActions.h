// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

/**
 * Unreal UMG editor actions
 */
class FUMGEditorCommands : public TCommands<FUMGEditorCommands>
{

public:
	FUMGEditorCommands() : TCommands<FUMGEditorCommands>
	(
		"UMGEditor", // Context name for fast lookup
		NSLOCTEXT("Contexts", "UMGEditor", "UMG Editor"), // Localized context name for displaying
		"EditorViewport",  // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{
	}
	
	/**
	 * UMG Editor Commands
	 */
	TSharedPtr< FUICommandInfo > CreateNativeBaseClass;
	TSharedPtr< FUICommandInfo > ExportAsPNG;
	TSharedPtr< FUICommandInfo > SetImageAsThumbnail;
	TSharedPtr< FUICommandInfo > ClearCustomThumbnail;
	TSharedPtr< FUICommandInfo > OpenAnimDrawer;

	TSharedPtr< FUICommandInfo > DismissOnCompile_ErrorsAndWarnings;
	TSharedPtr< FUICommandInfo > DismissOnCompile_Errors;
	TSharedPtr< FUICommandInfo > DismissOnCompile_Warnings;
	TSharedPtr< FUICommandInfo > DismissOnCompile_Never;

	TSharedPtr< FUICommandInfo > CreateOnCompile_ErrorsAndWarnings;
	TSharedPtr< FUICommandInfo > CreateOnCompile_Errors;
	TSharedPtr< FUICommandInfo > CreateOnCompile_Warnings;
	TSharedPtr< FUICommandInfo > CreateOnCompile_Never;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

public:
};
