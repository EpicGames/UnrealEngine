// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGEditorModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "Editor.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Settings/WidgetDesignerSettings.h"
#include "WidgetBlueprint.h"
#include "Animation/WidgetAnimation.h"
#include "EdGraphUtilities.h"
#include "Graph/GraphFunctionDetailsCustomization.h"
#include "Graph/UMGGraphPanelPinFactory.h"
#include "Graph/GraphVariableDetailsCustomization.h"

#include "AssetToolsModule.h"
#include "IAssetTypeActions.h"
#include "KismetCompilerModule.h"
#include "WidgetBlueprintCompiler.h"

#include "ISequencerModule.h"
#include "Animation/MarginTrackEditor.h"
#include "Animation/Sequencer2DTransformTrackEditor.h"
#include "Animation/WidgetMaterialTrackEditor.h"
#include "Animation/MovieSceneSequenceEditor_WidgetAnimation.h"
#include "Interfaces/IEditorStyleModule.h"
#include "IUMGModule.h"
#include "Designer/DesignerCommands.h"
#include "Navigation/SWidgetDesignerNavigation.h"

#include "ClassIconFinder.h"

#include "ISettingsModule.h"
#include "SequencerSettings.h"
#include "Settings/LevelEditorPlaySettings.h"

#include "BlueprintEditorModule.h"
#include "PropertyEditorModule.h"
#include "Customizations/DynamicEntryBoxDetails.h"
#include "Customizations/IBlueprintWidgetCustomizationExtender.h"
#include "Customizations/ListViewBaseDetails.h"
#include "Customizations/UIComponentCustomizationExtender.h"
#include "WidgetBlueprintThumbnailRenderer.h"
#include "Customizations/WidgetThumbnailCustomization.h"
#include "Widgets/SBindWidgetView.h"
#include "MovieSceneDynamicBindingUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

#include "Extensions/UIComponentContainer.h"
#include "Extensions/UIComponentContainerDesignerExtension.h"
#include "UIComponentUtils.h"
#include "UIComponentClipboardExtension.h"

#define LOCTEXT_NAMESPACE "UMG"

DEFINE_LOG_CATEGORY(LogUMGEditor);

const FName UMGEditorAppIdentifier = FName(TEXT("UMGEditorApp"));
static TAutoConsoleVariable<bool> CVarThumbnailRenderEnable(
	TEXT("UMG.ThumbnailRenderer.Enable"),
	true,
	TEXT("Option to enable/disable thumbnail rendering.")
);

class FUMGEditorModule : public IUMGEditorModule, public FGCObject
{
public:
	/** Constructor, set up console commands and variables **/
	FUMGEditorModule()
		: Settings(nullptr)
		, bThumbnailRenderersRegistered(false)
		, bOnPostEngineInitHandled(false)
		, bCachedArePostBuffersEnabled(false)
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		FModuleManager::LoadModuleChecked<IEditorStyleModule>("EditorStyle");
		FModuleManager::LoadModuleChecked<IUMGModule>("UMG");

		// Any attempt to use GEditor right now will fail as it hasn't been initialized yet. Waiting for post engine init resolves that.
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FUMGEditorModule::OnPostEngineInit);
		FEditorDelegates::StartPIE.AddRaw(this, &FUMGEditorModule::OnStartPIE);
		FEditorDelegates::EndPIE.AddRaw(this, &FUMGEditorModule::OnEndPIE);

		if (GIsEditor)
		{
			FDesignerCommands::Register();
			FBindWidgetCommands::Register();
		}

		MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
		ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();
		DesignerExtensibilityManager = MakeShared<FDesignerExtensibilityManager>();

		DesignerExtensibilityManager->AddDesignerExtensionFactory(SWidgetDesignerNavigation::MakeDesignerExtension());
		DesignerExtensibilityManager->AddDesignerExtensionFactory(MakeShared<FUIComponentContainerDesignerExtensionFactory>());

		PropertyBindingExtensibilityManager = MakeShared<FPropertyBindingExtensibilityManager>();
		ClipboardExtensibilityManager = MakeShared<FClipboardExtensibilityManager>();

		ComponentClipboardExtension = MakeShared<FUIComponentClipboardExtension>();
		GetClipboardExtensibilityManager()->AddExtension(ComponentClipboardExtension.ToSharedRef());
		
		WidgetDragDropExtensibilityManager = MakeShared<FWidgetDragDropExtensibilityManager>();
		WidgetContextMenuExtensibilityManager = MakeShared<FWidgetContextMenuExtensibilityManager >();

		UIComponentCustomizationExtender = FUIComponentCustomizationExtender::MakeInstance();
		AddWidgetCustomizationExtender(UIComponentCustomizationExtender.ToSharedRef());

		// Register widget blueprint compiler we do this no matter what.
		IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		KismetCompilerModule.GetCompilers().Add(&WidgetBlueprintCompiler);
		KismetCompilerModule.OverrideBPTypeForClass(UUserWidget::StaticClass(), UWidgetBlueprint::StaticClass());

		// Add Customization for variable in Graph editor
		if (FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>("Kismet"))
		{
			//BlueprintVariableCustomizationHandle = BlueprintEditorModule->RegisterVariableCustomization(FProperty::StaticClass(), FOnGetVariableCustomizationInstance::CreateStatic(&FGraphVariableDetailsCustomization::MakeInstance));
			//BlueprintFunctionCustomizationHandle = BlueprintEditorModule->RegisterFunctionCustomization(UK2Node_FunctionEntry::StaticClass(), FOnGetFunctionCustomizationInstance::CreateStatic(&FGraphFunctionDetailsCustomization::MakeInstance));
		}
		else
		{
			ModuleChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FUMGEditorModule::HandleModuleChanged);
		}

		FKismetCompilerContext::RegisterCompilerForBP(UWidgetBlueprint::StaticClass(), &UWidgetBlueprint::GetCompilerForWidgetBP );

		// Register with the sequencer module that we provide auto-key handlers.
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		SequenceEditorHandle                              = SequencerModule.RegisterSequenceEditor(UWidgetAnimation::StaticClass(), MakeUnique<FMovieSceneSequenceEditor_WidgetAnimation>());
		MarginTrackEditorCreateTrackEditorHandle          = SequencerModule.RegisterPropertyTrackEditor<FMarginTrackEditor>();
		TransformTrackEditorCreateTrackEditorHandle       = SequencerModule.RegisterPropertyTrackEditor<F2DTransformTrackEditor>();
		WidgetMaterialTrackEditorCreateTrackEditorHandle  = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FWidgetMaterialTrackEditor::CreateTrackEditor));

		RegisterSettings();

		// Class detail customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(TEXT("DynamicEntryBoxBase"), FOnGetDetailCustomizationInstance::CreateStatic(&FDynamicEntryBoxBaseDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(TEXT("DynamicEntryBox"), FOnGetDetailCustomizationInstance::CreateStatic(&FDynamicEntryBoxDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(TEXT("ListViewBase"), FOnGetDetailCustomizationInstance::CreateStatic(&FListViewBaseDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(TEXT("WidgetBlueprint"), FOnGetDetailCustomizationInstance::CreateStatic(&FWidgetThumbnailCustomization::MakeInstance));
	
		GraphPanelPinFactory = MakeShared<FUMGGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

		CVarThumbnailRenderEnable->AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FUMGEditorModule::ThumbnailRenderingEnabled));

		FixupDynamicBindingPayloadParameterNameHandle = UMovieScene::FixupDynamicBindingPayloadParameterNameEvent.AddStatic(FixupPayloadParameterNameForDynamicBinding);
		FixupWidgetDynamicBindingsHandle = UWidgetAnimation::FixupWidgetDynamicBindingsEvent.AddStatic(FixupWidgetDynamicBindings);
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
		FEditorDelegates::StartPIE.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);

		FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);

		if (UObjectInitialized() && bThumbnailRenderersRegistered && IConsoleManager::Get().FindConsoleVariable(TEXT("UMGEditor.ThumbnailRenderer.Enable"))->GetBool())
		{
			UThumbnailManager::Get().UnregisterCustomRenderer(UWidgetBlueprint::StaticClass());
		}

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
		FBlueprintEditorUtils::OnRenameVariableReferencesEvent.RemoveAll(this);

		if (IKismetCompilerInterface* KismetCompilerModule = FModuleManager::GetModulePtr<IKismetCompilerInterface>("KismetCompiler"))
		{
			KismetCompilerModule->GetCompilers().Remove(&WidgetBlueprintCompiler);
		}

		// Remove Customization for variable in Graph editor
		if (BlueprintVariableCustomizationHandle.IsValid() || BlueprintFunctionCustomizationHandle.IsValid())
		{
			if (FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>("Kismet"))
			{
				//BlueprintEditorModule->UnregisterVariableCustomization(FProperty::StaticClass(), BlueprintVariableCustomizationHandle);
				//BlueprintEditorModule->UnregisterFunctionCustomization(UK2Node_FunctionEntry::StaticClass(), BlueprintFunctionCustomizationHandle);
			}
		}

		// Unregister all the asset types that we registered
		if ( FModuleManager::Get().IsModuleLoaded("AssetTools") )
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			for ( int32 Index = 0; Index < CreatedAssetTypeActions.Num(); ++Index )
			{
				AssetTools.UnregisterAssetTypeActions(CreatedAssetTypeActions[Index].ToSharedRef());
			}
		}
		CreatedAssetTypeActions.Empty();

		// Unregister sequencer track creation delegates
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>( "Sequencer" );
		if ( SequencerModule != nullptr )
		{
			SequencerModule->UnregisterSequenceEditor(SequenceEditorHandle);

			SequencerModule->UnRegisterTrackEditor( MarginTrackEditorCreateTrackEditorHandle );
			SequencerModule->UnRegisterTrackEditor( TransformTrackEditorCreateTrackEditorHandle );
			SequencerModule->UnRegisterTrackEditor( WidgetMaterialTrackEditorCreateTrackEditorHandle );
		}

		UnregisterSettings();

		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule != nullptr)
		{
			PropertyModule->UnregisterCustomClassLayout(TEXT("DynamicEntryBoxBase"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("DynamicEntryBox"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("ListViewBase"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("WidgetBlueprint"));
		}

		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);
		}

		RemoveWidgetCustomizationExtender(UIComponentCustomizationExtender.ToSharedRef());

		//// Unregister the setting
		//ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		//if ( SettingsModule != nullptr )
		//{
		//	SettingsModule->UnregisterSettings("Editor", "ContentEditors", "WidgetDesigner");
		//	SettingsModule->UnregisterSettings("Project", "Editor", "UMGEditor");
		//}

		UMovieScene::FixupDynamicBindingPayloadParameterNameEvent.Remove(FixupDynamicBindingPayloadParameterNameHandle);
		UWidgetAnimation::FixupWidgetDynamicBindingsEvent.Remove(FixupWidgetDynamicBindingsHandle);
	}

	/** Gets the extensibility managers for outside entities to extend gui page editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
	virtual TSharedPtr<FDesignerExtensibilityManager> GetDesignerExtensibilityManager() override { return DesignerExtensibilityManager; }
	virtual TSharedPtr<FPropertyBindingExtensibilityManager> GetPropertyBindingExtensibilityManager() override { return PropertyBindingExtensibilityManager; }
	virtual TSharedPtr<FClipboardExtensibilityManager> GetClipboardExtensibilityManager() override { return ClipboardExtensibilityManager; }
	virtual TSharedPtr<FWidgetDragDropExtensibilityManager> GetWidgetDragDropExtensibilityManager() override { return WidgetDragDropExtensibilityManager; }
	virtual TSharedPtr<FWidgetContextMenuExtensibilityManager> GetWidgetContextMenuExtensibilityManager() override { return WidgetContextMenuExtensibilityManager; }

	/** Register settings objects. */
	void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("UMGSequencerSettings"));

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "UMGSequencerSettings",
				LOCTEXT("UMGSequencerSettingsSettingsName", "UMG Sequence Editor"),
				LOCTEXT("UMGSequencerSettingsSettingsDescription", "Configure the look and feel of the UMG Sequence Editor."),
				Settings);	
		}
	}

	/** Unregister settings objects. */
	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "UMGSequencerSettings");
		}
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		if (Settings)
		{
			Collector.AddReferencedObject(Settings);
		}
	}

	virtual FString GetReferencerName() const override
	{
		return "UMGEditorModule";
	}

	virtual FWidgetBlueprintCompiler* GetRegisteredCompiler() override
	{
		return &WidgetBlueprintCompiler;
	}

	/** Register common tabs */
	virtual FOnRegisterTabs& OnRegisterTabsForEditor() override
	{
		return RegisterTabsForEditor;
	}

	virtual void AddWidgetEditorToolbarExtender(FWidgetEditorToolbarExtender&& InToolbarExtender) override
	{
		WidgetEditorToolbarExtenders.Add(MoveTemp(InToolbarExtender));
	}

	virtual TArrayView<FWidgetEditorToolbarExtender> GetAllWidgetEditorToolbarExtenders() override
	{
		return WidgetEditorToolbarExtenders;
	}

	virtual void AddWidgetCustomizationExtender(const TSharedRef<IBlueprintWidgetCustomizationExtender>& WidgetCustomizationExtender) override
	{
		WidgetCustomizationExtenders.AddUnique(WidgetCustomizationExtender);
	}

	virtual void RemoveWidgetCustomizationExtender(const TSharedRef<IBlueprintWidgetCustomizationExtender>& WidgetCustomizationExtender) override
	{
		WidgetCustomizationExtenders.RemoveSingleSwap(WidgetCustomizationExtender, EAllowShrinking::No);
	}

	virtual TArrayView<TSharedRef<IBlueprintWidgetCustomizationExtender>> GetAllWidgetCustomizationExtenders() override
	{
		return WidgetCustomizationExtenders;
	}

	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() override 
	{
		return RegisterLayoutExtensions; 
	}

	virtual void RegisterInstancedCustomPropertyTypeLayout(FTopLevelAssetPath Type, FOnGetInstancePropertyTypeCustomizationInstance Delegate) override
	{
		bool bIncluded = CustomPropertyTypeLayout.ContainsByPredicate([Type](const FCustomPropertyTypeLayout& Element) { return Element.Type == Type; });
		if (ensure(!bIncluded))
		{
			FCustomPropertyTypeLayout& Layout = CustomPropertyTypeLayout.AddDefaulted_GetRef();
			Layout.Type = Type;
			Layout.Delegate = MoveTemp(Delegate);
		}
	}

	virtual void UnregisterInstancedCustomPropertyTypeLayout(FTopLevelAssetPath Type) override
	{
		int32 IndexOf = CustomPropertyTypeLayout.IndexOfByPredicate([Type](const FCustomPropertyTypeLayout& Element){ return Element.Type == Type; });
		if (CustomPropertyTypeLayout.IsValidIndex(IndexOf))
		{
			CustomPropertyTypeLayout.RemoveAtSwap(IndexOf);
		}
	}

	virtual TArrayView<const FCustomPropertyTypeLayout> GetAllInstancedCustomPropertyTypeLayout() const override
	{
		return CustomPropertyTypeLayout;
	}

	virtual FOnWidgetBlueprintCreated& OnWidgetBlueprintCreated() override
	{
		return BlueprintCreatedEvent;
	}

private:
	void RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action)
	{
		AssetTools.RegisterAssetTypeActions(Action);
		CreatedAssetTypeActions.Add(Action);
	}

	void HandleModuleChanged(FName ModuleName, EModuleChangeReason ChangeReason)
	{
		const FName KismetModule = "Kismet";
		if (ModuleName == KismetModule && ChangeReason == EModuleChangeReason::ModuleLoaded)
		{
			if (!BlueprintVariableCustomizationHandle.IsValid())
			{
				if (FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>(KismetModule))
				{
					//BlueprintVariableCustomizationHandle = BlueprintEditorModule->RegisterVariableCustomization(FProperty::StaticClass(), FOnGetVariableCustomizationInstance::CreateStatic(&FGraphVariableDetailsCustomization::MakeInstance));
					//BlueprintFunctionCustomizationHandle = BlueprintEditorModule->RegisterFunctionCustomization(UK2Node_FunctionEntry::StaticClass(), FOnGetFunctionCustomizationInstance::CreateStatic(&FGraphFunctionDetailsCustomization::MakeInstance));
				}
			}
			FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
			ModuleChangedHandle.Reset();
		}
	}

	void OnPostEngineInit()
	{
		if (GIsEditor)
		{
			if (IConsoleManager::Get().FindConsoleVariable(TEXT("UMG.ThumbnailRenderer.Enable"))->GetBool())
			{
				UThumbnailManager::Get().RegisterCustomRenderer(UWidgetBlueprint::StaticClass(), UWidgetBlueprintThumbnailRenderer::StaticClass());
				bThumbnailRenderersRegistered = true;
			}
		}
		bOnPostEngineInitHandled = true;
	}

	void OnStartPIE(const bool bIsSimulating)
	{
		if (UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine))
		{
			if (TOptional<FRequestPlaySessionParams> PlayRequest = EditorEngine->GetPlaySessionRequest())
			{
				if (!PlayRequest.GetValue().EditorPlaySettings)
				{
					return;
				}

				int32 NumClients;
				PlayRequest.GetValue().EditorPlaySettings->GetPlayNumberOfClients(NumClients);

				if (NumClients > 1)
				{
					if (IConsoleVariable* PostBuffersEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.CopyBackbufferToSlatePostRenderTargets")))
					{
						UE_LOGF(LogUMGEditor, Log, "Disabling Slate Post Buffers for multi-window PIE session, currently not supported.");

						bCachedArePostBuffersEnabled = PostBuffersEnabled->GetBool();
						PostBuffersEnabled->Set(false);
					}
				}
			}
		}
	}

	void OnEndPIE(const bool bIsSimulating)
	{
		if (bCachedArePostBuffersEnabled)
		{
			if (IConsoleVariable* PostBuffersEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.CopyBackbufferToSlatePostRenderTargets")))
			{
				if (!PostBuffersEnabled->GetBool())
				{
					UE_LOGF(LogUMGEditor, Warning, "Restoring Slate Post Buffers, previously disabled due to multi-window PIE session.");

					PostBuffersEnabled->Set(bCachedArePostBuffersEnabled);
				}
			}
		}
	}

	static void ThumbnailRenderingEnabled(IConsoleVariable* Variable)
	{
		FUMGEditorModule* UMGEditorModule = FModuleManager::GetModulePtr<FUMGEditorModule>(TEXT("UMGEditor"));
		if (UObjectInitialized() && UMGEditorModule && UMGEditorModule->bOnPostEngineInitHandled)
		{
			if (Variable->GetBool())
			{
				UThumbnailManager::Get().RegisterCustomRenderer(UWidgetBlueprint::StaticClass(), UWidgetBlueprintThumbnailRenderer::StaticClass());
			}
			else
			{
				UThumbnailManager::Get().UnregisterCustomRenderer(UWidgetBlueprint::StaticClass());
			}
		}
	}

	static void FixupPayloadParameterNameForDynamicBinding(UMovieScene* MovieScene, UK2Node* InNode, FName OldPinName, FName NewPinName)
	{
		using namespace UE::MovieScene;

		check(MovieScene);

		auto FixupPayloadParameterName = [InNode, OldPinName, NewPinName](FMovieSceneDynamicBinding& DynamicBinding)
		{
			if (DynamicBinding.WeakEndpoint.Get() == InNode)
			{
				if (FMovieSceneDynamicBindingPayloadVariable* Variable = DynamicBinding.PayloadVariables.Find(OldPinName))
				{
					DynamicBinding.PayloadVariables.Add(NewPinName, MoveTemp(*Variable));
					DynamicBinding.PayloadVariables.Remove(OldPinName);
				}
			}
		};

		UMovieSceneSequence* ThisSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
		TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GEditor->GetEditorWorldContext().World(), ThisSequence);

		if (UWidgetAnimation* WidgetAnimation = MovieScene->GetTypedOuter<UWidgetAnimation>())
		{
			for (FWidgetAnimationBinding& WidgetAnimationBinding : WidgetAnimation->AnimationBindings)
			{
				FixupPayloadParameterName(WidgetAnimationBinding.DynamicBinding);
			}
		}
	}

	static void FixupWidgetDynamicBindings(UWidgetAnimation* WidgetAnimation)
	{
		if (WidgetAnimation)
		{
			FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(WidgetAnimation);
			if (!SequenceEditor)
			{
				return;
			}

			UBlueprint* SequenceDirectorBP = SequenceEditor->GetOrCreateDirectorBlueprint(WidgetAnimation);
			if (!SequenceDirectorBP)
			{
				return;
			}

			FMovieSceneDynamicBindingUtils::EnsureBlueprintExtensionCreated(WidgetAnimation, SequenceDirectorBP);
			FKismetEditorUtilities::CompileBlueprint(SequenceDirectorBP);
		}
	}

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FDesignerExtensibilityManager> DesignerExtensibilityManager;
	TSharedPtr<FUIComponentClipboardExtension> ComponentClipboardExtension;
	TSharedPtr<FPropertyBindingExtensibilityManager> PropertyBindingExtensibilityManager;
	TSharedPtr<FClipboardExtensibilityManager> ClipboardExtensibilityManager;
	TSharedPtr<FWidgetDragDropExtensibilityManager> WidgetDragDropExtensibilityManager;
	TSharedPtr<FWidgetContextMenuExtensibilityManager> WidgetContextMenuExtensibilityManager;
	TSharedPtr<FGraphPanelPinFactory> GraphPanelPinFactory;

	FDelegateHandle SequenceEditorHandle;
	FDelegateHandle MarginTrackEditorCreateTrackEditorHandle;
	FDelegateHandle TransformTrackEditorCreateTrackEditorHandle;
	FDelegateHandle WidgetMaterialTrackEditorCreateTrackEditorHandle;

	TSharedPtr<FUIComponentCustomizationExtender> UIComponentCustomizationExtender;

	/** All created asset type actions.  Cached here so that we can unregister it during shutdown. */
	TArray< TSharedPtr<IAssetTypeActions> > CreatedAssetTypeActions;

	/** All toolbar extenders. Utilized by tool palette */
	TArray<FWidgetEditorToolbarExtender> WidgetEditorToolbarExtenders;
	TArray<TSharedRef<IBlueprintWidgetCustomizationExtender>> WidgetCustomizationExtenders;

	TObjectPtr<USequencerSettings> Settings;

	/** Compiler customization for Widgets */
	FWidgetBlueprintCompiler WidgetBlueprintCompiler;

	/** */
	FOnRegisterTabs RegisterTabsForEditor;

	/** Support layout extensions */
	FOnRegisterLayoutExtensions RegisterLayoutExtensions;
	/** OnWidgetBlueprintCreated event */
	FOnWidgetBlueprintCreated BlueprintCreatedEvent;

	/** */
	TArray<FCustomPropertyTypeLayout> CustomPropertyTypeLayout;

	/** Handle for the FModuleManager::OnModulesChanged */
	FDelegateHandle ModuleChangedHandle;
	/** Handle for FBlueprintEditorModule::RegisterVariableCustomization */
	FDelegateHandle BlueprintVariableCustomizationHandle;
	/** Handle for FBlueprintEditorModule::RegisterFunctionCustomization */
	FDelegateHandle BlueprintFunctionCustomizationHandle;
	FDelegateHandle FixupDynamicBindingPayloadParameterNameHandle;
	FDelegateHandle FixupWidgetDynamicBindingsHandle;

	bool bThumbnailRenderersRegistered;
	bool bOnPostEngineInitHandled;
	bool bCachedArePostBuffersEnabled;
};

IMPLEMENT_MODULE(FUMGEditorModule, UMGEditor);

#undef LOCTEXT_NAMESPACE
