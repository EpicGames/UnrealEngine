// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility/WidgetSlotPair.h"
#include "Components/PanelSlot.h"
#include "WidgetBlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetSlotPair)

UWidgetSlotPair::UWidgetSlotPair()
{
}

void UWidgetSlotPair::SetWidgetName(FName InWidgetName)
{
	WidgetName = InWidgetName;
}

void UWidgetSlotPair::SetWidget(UWidget* InWidget)
{
	WidgetName = InWidget->GetFName();

	TMap<FName, FString> ExportedSlotProperties;
	FWidgetBlueprintEditorUtils::ExportPropertiesToText(InWidget->Slot, ExportedSlotProperties);

	for ( const auto& Entry : ExportedSlotProperties )
	{
		SlotPropertyNames.Add(Entry.Key);
		SlotPropertyValues.Add(Entry.Value);
	}
}

FName UWidgetSlotPair::GetWidgetName() const
{
	return WidgetName;
}

void UWidgetSlotPair::GetSlotProperties(TMap<FName, FString>& OutSlotProperties) const
{
	checkSlow(SlotPropertyNames.Num() == SlotPropertyValues.Num());

	for ( int32 PropertyIndex = 0; PropertyIndex < SlotPropertyNames.Num(); PropertyIndex++ )
	{
		OutSlotProperties.Add(SlotPropertyNames[PropertyIndex], SlotPropertyValues[PropertyIndex]);
	}
}
