// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WidgetSlotPair.generated.h"

class UWidget;

UCLASS()
class UWidgetSlotPair : public UObject
{
	GENERATED_BODY()
	
public:

	UWidgetSlotPair();

	void SetWidget(UWidget* Widget);

	void SetWidgetName(FName WidgetName);
	FName GetWidgetName() const;

	void GetSlotProperties(TMap<FName, FString>& OutSlotProperties) const;

private:

	UPROPERTY()
	FName WidgetName;

	UPROPERTY()
	TArray<FName> SlotPropertyNames;

	UPROPERTY()
	TArray<FString> SlotPropertyValues;
};
