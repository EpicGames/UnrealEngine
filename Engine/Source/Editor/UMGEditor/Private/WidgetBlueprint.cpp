// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprint.h"

#include "Components/NamedSlot.h"
#include "Components/Widget.h"
#include "Blueprint/UserWidget.h"
#include "MovieScene.h"

#include "StructUtils/UserDefinedStruct.h"
#include "EdGraph/EdGraph.h"
#include "Blueprint/WidgetTree.h"
#include "Animation/WidgetAnimation.h"
#include "Animation/WidgetAnimationHandle.h"
#include "Animation/UMGSequencePlayer.h"

#include "Kismet2/StructureEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Binding/PropertyBinding.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/PropertyTag.h"
#include "WidgetBlueprintCompiler.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "WidgetGraphSchema.h"
#include "UMGEditorProjectSettings.h"

#include "Extensions/UIComponent.h"
#include "UIComponentWidgetBlueprintExtension.h"

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Modules/ModuleManager.h"
#include "DiffResults.h"
#include "Misc/DataValidation.h"
#endif
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Composite.h"
#include "K2Node_FunctionResult.h"
#include "Blueprint/WidgetNavigation.h"
#include "WidgetBlueprintEditorUtils.h"
#include "WidgetBlueprintExtension.h"
#include "WidgetEditingProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprint)

#define LOCTEXT_NAMESPACE "UMG"

FWidgetBlueprintDelegates::FGetAssetTagsWithContext FWidgetBlueprintDelegates::GetAssetTagsWithContext;

FEditorPropertyPathSegment::FEditorPropertyPathSegment()
	: Struct(nullptr)
	, MemberName(NAME_None)
	, MemberGuid()
	, IsProperty(true)
{
}

FEditorPropertyPathSegment::FEditorPropertyPathSegment(const FProperty* InProperty)
{
	IsProperty = true;
	MemberName = InProperty->GetFName();
	if ( InProperty->GetOwnerStruct() )
	{
		Struct = InProperty->GetOwnerStruct();
		MemberGuid = FStructureEditorUtils::GetGuidForProperty(InProperty);
	}
	else if ( InProperty->GetOwnerClass() )
	{
		Struct = InProperty->GetOwnerClass();
		UBlueprint::GetGuidFromClassByFieldName<FProperty>(InProperty->GetOwnerClass(), InProperty->GetFName(), MemberGuid);
	}
	else
	{
		// Should not be possible to hit.
		check(false);
	}
}

FEditorPropertyPathSegment::FEditorPropertyPathSegment(const UFunction* InFunction)
{
	IsProperty = false;
	MemberName = InFunction->GetFName();
	if ( InFunction->GetOwnerClass() )
	{
		Struct = InFunction->GetOwnerClass();
		UBlueprint::GetGuidFromClassByFieldName<UFunction>(InFunction->GetOwnerClass(), InFunction->GetFName(), MemberGuid);
	}
	else
	{
		// Should not be possible to hit.
		check(false);
	}
}

FEditorPropertyPathSegment::FEditorPropertyPathSegment(const UEdGraph* InFunctionGraph)
{
	IsProperty = false;
	MemberName = InFunctionGraph->GetFName();
	UBlueprint* Blueprint = CastChecked<UBlueprint>(InFunctionGraph->GetOuter());
	Struct = Blueprint->GeneratedClass;
	check(Struct);
	MemberGuid = InFunctionGraph->GraphGuid;
}

void FEditorPropertyPathSegment::Rebase(UBlueprint* SegmentBase)
{
	Struct = SegmentBase->GeneratedClass;
}

bool FEditorPropertyPathSegment::ValidateMember(FDelegateProperty* DelegateProperty, FText& OutError) const
{
	// We may be binding to a function that doesn't have a explicit binder system that can handle it.  In that case
	// check to see if the function signatures are compatible, if it is, even if we don't have a binder we can just
	// directly bind the function to the delegate.
	if ( UFunction* Function = GetMember().Get<UFunction>() )
	{
		// Check the signatures to ensure these functions match.
		if ( Function->IsSignatureCompatibleWith(DelegateProperty->SignatureFunction, UFunction::GetDefaultIgnoredSignatureCompatibilityFlags() | CPF_ReturnParm) )
		{
			return true;
		}
	}

	// Next check to see if we have a binder suitable for handling this case.
	if ( DelegateProperty->SignatureFunction->NumParms == 1 )
	{
		if ( FProperty* ReturnProperty = DelegateProperty->SignatureFunction->GetReturnProperty() )
		{
			// TODO I don't like having the path segment system needing to have knowledge of the binding layer.
			// think about divorcing the two.

			// Find the binder that can handle the delegate return type.
			TSubclassOf<UPropertyBinding> Binder = UWidget::FindBinderClassForDestination(ReturnProperty);
			if ( Binder == nullptr )
			{
				OutError = FText::Format(LOCTEXT("Binding_Binder_NotFound", "Member:{0}: No binding exists for {1}."),
					GetMemberDisplayText(),
					ReturnProperty->GetClass()->GetDisplayNameText());
				return false;
			}

			FFieldVariant Field = GetMember();
			if (Field.IsValid())
			{
				if ( FProperty* Property = Field.Get<FProperty>() )
				{
					// Ensure that the binder also can handle binding from the property we care about.
					if ( Binder->GetDefaultObject<UPropertyBinding>()->IsSupportedSource(Property) )
					{
						return true;
					}
					else
					{
						OutError = FText::Format(LOCTEXT("Binding_UnsupportedType_Property", "Member:{0} Unable to bind {1}, unsupported type."),
							GetMemberDisplayText(),
							Property->GetClass()->GetDisplayNameText());
						return false;
					}
				}
				else if ( UFunction* Function = Field.Get<UFunction>() )
				{
					if ( Function->NumParms == 1 )
					{
						if ( Function->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure) )
						{
							if ( FProperty* MemberReturn = Function->GetReturnProperty() )
							{
								// Ensure that the binder also can handle binding from the property we care about.
								if ( Binder->GetDefaultObject<UPropertyBinding>()->IsSupportedSource(MemberReturn) )
								{
									return true;
								}
								else
								{
									OutError = FText::Format(LOCTEXT("Binding_UnsupportedType_Function", "Member:{0} Unable to bind {1}, unsupported type."),
										GetMemberDisplayText(),
										MemberReturn->GetClass()->GetDisplayNameText());
									return false;
								}
							}
							else
							{
								OutError = FText::Format(LOCTEXT("Binding_NoReturn", "Member:{0} Has no return value, unable to bind."),
									GetMemberDisplayText());
								return false;
							}
						}
						else
						{
							OutError = FText::Format(LOCTEXT("Binding_Pure", "Member:{0} Unable to bind, the function is not marked as pure."),
								GetMemberDisplayText());
							return false;
						}
					}
					else
					{
						OutError = FText::Format(LOCTEXT("Binding_NumArgs", "Member:{0} Has the wrong number of arguments, it needs to return 1 value and take no parameters."),
							GetMemberDisplayText());
						return false;
					}
				}
			}
		}
	}

	OutError = LOCTEXT("Binding_UnknownError", "Unknown Error");

	return false;
}

FFieldVariant FEditorPropertyPathSegment::GetMember() const
{
	FName FieldName = GetMemberName();
	if ( FieldName != NAME_None )
	{
		FFieldVariant Field = FindUFieldOrFProperty(Struct, FieldName);
		//if ( Field == nullptr )
		//{
		//	if ( UClass* Class = Cast<UClass>(Struct) )
		//	{
		//		if ( UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy) )
		//		{
		//			if ( UClass* SkeletonClass = Blueprint->SkeletonGeneratedClass )
		//			{
		//				Field = FindUField<UField>(SkeletonClass, FieldName);
		//			}
		//		}
		//	}
		//}

		return Field;
	}

	return FFieldVariant();
}

FName FEditorPropertyPathSegment::GetMemberName() const
{
	if ( MemberGuid.IsValid() )
	{
		FName NameFromGuid = NAME_None;

		if ( UClass* Class = Cast<UClass>(Struct) )
		{
			if ( UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy) )
			{
				if ( IsProperty )
				{
					NameFromGuid = UBlueprint::GetFieldNameFromClassByGuid<FProperty>(Class, MemberGuid);
				}
				else
				{
					NameFromGuid = UBlueprint::GetFieldNameFromClassByGuid<UFunction>(Class, MemberGuid);
				}
			}
		}
		else if ( UUserDefinedStruct* UserStruct = Cast<UUserDefinedStruct>(Struct) )
		{
			if ( FProperty* Property = FStructureEditorUtils::GetPropertyByGuid(UserStruct, MemberGuid) )
			{
				NameFromGuid = Property->GetFName();
			}
		}

		//check(NameFromGuid != NAME_None);
		return NameFromGuid;
	}
	
	//check(MemberName != NAME_None);
	return MemberName;
}

FText FEditorPropertyPathSegment::GetMemberDisplayText() const
{
	if ( MemberGuid.IsValid() )
	{
		if ( UClass* Class = Cast<UClass>(Struct) )
		{
			if ( UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy) )
			{
				if ( IsProperty )
				{
					return FText::FromName(UBlueprint::GetFieldNameFromClassByGuid<FProperty>(Class, MemberGuid));
				}
				else
				{
					return FText::FromName(UBlueprint::GetFieldNameFromClassByGuid<UFunction>(Class, MemberGuid));
				}
			}
		}
		else if ( UUserDefinedStruct* UserStruct = Cast<UUserDefinedStruct>(Struct) )
		{
			if ( FProperty* Property = FStructureEditorUtils::GetPropertyByGuid(UserStruct, MemberGuid) )
			{
				return Property->GetDisplayNameText();
			}
		}
	}

	return FText::FromName(MemberName);
}

FGuid FEditorPropertyPathSegment::GetMemberGuid() const
{
	return MemberGuid;
}

FEditorPropertyPath::FEditorPropertyPath()
{
}

FEditorPropertyPath::FEditorPropertyPath(const TArray<FFieldVariant>& BindingChain)
{
	for ( FFieldVariant Field : BindingChain )
	{
		if ( const FProperty* Property = Field.Get<FProperty>())
		{
			Segments.Add(FEditorPropertyPathSegment(Property));
		}
		else if ( const UFunction* Function = Field.Get<UFunction>())
		{
			Segments.Add(FEditorPropertyPathSegment(Function));
		}
		else
		{
			// Should never happen
			check(false);
		}
	}
}

bool FEditorPropertyPath::Rebase(UBlueprint* SegmentBase)
{
	if ( !IsEmpty() )
	{
		Segments[0].Rebase(SegmentBase);
		return true;
	}

	return false;
}

bool FEditorPropertyPath::Validate(FDelegateProperty* Destination, FText& OutError) const
{
	if ( IsEmpty() )
	{
		OutError = LOCTEXT("Binding_Empty", "The binding is empty.");
		return false;
	}

	for ( int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); SegmentIndex++ )
	{
		const FEditorPropertyPathSegment& Segment = Segments[SegmentIndex];
		if ( UStruct* OwnerStruct = Segment.GetStruct() )
		{
			if ( Segment.GetMember() == nullptr )
			{
				OutError = FText::Format(LOCTEXT("Binding_MemberNotFound", "Binding: '{0}' : '{1}' was not found on '{2}'."),
					GetDisplayText(),
					Segment.GetMemberDisplayText(),
					OwnerStruct->GetDisplayNameText());

				return false;
			}
		}
		else
		{
			OutError = FText::Format(LOCTEXT("Binding_StructNotFound", "Binding: '{0}' : Unable to locate owner class or struct for '{1}'"),
				GetDisplayText(),
				Segment.GetMemberDisplayText());

			return false;
		}
	}

	// Validate the last member in the segment
	const FEditorPropertyPathSegment& LastSegment = Segments[Segments.Num() - 1];
	return LastSegment.ValidateMember(Destination, OutError);
}

FText FEditorPropertyPath::GetDisplayText() const
{
	FString DisplayText;

	for ( int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); SegmentIndex++ )
	{
		const FEditorPropertyPathSegment& Segment = Segments[SegmentIndex];
		DisplayText.Append(Segment.GetMemberDisplayText().ToString());
		if ( SegmentIndex < ( Segments.Num() - 1 ) )
		{
			DisplayText.Append(TEXT("."));
		}
	}

	return FText::FromString(DisplayText);
}

FDynamicPropertyPath FEditorPropertyPath::ToPropertyPath() const
{
	TArray<FString> PropertyChain;
	for (const FEditorPropertyPathSegment& Segment : Segments)
	{
		FName SegmentName = Segment.GetMemberName();
		if (SegmentName != NAME_None)
		{
			PropertyChain.Add(SegmentName.ToString());
		}
		else
		{
			return FDynamicPropertyPath();
		}
	}
	return FDynamicPropertyPath(PropertyChain);
}

bool FDelegateEditorBinding::IsAttributePropertyBinding(UWidgetBlueprint* Blueprint) const
{
	// First find the target widget we'll be attaching the binding to.
	if (UWidget* TargetWidget = Blueprint->WidgetTree->FindWidget(FName(*ObjectName)))
	{
		// Next find the underlying delegate we're actually binding to, if it's an event the name will be the same,
		// for properties we need to lookup the delegate property we're actually going to be binding to.
		FDelegateProperty* BindableProperty = FindFProperty<FDelegateProperty>(TargetWidget->GetClass(), FName(*(PropertyName.ToString() + TEXT("Delegate"))));
		return BindableProperty != nullptr;
	}

	return false;
}

bool FDelegateEditorBinding::DoesBindingTargetExist(UWidgetBlueprint* Blueprint) const
{
	// First find the target widget we'll be attaching the binding to.
	if (UWidget* TargetWidget = Blueprint->WidgetTree->FindWidget(FName(*ObjectName)))
	{
		return true;
	}

	return false;
}

bool FDelegateEditorBinding::IsBindingValid(UClass* BlueprintGeneratedClass, UWidgetBlueprint* Blueprint, FCompilerResultsLog& MessageLog) const
{
	FDelegateRuntimeBinding RuntimeBinding = ToRuntimeBinding(Blueprint);

	// First find the target widget we'll be attaching the binding to.
	if ( UWidget* TargetWidget = Blueprint->WidgetTree->FindWidget(FName(*ObjectName)) )
	{
		// Next find the underlying delegate we're actually binding to, if it's an event the name will be the same,
		// for properties we need to lookup the delegate property we're actually going to be binding to.
		FDelegateProperty* BindableProperty = FindFProperty<FDelegateProperty>(TargetWidget->GetClass(), FName(*( PropertyName.ToString() + TEXT("Delegate") )));
		FDelegateProperty* EventProperty = FindFProperty<FDelegateProperty>(TargetWidget->GetClass(), PropertyName);

		bool bNeedsToBePure = BindableProperty ? true : false;
		FDelegateProperty* DelegateProperty = BindableProperty ? BindableProperty : EventProperty;

		// Locate the delegate property on the widget that's a delegate for a property we want to bind.
		if ( DelegateProperty )
		{
			if ( !SourcePath.IsEmpty() )
			{
				FText ValidationError;
				if ( SourcePath.Validate(DelegateProperty, ValidationError) == false )
				{
					MessageLog.Error(
						*FText::Format(
							LOCTEXT("BindingErrorFmt", "Binding: Property '@@' on Widget '@@': {0}"),
							ValidationError
						).ToString(),
						DelegateProperty,
						TargetWidget
					);

					return false;
				}

				// We allow for widget delegates to have deprecated metadata without fully deprecating.
				// Since full deprecation breaks existing widgets, checking as below allows for slow deprecation.
				FString DeprecationWarning = DelegateProperty->GetMetaData("DeprecationMessage");
				if (!DeprecationWarning.IsEmpty())
				{
					MessageLog.Warning(
						*FText::Format(
							LOCTEXT("BindingWarningDeprecated", "Binding: Deprecated property '@@' on Widget '@@': {0}"),
							FText::FromString(DeprecationWarning)
						).ToString(),
						DelegateProperty,
						TargetWidget
					);
				}

				return true;
			}
			else
			{
				// On our incoming blueprint generated class, try and find the function we claim exists that users
				// are binding their property too.
				if ( UFunction* Function = BlueprintGeneratedClass->FindFunctionByName(RuntimeBinding.FunctionName, EIncludeSuperFlag::IncludeSuper) )
				{
					// Check the signatures to ensure these functions match.
					if ( Function->IsSignatureCompatibleWith(DelegateProperty->SignatureFunction, UFunction::GetDefaultIgnoredSignatureCompatibilityFlags() | CPF_ReturnParm) )
					{
						// Only allow binding pure functions to property bindings.
						if ( bNeedsToBePure && !Function->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure) )
						{
							FText const ErrorFormat = LOCTEXT("BindingNotBoundToPure", "Binding: property '@@' on widget '@@' needs to be bound to a pure function, '@@' is not pure.");
							MessageLog.Error(*ErrorFormat.ToString(), DelegateProperty, TargetWidget, Function);

							return false;
						}

						return true;
					}
					else
					{
						FText const ErrorFormat = LOCTEXT("BindingFunctionSigDontMatch", "Binding: property '@@' on widget '@@' bound to function '@@', but the sigatnures don't match.  The function must return the same type as the property and have no parameters.");
						MessageLog.Error(*ErrorFormat.ToString(), DelegateProperty, TargetWidget, Function);
					}
				}
				else
				{
					// Bindable property removed.
				}
			}
		}
		else
		{
			// Bindable Property Removed
		}
	}
	else
	{
		// Ignore missing widgets
	}

	return false;
}

FDelegateRuntimeBinding FDelegateEditorBinding::ToRuntimeBinding(UWidgetBlueprint* Blueprint) const
{
	FDelegateRuntimeBinding Binding;
	Binding.ObjectName = ObjectName;
	Binding.PropertyName = PropertyName;
	if ( Kind == EBindingKind::Function )
	{
		Binding.FunctionName = ( MemberGuid.IsValid() ) ? Blueprint->GetFieldNameFromClassByGuid<UFunction>(Blueprint->SkeletonGeneratedClass, MemberGuid) : FunctionName;
	}
	else
	{
		Binding.FunctionName = FunctionName;
	}
	Binding.Kind = Kind;
	Binding.SourcePath = SourcePath.ToPropertyPath();

	return Binding;
}

/////////////////////////////////////////////////////
// UWidgetBlueprint

UWidgetBlueprint::UWidgetBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCanCallInitializedWithoutPlayerContext(false)
	, TickFrequency(EWidgetTickFrequency::Auto)
{
	// Widget GUIDs must be cooked to support fixups at runtime, so enable by default
	ShouldCookPropertyGuidsValue = EShouldCookBlueprintPropertyGuids::Yes;
}

void UWidgetBlueprint::ReplaceDeprecatedNodes()
{
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WidgetStopDuplicatingAnimations)
	{
		// Update old graphs to all use the widget graph schema.
		TArray<UEdGraph*> Graphs;
		GetAllGraphs(Graphs);

		for (UEdGraph* Graph : Graphs)
		{
			Graph->Schema = UWidgetGraphSchema::StaticClass();
		}
	}

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::BlueprintPinsUseRealNumbers)
	{
		// Revert any overzealous PC_Float to PC_Real/PC_Double conversions.

		// The Blueprint real number changes will automatically convert pin types to doubles if used in a non-native context.
		// However, UMG property bindings are a special case: the BP functions that bind to the native delegate must agree on their underlying types.
		// Specifically, bindings used with float properties *must* use the PC_Float type as the return value in a BP function.
		// In order to correct this behavior, we need to:
		// * Iterate through the property bindings.
		// * Find the corresponding delegate signature.
		// * Find the function graph that matches the binding.
		// * Find the result node.
		// * Change the pin type back to float if that's what the delegate signature expects.

		TArray<UEdGraph*> Graphs;
		GetAllGraphs(Graphs);

		for (const FDelegateEditorBinding& Binding : Bindings)
		{
			if (Binding.IsAttributePropertyBinding(this))
			{
				check(WidgetTree);
				if (UWidget* TargetWidget = WidgetTree->FindWidget(FName(*Binding.ObjectName)))
				{
					const FDelegateProperty* BindableProperty =
						FindFProperty<FDelegateProperty>(TargetWidget->GetClass(), FName(*(Binding.PropertyName.ToString() + TEXT("Delegate"))));

					if (BindableProperty)
					{
						FName FunctionName = Binding.FunctionName;

						if (!Binding.SourcePath.IsEmpty())
						{
							check(Binding.SourcePath.Segments.Num() > 0);
							const FEditorPropertyPathSegment& LastSegment = Binding.SourcePath.Segments[Binding.SourcePath.Segments.Num() - 1];
							FunctionName = LastSegment.GetMemberName();
						}

						auto GraphMatchesBindingPredicate = [FunctionName](const UEdGraph* Graph) {
							check(Graph);
							return (FunctionName == Graph->GetFName());
						};

						if (UEdGraph** GraphEntry = Graphs.FindByPredicate(GraphMatchesBindingPredicate))
						{
							UEdGraph* CurrentGraph = *GraphEntry;
							check(CurrentGraph);

							for (UEdGraphNode* Node : CurrentGraph->Nodes)
							{
								check(Node);
								if (Node->IsA<UK2Node_FunctionResult>())
								{
									for (UEdGraphPin* Pin : Node->Pins)
									{
										check(Pin);
										if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
										{
											FName PinName = Pin->GetFName();

											const UFunction* DelegateFunction = BindableProperty->SignatureFunction;
											check(DelegateFunction);

											auto OutputParameterMatchesPin = [PinName](FFloatProperty* FloatParam) {
												check(FloatParam);
												bool bHasMatch =
													(FloatParam->PropertyFlags & CPF_OutParm) &&
													(FloatParam->GetFName() == PinName);

												return bHasMatch;
											};

											bool bFoundMatchingParam = false;
											for (TFieldIterator<FFloatProperty> It(DelegateFunction); It; ++It)
											{
												if (OutputParameterMatchesPin(*It))
												{
													Pin->PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
													bFoundMatchingParam = true;
													break;
												}
											}

											if (bFoundMatchingParam)
											{
												UK2Node_FunctionResult* FunctionResultNode = CastChecked<UK2Node_FunctionResult>(Node);
												for (TSharedPtr<FUserPinInfo>& UserPin : FunctionResultNode->UserDefinedPins)
												{
													check(UserPin);
													if (UserPin->PinName == PinName)
													{
														check(UserPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real);
														UserPin->PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
														break;
													}
												}
											}

											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
#endif

	Super::ReplaceDeprecatedNodes();
}

#if WITH_EDITORONLY_DATA
void UWidgetBlueprint::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (UWidgetBlueprintGeneratedClass* BPGC = Cast< UWidgetBlueprintGeneratedClass>(GeneratedClass))
	{
		FWidgetBlueprintCompilerContext::ClearGeneratedClassWidgetTree(BPGC, false);
		FWidgetBlueprintCompilerContext::UpdateGeneratedClassWidgetTree(this, BPGC, BPGC->GetWidgetTreeArchetype(), nullptr);
	}
}


void UWidgetBlueprint::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Add AvailableNamedSlots,  also available on generated class, to the WidgetBlueprint
	if (const UWidgetBlueprintGeneratedClass* WidgetBPGeneratedClass = Cast<const UWidgetBlueprintGeneratedClass>(GeneratedClass))
	{
		TStringBuilder<512> Builder;
		for (FName NamedSlot : WidgetBPGeneratedClass->AvailableNamedSlots)
		{
			if (!NamedSlot.IsNone())
			{
				if (Builder.Len() > 0)
				{
					Builder << TEXT(',');
				}
				Builder << NamedSlot;
			}
		}
		Context.AddTag(FAssetRegistryTag(FName("AvailableNamedSlots"), Builder.ToString(), FAssetRegistryTag::TT_Hidden));
	}

	FWidgetBlueprintDelegates::GetAssetTagsWithContext.Broadcast(this, Context);
}

void UWidgetBlueprint::NotifyGraphRenamed(class UEdGraph* Graph, FName OldName, FName NewName)
{
	Super::NotifyGraphRenamed(Graph, OldName, NewName);
	
	const TObjectPtr<UBlueprintExtension>* UIComponentExtensionPtr = GetExtensions().FindByPredicate([](const UBlueprintExtension* Other) {
		return Other && Other->GetClass() == UUIComponentWidgetBlueprintExtension::StaticClass(); 
	});

	UUIComponentWidgetBlueprintExtension* UIComponentExtension = nullptr;
	if (UIComponentExtensionPtr)
	{
		UIComponentExtension = CastChecked<UUIComponentWidgetBlueprintExtension>(*UIComponentExtensionPtr);
	}

	// Update any explicit widget bindings.
	WidgetTree->ForEachWidget([Graph, OldName, NewName, UIComponentExtension](UWidget* Widget) {
		if (Widget->Navigation)
		{
			Widget->Navigation->SetFlags(RF_Transactional);
			Widget->Navigation->Modify();
			Widget->Navigation->TryToRenameBinding(OldName, NewName);
		}

		if (UIComponentExtension != nullptr)
		{
			for (UUIComponent* Component : UIComponentExtension->GetComponentsFor(Widget))
			{
				Component->SetFlags(RF_Transactional);
				Component->Modify();
				Component->GraphRenamed(Graph, OldName, NewName);
			}
		}
	});
}

EDataValidationResult UWidgetBlueprint::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = UBlueprint::IsDataValid(Context);

	// Check for leaks
	if (IsInGameThread())
	{
		if (DetectSlateWidgetLeaks(Context))
		{
			Result = CombineDataValidationResults(Result, EDataValidationResult::Invalid);
		}
	}

	// Check the ui components
	if (const UUIComponentWidgetBlueprintExtension* UIExtension =
		UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(this))
	{
		Result = CombineDataValidationResults(Result, UIExtension->ValidateComponents(Context));
	}

	return Result;
}

bool UWidgetBlueprint::DetectSlateWidgetLeaks(FDataValidationContext& Context) const
{
	// We can't safely run this in anything but a running editor, since widgets
	// rely on a functioning slate application.
	if (IsRunningCommandlet())
	{
		return false;
	}

	// The detection relies on instantiation of the class: don't try to create an abstract class. 
	// The validation will have to be run on the WBP inheriting from abstract ones.
	if (GeneratedClass == nullptr || GeneratedClass->HasAnyClassFlags(CLASS_Abstract))
	{
		return false;
	}

	UWorld* DummyWorld = NewObject<UWorld>();
	UUserWidget* TempUserWidget = NewObject<UUserWidget>(DummyWorld, GeneratedClass);
	TempUserWidget->ClearFlags(RF_Transactional);
	TempUserWidget->SetDesignerFlags(EWidgetDesignFlags::Designing);

	// If there's no widget tree, there's no test to be performed.
	if (WidgetTree == nullptr)
	{
		return false;
	}

	// Update the widget tree directly to match the blueprint tree.  That way the preview can update
	// without needing to do a full recompile.
	TempUserWidget->DuplicateAndInitializeFromWidgetTree(WidgetTree, TMap<FName, UWidget*>());

	// We don't want this widget doing all the normal startup and acting like it's the real deal
	// trying to do gameplay stuff, so make sure it's in design mode.
	TempUserWidget->SetDesignerFlags(EWidgetDesignFlags::Designing);

	// Force construction of the slate widgets, and immediately let it go.
	TWeakPtr<SWidget> PreviewSlateWidgetWeak = TempUserWidget->TakeWidget();

	bool bFoundLeak = false;

	// NOTE: This doesn't explore sub UUserWidget trees, searching for leaks there on purpose,
	//       those widgets will be handled by their own validation steps.

	// Verify everything is going to be garbage collected.
	TempUserWidget->WidgetTree->ForEachWidget([&Context, &bFoundLeak](UWidget* Widget) {
		if (!bFoundLeak)
		{
			TWeakPtr<SWidget> PreviewChildWidget = Widget->GetCachedWidget();
			if (PreviewChildWidget.IsValid())
			{
				bFoundLeak = true;
				if (const UPanelWidget* ParentWidget = Widget->GetParent())
				{
					Context.AddError(
						FText::Format(
							LOCTEXT("LeakingWidgetsWithParent_WarningFmt", "Leak Detected!  {0} ({1}) still has living Slate widgets, it or the parent {2} ({3}) is keeping them in memory.  Make sure all Slate resources (TSharedPtr<SWidget>'s) are being released in the UWidget's ReleaseSlateResources().  Also check the USlot's ReleaseSlateResources()."),
							FText::FromString(Widget->GetName()),
							FText::FromString(Widget->GetClass()->GetName()),
							FText::FromString(ParentWidget->GetName()),
							FText::FromString(ParentWidget->GetClass()->GetName())
						)
					);
				}
				else
				{
					Context.AddError(
						FText::Format(
							LOCTEXT("LeakingWidgetsWithoutParent_WarningFmt", "Leak Detected!  {0} ({1}) still has living Slate widgets, it or the parent widget is keeping them in memory.  Make sure all Slate resources (TSharedPtr<SWidget>'s) are being released in the UWidget's ReleaseSlateResources().  Also check the USlot's ReleaseSlateResources()."),
							FText::FromString(Widget->GetName()),
							FText::FromString(Widget->GetClass()->GetName())
						)
					);
				}
			}
		}
	});

	DummyWorld->MarkObjectsPendingKill();
	return bFoundLeak;
}

bool UWidgetBlueprint::FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const
{
	const UWidgetBlueprint* OtherWidgetBP = Cast<UWidgetBlueprint>(OtherBlueprint);
	if (!OtherWidgetBP)
	{
		return false;
	}

	// Look for all widget instances in both, add shared ones to ObjectsToDiff and add notes for add/remove
	TMap<FString, UWidget*> WidgetMap;
	TMap<FString, UWidget*> OtherWidgetMap;

	WidgetTree->ForEachWidget([&](UWidget* Widget) 
	{
		FString WidgetPath = Widget->GetPathName(this);
		WidgetMap.Add(WidgetPath, Widget);
	});

	OtherWidgetBP->WidgetTree->ForEachWidget([&](UWidget* Widget)
	{
		FString WidgetPath = Widget->GetPathName(OtherWidgetBP);
		OtherWidgetMap.Add(WidgetPath, Widget);
	});

	for (TPair<FString, UWidget*> Pair : WidgetMap)
	{
		UWidget** FoundOtherWidget = OtherWidgetMap.Find(Pair.Key);
		UWidget* Widget = Pair.Value;

		if (FoundOtherWidget)
		{
			if (Results.CanStoreResults())
			{
				// Add to general object diff map
				FDiffSingleResult Diff;
				Diff.Diff = EDiffType::OBJECT_REQUEST_DIFF;
				Diff.Object1 = Widget;
				Diff.Object2 = *FoundOtherWidget;
				Diff.OwningObjectPath = Pair.Key;			
				FFormatNamedArguments Args;
				Args.Add(TEXT("WidgetTitle"), Widget->GetLabelTextWithMetadata());
				Args.Add(TEXT("WidgetPath"), FText::FromString(Pair.Key));
				Diff.ToolTip = FText::Format(LOCTEXT("DIF_RequestWidgetTooltip", "Widget {WidgetTitle}\nPath: {WidgetPath}"), Args);
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_RequestWidgetLabel", "Widget {WidgetTitle}"), Args);
				Diff.Category = EDiffType::CONTROL;

				Results.Add(Diff);

				UPanelSlot* Slot = Widget->Slot;
				UPanelSlot* OtherSlot = (*FoundOtherWidget)->Slot;

				if (Slot && OtherSlot)
				{
					FDiffSingleResult SlotDiff;
					SlotDiff.Diff = EDiffType::OBJECT_REQUEST_DIFF;
					SlotDiff.Object1 = Slot;
					SlotDiff.Object2 = OtherSlot;
					SlotDiff.OwningObjectPath = Pair.Key;
					FFormatNamedArguments SlotArgs;
					SlotArgs.Add(TEXT("WidgetTitle"), Widget->GetLabelTextWithMetadata());
					SlotArgs.Add(TEXT("WidgetPath"), FText::FromString(Pair.Key));
					SlotDiff.ToolTip = FText::Format(LOCTEXT("DIF_RequestSlotTooltip", "Slot for {WidgetTitle}\nPath: {WidgetPath}"), SlotArgs);
					SlotDiff.DisplayString = FText::Format(LOCTEXT("DIF_RequestSlotLabel", "Slot for {WidgetTitle}"), SlotArgs);
					Diff.Category = EDiffType::CONTROL;

					Results.Add(SlotDiff);
				}
			}
		}
		else
		{
			// This is newly added
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::OBJECT_ADDED;
			Diff.Object1 = Widget;
			Diff.OwningObjectPath = Pair.Key;

			if (Results.CanStoreResults())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("WidgetTitle"), Widget->GetLabelTextWithMetadata());
				Args.Add(TEXT("WidgetPath"), FText::FromString(Pair.Key));
				Diff.ToolTip = FText::Format(LOCTEXT("DIF_AddedWidgetTooltip", "Added Widget {WidgetTitle}\nPath: {WidgetPath}"), Args);
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_AddedWidgetLabel", "Added Widget {WidgetTitle}"), Args);
				Diff.Category = EDiffType::ADDITION;
			}

			Results.Add(Diff);
		}
	}

	for (TPair<FString, UWidget*> Pair : OtherWidgetMap)
	{
		UWidget** FoundMyWidget = WidgetMap.Find(Pair.Key);
		UWidget* OtherWidget = Pair.Value;

		if (!FoundMyWidget)
		{
			// This is newly added
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::OBJECT_REMOVED;
			Diff.Object1 = OtherWidget;
			Diff.OwningObjectPath = Pair.Key;

			if (Results.CanStoreResults())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("WidgetTitle"), OtherWidget->GetLabelTextWithMetadata());
				Args.Add(TEXT("WidgetPath"), FText::FromString(Pair.Key));
				Diff.ToolTip = FText::Format(LOCTEXT("DIF_RemovedWidgetTooltip", "Removed Widget {WidgetTitle}\nPath:{WidgetPath}"), Args);
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_RemovedWidgetLabel", "Removed Widget {WidgetTitle}"), Args);
				Diff.Category = EDiffType::SUBTRACTION;
			}

			Results.Add(Diff);
		}
	}

	for (const TObjectPtr<UBlueprintExtension>& Extension : GetExtensions())
	{
		if (Extension->IsA<UWidgetBlueprintExtension>())
		{
			if (UWidgetBlueprintExtension* WidgetExtension = StaticCast<UWidgetBlueprintExtension*>(Extension.Get()))
			{
				WidgetExtension->FindDiffs(this, OtherWidgetBP, Results);
			}
		}
	}

	// Add info warning
	if (Results.CanStoreResults())
	{
		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::INFO_MESSAGE;
		Diff.Category = EDiffType::CONTROL;
		Diff.ToolTip = LOCTEXT("DIF_WidgetWarningMessage", "Warning: This may be missing changes to Animations and Bindings");
		Diff.DisplayString = Diff.ToolTip;

		Results.Add(Diff);
	}

	return true;
}

void UWidgetBlueprint::OnVariableAdded(const FName& VariableName)
{
	ensure(!WidgetVariableNameToGuidMap.Contains(VariableName));
	Modify();
	WidgetVariableNameToGuidMap.Emplace(VariableName, FGuid::NewGuid());
}

void UWidgetBlueprint::OnVariableRenamed(const FName& OldName, const FName& NewName)
{
	if (OldName == NewName)
	{
		return;
	}

	// We should never have an entry for the new name and always have an entry for the current name
	ensure(!WidgetVariableNameToGuidMap.Contains(NewName));
	
	FGuid VariableGuid;
	if (ensure(WidgetVariableNameToGuidMap.RemoveAndCopyValue(OldName, VariableGuid)))
	{
		Modify();
		WidgetVariableNameToGuidMap.Emplace(NewName, VariableGuid);
	}
	else
	{
		OnVariableAdded(NewName);
	}
}

void UWidgetBlueprint::OnVariableRemoved(const FName& VariableName)
{
	Modify();
	WidgetVariableNameToGuidMap.Remove(VariableName);
}
#endif // WITH_EDITORONLY_DATA

void UWidgetBlueprint::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

void UWidgetBlueprint::PostLoad()
{
	Super::PostLoad();

	WidgetTree->ClearFlags(RF_ArchetypeObject);

	WidgetTree->ForEachWidget([&] (UWidget* Widget) {
		Widget->ConnectEditorData();
	});

	if ( GetLinkerUEVersion() < VER_UE4_RENAME_WIDGET_VISIBILITY )
	{
		static const FName Visiblity(TEXT("Visiblity"));
		static const FName Visibility(TEXT("Visibility"));

		for ( FDelegateEditorBinding& Binding : Bindings )
		{
			if ( Binding.PropertyName == Visiblity )
			{
				Binding.PropertyName = Visibility;
			}
		}
	}

	if ( GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::WidgetGraphSchema )
	{
		// Update old graphs to all use the widget graph schema.
		TArray<UEdGraph*> Graphs;
		GetAllGraphs(Graphs);

		for ( UEdGraph* Graph : Graphs )
		{
			Graph->Schema = UWidgetGraphSchema::StaticClass();
		}
	}

	if ( GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UpgradeWidgetBlueprintLegacySequencePlayer )
	{
		FixupLegacySequencePlayerCalls();
	}
}

void UWidgetBlueprint::FixupLegacySequencePlayerCalls()
{
	TArray<UEdGraph*> Graphs;
	GetAllGraphs(Graphs);
	for (UEdGraph* Graph : Graphs)
	{
		TArray<UK2Node_CallFunction*> CallFunctionNodes;
		Graph->GetNodesOfClass(CallFunctionNodes);
		for (UK2Node_CallFunction* CallFunctionNode : CallFunctionNodes)
		{
			// Old calls to GetUserTag and SetUserTag were done on the UUMGSequencePlayer object returned by
			// calls to PlayAnimation and its variants. These would be hooked up on the "self" pin.
			//
			// The new calls are done one the animation handle structure, via functions on 
			// UWidgetAnimationHandleFunctionLibrary. In this case, since these functions are static, the
			// "self" pin is hidden, and we want calls to be done on the "Target" pin which didn't exist
			// before, and doesn't exist yet as we are being called in PostLoad.
			// 
			// Core redirectors can't handle this case so we fix them up here. We rename "self" to Target
			// so that connections are preserved.
			//
			// TODO: this wouldn't fix calls done _outside_ of the widget, those will be broken.
			//
			const FName FunctionName = CallFunctionNode->GetFunctionName();
			const bool bMatchesAnimationFunctionName = (
					FunctionName == GET_FUNCTION_NAME_CHECKED(UUMGSequencePlayer, GetUserTag) ||
					FunctionName == GET_FUNCTION_NAME_CHECKED(UUMGSequencePlayer, SetUserTag));
			if (!bMatchesAnimationFunctionName)
			{
				continue;
			}

			UEdGraphPin* SelfPin = GetDefault<UEdGraphSchema_K2>()->FindSelfPin(*CallFunctionNode, EGPD_Input);
			const bool bMatchesAnimationClass = (
					SelfPin &&
					SelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object &&
					SelfPin->PinType.PinSubCategoryObject == UUMGSequencePlayer::StaticClass());
			if (!bMatchesAnimationClass)
			{
				continue;
			}

			// Redirect from UUMGSequencePlayer::XxxUserTag to UWidgetAnimationHandleFunctionLibrary::XxxUserTag.
			CallFunctionNode->FunctionReference.SetExternalMember(
					FunctionName, UWidgetAnimationHandleFunctionLibrary::StaticClass());

			// IMPORTANT: the name "Target" must match the parameter name in the GetUserTag and 
			// SetUserTag methods!
			UEdGraphPin* TargetPin = CallFunctionNode->FindPin(TEXT("Target"));
			if (SelfPin && !SelfPin->LinkedTo.IsEmpty() && !TargetPin)
			{
				SelfPin->PinName = TEXT("Target");
#if WITH_EDITORONLY_DATA
				SelfPin->PinFriendlyName = FText();
#endif
			}
		}
	}
}

void UWidgetBlueprint::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if ( !bDuplicatingReadOnly )
	{
		// We need to update all the bindings and change each bindings first segment in the path
		// to be the new class this blueprint generates, as all bindings must first originate on 
		// the widget blueprint, the first segment is always a reference to 'self'.
		for ( FDelegateEditorBinding& Binding : Bindings )
		{
			Binding.SourcePath.Rebase(this);
		}
	}
}

UClass* UWidgetBlueprint::GetBlueprintClass() const
{
	return UWidgetBlueprintGeneratedClass::StaticClass();
}

bool UWidgetBlueprint::AllowsDynamicBinding() const
{
	return true;
}

bool UWidgetBlueprint::SupportsInputEvents() const
{
	return true;
}

void UWidgetBlueprint::GatherDependencies(TSet<TWeakObjectPtr<UBlueprint>>& InDependencies) const
{
	Super::GatherDependencies(InDependencies);

	if ( WidgetTree )
	{
		WidgetTree->ForEachWidget([&] (UWidget* Widget) {
			if ( UBlueprint* WidgetBlueprint = UBlueprint::GetBlueprintFromClass(Widget->GetClass()) )
			{
				bool bWasAlreadyInSet;
				InDependencies.Add(WidgetBlueprint, &bWasAlreadyInSet);

				if ( !bWasAlreadyInSet )
				{
					WidgetBlueprint->GatherDependencies(InDependencies);
				}
			}
		});
	}
}

bool UWidgetBlueprint::ValidateGeneratedClass(const UClass* InClass)
{
	const UWidgetBlueprintGeneratedClass* GeneratedClass = Cast<const UWidgetBlueprintGeneratedClass>(InClass);
	if ( !ensure(GeneratedClass) )
	{
		return false;
	}
	const UWidgetBlueprint* Blueprint = Cast<UWidgetBlueprint>(GetBlueprintFromClass(GeneratedClass));
	if ( !ensure(Blueprint) )
	{
		return false;
	}

	if ( !ensure(Blueprint->WidgetTree && ( Blueprint->WidgetTree->GetOuter() == Blueprint )) )
	{
		return false;
	}
	else
	{
		TArray < UWidget* > AllWidgets;
		Blueprint->WidgetTree->GetAllWidgets(AllWidgets);
		for ( UWidget* Widget : AllWidgets )
		{
			if ( !ensure(Widget->GetOuter() == Blueprint->WidgetTree) )
			{
				return false;
			}
		}
	}

	UWidgetTree* WidgetTree = GeneratedClass->GetWidgetTreeArchetype();

	if ( !ensure(WidgetTree && (WidgetTree->GetOuter() == GeneratedClass )) )
	{
		return false;
	}
	else
	{
		TArray<UWidget*> AllWidgets;
		WidgetTree->GetAllWidgets(AllWidgets);
		for ( UWidget* Widget : AllWidgets )
		{
			if ( !ensure(Widget->GetOuter() == WidgetTree) )
			{
				return false;
			}
		}
	}

	return true;
}

TSharedPtr<FKismetCompilerContext> UWidgetBlueprint::GetCompilerForWidgetBP(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
{
	return TSharedPtr<FKismetCompilerContext>(new FWidgetBlueprintCompilerContext(CastChecked<UWidgetBlueprint>(BP), InMessageLog, InCompileOptions));
}

void UWidgetBlueprint::GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const
{
	AllowedChildrenOfClasses.Add( UUserWidget::StaticClass() );
}

bool UWidgetBlueprint::IsWidgetFreeFromCircularReferences(UUserWidget* UserWidget) const
{
	if (UserWidget != nullptr)
	{
		if (UserWidget->GetClass() == GeneratedClass)
		{
			// If this user widget is the same as the blueprint's generated class, we should reject it because it
			// will cause a circular reference within the blueprint.
			return false;
		}
		else if (UWidgetBlueprint* GeneratedByBlueprint = Cast<UWidgetBlueprint>(UserWidget->WidgetGeneratedBy))
		{
			// Check the generated by blueprints - this will catch even cases where one has the other in the widget tree but hasn't compiled yet
			if (GeneratedByBlueprint->WidgetTree && GeneratedByBlueprint->WidgetTree->RootWidget)
			{
				TArray<UWidget*> ChildWidgets;
				GeneratedByBlueprint->WidgetTree->GetAllWidgets(ChildWidgets);
				for (UWidget* ChildWidget : ChildWidgets)
				{
					if (UWidgetBlueprint* ChildGeneratedBlueprint = Cast<UWidgetBlueprint>(ChildWidget->WidgetGeneratedBy))
					{
						if (this == ChildGeneratedBlueprint)
						{
							return false;
						}
					}
				}
			}
		}
		else if (UserWidget->WidgetTree)
		{
			// This loop checks for references that existed in the compiled blueprint, in case it's changed since then
			TArray<UWidget*> ChildWidgets;
			UserWidget->WidgetTree->GetAllWidgets(ChildWidgets);

			for (UWidget* Widget : ChildWidgets)
			{
				if (Cast<UUserWidget>(Widget) != nullptr)
				{
					if ( !IsWidgetFreeFromCircularReferences(Cast<UUserWidget>(Widget)) )
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

namespace UE::UMG::Private
{
bool HasCircularReferences(const UClass* CurrentClass, TArray<const UClass*, TInlineAllocator<32>> DiscoveredBlueprint, UWidget*& OutResult)
{
	if (DiscoveredBlueprint.ContainsByPredicate([CurrentClass](const UClass* Other) { return CurrentClass->IsChildOf(Other); }))
	{
		return true;
	}
	DiscoveredBlueprint.Add(CurrentClass);

	if (const UWidgetBlueprintGeneratedClass* CurrentWidgetClass = Cast<const UWidgetBlueprintGeneratedClass>(CurrentClass))
	{
		TArray<UWidget*> AllWidgets;
		if (const UWidgetBlueprint* WidgetBP = Cast<const UWidgetBlueprint>(CurrentWidgetClass->ClassGeneratedBy))
		{
			if (WidgetBP->WidgetTree)
			{
				WidgetBP->WidgetTree->GetAllWidgets(AllWidgets);
			}
		}
		else if (UWidgetTree* CurrentWidgetTree = CurrentWidgetClass->GetWidgetTreeArchetype())
		{
			CurrentWidgetTree->GetAllWidgets(AllWidgets);

		}

		for (UWidget* Widget : AllWidgets)
		{
			if (UUserWidget* UserWidget = Cast<UUserWidget>(Widget))
			{
				if (HasCircularReferences(UserWidget->GetClass(), DiscoveredBlueprint, OutResult))
				{
					OutResult = Widget;
					return true;
				}
			}
		}
	}

	return false;
}

const UWidgetBlueprint* GetGeneratedWidgetBlueprintFromClass(const UClass* CurrentClass)
{
	if (const UWidgetBlueprintGeneratedClass* GeneratedClass = Cast<const UWidgetBlueprintGeneratedClass>(CurrentClass))
	{
		return Cast<const UWidgetBlueprint>(GeneratedClass->ClassGeneratedBy);
	}
	return nullptr;
}

bool TryBuildSetNameForWidgets(const UClass* CurrentClass, TSet<FName>& InOutWidgetNames, TSet<UWidget*>& InOutConflictingWidgets)
{
	if (const UWidgetBlueprint* WidgetBP = GetGeneratedWidgetBlueprintFromClass(CurrentClass))
	{
		// only search in parent class if this class isn't overwriting the root
		if (WidgetBP->WidgetTree->RootWidget == nullptr)
		{
			if (!TryBuildSetNameForWidgets(WidgetBP->ParentClass, InOutWidgetNames, InOutConflictingWidgets))
			{
				return false;
			}
		}

		if (WidgetBP->WidgetTree)
		{
			WidgetBP->WidgetTree->ForEachWidget([&InOutWidgetNames,&InOutConflictingWidgets](UWidget* Widget) {
				FName WidgetName = Widget->GetFName();
				if (InOutWidgetNames.Contains(WidgetName))
				{
					InOutConflictingWidgets.Add(Widget);
				}
				else
				{
					InOutWidgetNames.Add(Widget->GetFName());
				}
				});
		}
	}
	return InOutConflictingWidgets.IsEmpty();
}

}

TValueOrError<void, UWidget*> UWidgetBlueprint::HasCircularReferences() const
{
	if (GeneratedClass)
	{
		TArray<const UClass*, TInlineAllocator<32>> DiscoveredBlueprint;
		UWidget* Result = nullptr;
		if (UE::UMG::Private::HasCircularReferences(GeneratedClass, DiscoveredBlueprint, Result))
		{
			return MakeError(Result);
		}
	}
	return MakeValue();
}

TValueOrError<void, TSet<UWidget*>> UWidgetBlueprint::HasConflictingWidgetNamesFromInheritance() const
{
	if (GeneratedClass)
	{
		// we search for conflicting widget names when the parent is also a generated widgetblueprint
		// this allows bailing out early on most compilations
		if (const UWidgetBlueprint* WidgetBP = UE::UMG::Private::GetGeneratedWidgetBlueprintFromClass(GeneratedClass) )
		{
			if (Cast<const UWidgetBlueprintGeneratedClass>(WidgetBP->ParentClass) != nullptr)
			{
				TSet<FName> WidgetNames;
				TSet<UWidget*> Result;
				if (!UE::UMG::Private::TryBuildSetNameForWidgets(GeneratedClass, WidgetNames, Result))
				{
					return MakeError(Result);
				}
			}
		}
	}
	return MakeValue();
}

UPackage* UWidgetBlueprint::GetWidgetTemplatePackage() const
{
	return GetOutermost();
}

static bool HasLatentActions(UEdGraph* Graph)
{
	if (!Graph)
	{
		return false;
	}

	for (const UEdGraphNode* Node : Graph->Nodes)
	{
		if (const UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(Node))
		{
			// Check any function call nodes to see if they are latent.
			UFunction* TargetFunction = CallFunctionNode->GetTargetFunction();
			if (TargetFunction && TargetFunction->HasMetaData(FBlueprintMetadata::MD_Latent))
			{
				return true;
			}
		}

		else if (const UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(Node))
		{
			// Any macro graphs that haven't already been checked need to be checked for latent function calls
			//if (InspectedGraphList.Find(MacroInstanceNode->GetMacroGraph()) == INDEX_NONE)
			{
				if (HasLatentActions(MacroInstanceNode->GetMacroGraph()))
				{
					return true;
				}
			}
		}
		else if (const UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(Node))
		{
			// Any collapsed graphs that haven't already been checked need to be checked for latent function calls
			//if (InspectedGraphList.Find(CompositeNode->BoundGraph) == INDEX_NONE)
			{
				if (HasLatentActions(CompositeNode->BoundGraph))
				{
					return true;
				}
			}
		}
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWidgetBlueprint::UpdateTickabilityStats(bool& OutHasLatentActions, bool& OutHasAnimations, bool& OutClassRequiresNativeTick)
{
	bool bDummyFlag = false;
	UpdateTickabilityStatsAndInputRegistration(OutHasLatentActions, OutHasAnimations, OutClassRequiresNativeTick, bDummyFlag);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWidgetBlueprint::UpdateTickabilityStatsAndInputRegistration(bool& OutHasLatentActions, bool& OutHasAnimations, bool& OutClassRequiresNativeTick, bool& OutClassRequiresInputReg)
{
	if (GeneratedClass && GeneratedClass->ClassConstructor)
	{
		UWidgetBlueprintGeneratedClass* WidgetBPGeneratedClass = CastChecked<UWidgetBlueprintGeneratedClass>(GeneratedClass);
		UUserWidget* DefaultWidget = WidgetBPGeneratedClass->GetDefaultObject<UUserWidget>();

		TArray<UBlueprint*> BlueprintParents;
		UBlueprint::GetBlueprintHierarchyFromClass(WidgetBPGeneratedClass, BlueprintParents);

		bool bHasLatentActions = false;
		bool bHasAnimations = false;
		const bool bHasScriptImplementedTick = DefaultWidget->bHasScriptImplementedTick;
		bool bParentsRequireInputReg = false;

		for (UBlueprint* Blueprint : BlueprintParents)
		{
			UWidgetBlueprint* WidgetBP = Cast<UWidgetBlueprint>(Blueprint);
			if (WidgetBP)
			{
				// Check this blueprint parent for if it requires input registration
				UWidgetBlueprintGeneratedClass* GenClass = CastChecked<UWidgetBlueprintGeneratedClass>(WidgetBP->GeneratedClass);
				if (const UUserWidget* WidgetBPCDO = GenClass->GetDefaultObject<UUserWidget>())
				{
					bParentsRequireInputReg |= WidgetBPCDO->bAutomaticallyRegisterInputOnConstruction;
				}                           		                   	
            	
				bHasAnimations |= WidgetBP->Animations.Num() > 0;

				if (!bHasLatentActions)
				{
					TArray<UEdGraph*> AllGraphs;
					WidgetBP->GetAllGraphs(AllGraphs);

					for (UEdGraph* Graph : AllGraphs)
					{
						if (HasLatentActions(Graph))
						{
							bHasLatentActions = true;						
						}
					}
				}
			}
		}

		UClass* NativeParent = FBlueprintEditorUtils::GetNativeParent(this);
		static const FName DisableNativeTickMetaTag("DisableNativeTick");
		const bool bClassRequiresNativeTick = !NativeParent->HasMetaData(DisableNativeTickMetaTag);

		TickFrequency = DefaultWidget->GetDesiredTickFrequency();
		TickPredictionReason = TEXT("");
		TickPrediction = EWidgetCompileTimeTickPrediction::WontTick;
		switch (TickFrequency)
		{
		case EWidgetTickFrequency::Never:
			TickPrediction = EWidgetCompileTimeTickPrediction::WontTick;
			break;
		case EWidgetTickFrequency::Auto:
		{
			TArray<FString> Reasons;
			if (bHasScriptImplementedTick)
			{
				Reasons.Add(TEXT("Script"));
			}

			if (bClassRequiresNativeTick)
			{
				Reasons.Add(TEXT("Native"));
			}

			if (bHasAnimations)
			{
				Reasons.Add(TEXT("Anim"));
			}

			if (bHasLatentActions)
			{
				Reasons.Add(TEXT("Latent"));
			}

			for (int32 ReasonIdx = 0; ReasonIdx < Reasons.Num(); ++ReasonIdx)
			{
				TickPredictionReason += Reasons[ReasonIdx];
				if (ReasonIdx != Reasons.Num() - 1)
				{
					TickPredictionReason.AppendChar('|');
				}
			}

			if (bHasScriptImplementedTick || bClassRequiresNativeTick)
			{
				// Widget has an implemented tick or the generated class is not a direct child of UUserWidget (means it could have a native tick) then it will definitely tick
				TickPrediction = EWidgetCompileTimeTickPrediction::WillTick;
			}
			else if (bHasAnimations || bHasLatentActions)
			{
				// Widget has latent actions or animations and will tick if these are triggered
				TickPrediction = EWidgetCompileTimeTickPrediction::OnDemand;
			}
		}
		break;
		}

		OutHasLatentActions = bHasLatentActions;
		OutHasAnimations = bHasAnimations;
		OutClassRequiresNativeTick = bClassRequiresNativeTick;
		OutClassRequiresInputReg = bParentsRequireInputReg;
	}
}

bool UWidgetBlueprint::ArePropertyBindingsAllowed() const
{
	return GetRelevantSettings()->CompilerOption_PropertyBindingRule(this) == EPropertyBindingPermissionLevel::Allow;
}

TArray<FName> UWidgetBlueprint::GetInheritedAvailableNamedSlots() const
{
	if (const UWidgetBlueprintGeneratedClass* GeneratedBPClass = Cast<UWidgetBlueprintGeneratedClass>(GeneratedClass->GetSuperClass()))
	{
		return GeneratedBPClass->AvailableNamedSlots;
	}
	
	return TArray<FName>();
}

TSet<FName> UWidgetBlueprint::GetInheritedNamedSlotsWithContentInSameTree() const
{
	if (const UWidgetBlueprintGeneratedClass* GeneratedBPClass = Cast<UWidgetBlueprintGeneratedClass>(GeneratedClass->GetSuperClass()))
	{
		return GeneratedBPClass->NamedSlotsWithContentInSameTree;
	}

	return TSet<FName>();
}

UWidgetEditingProjectSettings* UWidgetBlueprint::GetRelevantSettings()
{
	return GetMutableDefault<UUMGEditorProjectSettings>();
}

const UWidgetEditingProjectSettings* UWidgetBlueprint::GetRelevantSettings() const
{
	return GetDefault<UUMGEditorProjectSettings>();
}

#if WITH_EDITOR
void UWidgetBlueprint::LoadModulesRequiredForCompilation()
{
	static const FName ModuleName(TEXT("UMGEditor"));
	FModuleManager::Get().LoadModule(ModuleName);
}
#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE 
