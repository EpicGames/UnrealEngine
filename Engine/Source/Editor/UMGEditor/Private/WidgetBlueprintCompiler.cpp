// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintCompiler.h"
#include "Components/SlateWrapperTypes.h"
#include "Blueprint/UserWidget.h"

#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_VariableGet.h"
#include "Blueprint/WidgetTree.h"
#include "Animation/WidgetAnimation.h"
#include "MovieScene.h"

#include "FieldNotification/CustomizationHelper.h"
#include "FieldNotificationHelpers.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Components/NamedSlot.h"
#include "WidgetBlueprintEditorUtils.h"
#include "WidgetGraphSchema.h"
#include "IUMGModule.h"
#include "WidgetEditingProjectSettings.h"
#include "WidgetCompilerRule.h"
#include "WidgetBlueprintExtension.h"
#include "Editor/WidgetCompilerLog.h"
#include "Editor.h"
#include "Algo/RemoveIf.h"

#define LOCTEXT_NAMESPACE "UMG"

#define CPF_Instanced (CPF_PersistentInstance | CPF_ExportObject | CPF_InstancedReference)

extern COREUOBJECT_API bool GMinimalCompileOnLoad;

namespace UE::WidgetBlueprintCompiler::Private
{
	FProperty* FindChildProperty(const UStruct* Struct, const FName& PropertyName)
	{
		for (FField* Field = Struct->ChildProperties; Field != nullptr; Field = Field->Next)
		{
			if (Field->GetFName() == PropertyName)
			{
				return CastField<FProperty>(Field);
			}
		}
				
		return nullptr;
	}
}

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompiler::FPopulateGeneratedVariablesContext
FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext::FPopulateGeneratedVariablesContext(FWidgetBlueprintCompilerContext& InContext)
	: Context(InContext)
{}

void FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext::AddGeneratedVariable(FBPVariableDescription&& VariableDescription) const
{
	Context.AddGeneratedVariable(MoveTemp(VariableDescription));
}

UWidgetBlueprint* FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext::GetWidgetBlueprint() const
{
	return Context.WidgetBlueprint();
}

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompiler::FCreateVariableContext
FWidgetBlueprintCompilerContext::FCreateVariableContext::FCreateVariableContext(FWidgetBlueprintCompilerContext& InContext)
	: Context(InContext)
{}

FProperty* FWidgetBlueprintCompilerContext::FCreateVariableContext::CreateVariable(const FName Name, const FEdGraphPinType& Type) const
{
	return Context.CreateVariable(Name, Type);
}

FMulticastDelegateProperty* FWidgetBlueprintCompilerContext::FCreateVariableContext::CreateMulticastDelegateVariable(const FName Name, const FEdGraphPinType& Type) const
{
	return Context.CreateMulticastDelegateVariable(Name, Type);
}

FMulticastDelegateProperty* FWidgetBlueprintCompilerContext::FCreateVariableContext::CreateMulticastDelegateVariable(const FName Name) const
{
	return Context.CreateMulticastDelegateVariable(Name);
}

void FWidgetBlueprintCompilerContext::FCreateVariableContext::AddGeneratedFunctionGraph(UEdGraph* Graph) const
{
	Context.GeneratedFunctionGraphs.Add(Graph);
}

UWidgetBlueprint* FWidgetBlueprintCompilerContext::FCreateVariableContext::GetWidgetBlueprint() const
{
	return Context.WidgetBlueprint();
}

UWidgetBlueprintGeneratedClass* FWidgetBlueprintCompilerContext::FCreateVariableContext::GetSkeletonGeneratedClass() const
{
	return Context.NewWidgetBlueprintClass;
}

UWidgetBlueprintGeneratedClass* FWidgetBlueprintCompilerContext::FCreateVariableContext::GetGeneratedClass() const
{
	return Context.NewWidgetBlueprintClass;
}


EKismetCompileType::Type FWidgetBlueprintCompilerContext::FCreateVariableContext::GetCompileType() const
{
	return Context.CompileOptions.CompileType;
}


//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompiler::FCreateFunctionContext
FWidgetBlueprintCompilerContext::FCreateFunctionContext::FCreateFunctionContext(FWidgetBlueprintCompilerContext& InContext)
	: Context(InContext)
{}

void FWidgetBlueprintCompilerContext::FCreateFunctionContext::AddGeneratedFunctionGraph(UEdGraph* Graph) const
{
	Context.GeneratedFunctionGraphs.Add(Graph);
}

void FWidgetBlueprintCompilerContext::FCreateFunctionContext::AddGeneratedUbergraphPage(UEdGraph* Graph) const
{
	Context.GeneratedUbergraphPages.Add(Graph);
}

UWidgetBlueprintGeneratedClass* FWidgetBlueprintCompilerContext::FCreateFunctionContext::GetGeneratedClass() const
{
	return Context.NewWidgetBlueprintClass;
}


//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompiler
FWidgetBlueprintCompiler::FWidgetBlueprintCompiler()
	: ReRegister(nullptr)
	, CompileCount(0)
{

}

bool FWidgetBlueprintCompiler::CanCompile(const UBlueprint* Blueprint)
{
	return Cast<UWidgetBlueprint>(Blueprint) != nullptr;
}


void FWidgetBlueprintCompiler::PreCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions)
{
	if (ReRegister == nullptr
		&& CanCompile(Blueprint)
		&& CompileOptions.CompileType == EKismetCompileType::Full)
	{
		ReRegister = new TComponentReregisterContext<UWidgetComponent>();
	}

	CompileCount++;
}

void FWidgetBlueprintCompiler::Compile(UBlueprint * Blueprint, const FKismetCompilerOptions & CompileOptions, FCompilerResultsLog & Results)
{
	if (UWidgetBlueprint* WidgetBlueprint = CastChecked<UWidgetBlueprint>(Blueprint))
	{
		FWidgetBlueprintCompilerContext Compiler(WidgetBlueprint, Results, CompileOptions);
		Compiler.Compile();
		check(Compiler.NewClass);
	}
}

void FWidgetBlueprintCompiler::PostCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions)
{
	CompileCount--;

	if (CompileCount == 0 && ReRegister)
	{
		delete ReRegister;
		ReRegister = nullptr;

		if (GIsEditor && GEditor)
		{
			GEditor->RedrawAllViewports(true);
		}
	}
}

bool FWidgetBlueprintCompiler::GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const
{
	if (ParentClass == UUserWidget::StaticClass() || ParentClass->IsChildOf(UUserWidget::StaticClass()))
	{
		OutBlueprintClass = UWidgetBlueprint::StaticClass();
		OutBlueprintGeneratedClass = UWidgetBlueprintGeneratedClass::StaticClass();
		return true;
	}

	return false;
}

FWidgetBlueprintCompilerContext::FWidgetBlueprintCompilerContext(UWidgetBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
	: Super(SourceSketch, InMessageLog, InCompilerOptions)
	, NewWidgetBlueprintClass(nullptr)
	, OldWidgetTree(nullptr)
	, WidgetSchema(nullptr)
{
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [this](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->BeginCompilation(*this);
		});
}

FWidgetBlueprintCompilerContext::~FWidgetBlueprintCompilerContext()
{
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->EndCompilation();
		});
}

UEdGraphSchema_K2* FWidgetBlueprintCompilerContext::CreateSchema()
{
	WidgetSchema = NewObject<UWidgetGraphSchema>();
	return WidgetSchema;
}

void FWidgetBlueprintCompilerContext::CreateFunctionList()
{
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [Self = this](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->CreateFunctionList(FCreateFunctionContext(*Self));
		});

	Super::CreateFunctionList();

	for ( FDelegateEditorBinding& EditorBinding : WidgetBlueprint()->Bindings )
	{
		if ( EditorBinding.SourcePath.IsEmpty() )
		{
			const FName PropertyName = EditorBinding.SourceProperty;

			FProperty* Property = FindFProperty<FProperty>(Blueprint->SkeletonGeneratedClass, PropertyName);
			if ( Property )
			{
				// Create the function graph.
				FString FunctionName = FString(TEXT("__Get")) + PropertyName.ToString();
				UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, FunctionName), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

				// Update the function binding to match the generated graph name
				EditorBinding.FunctionName = FunctionGraph->GetFName();

				const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(FunctionGraph->GetSchema());

				Schema->CreateDefaultNodesForGraph(*FunctionGraph);

				K2Schema->MarkFunctionEntryAsEditable(FunctionGraph, true);

				// Create a function entry node
				FGraphNodeCreator<UK2Node_FunctionEntry> FunctionEntryCreator(*FunctionGraph);
				UK2Node_FunctionEntry* EntryNode = FunctionEntryCreator.CreateNode();
				EntryNode->FunctionReference.SetSelfMember(FunctionGraph->GetFName());
				FunctionEntryCreator.Finalize();

				FGraphNodeCreator<UK2Node_FunctionResult> FunctionReturnCreator(*FunctionGraph);
				UK2Node_FunctionResult* ReturnNode = FunctionReturnCreator.CreateNode();
				ReturnNode->FunctionReference.SetSelfMember(FunctionGraph->GetFName());
				ReturnNode->NodePosX = EntryNode->NodePosX + EntryNode->NodeWidth + 256;
				ReturnNode->NodePosY = EntryNode->NodePosY;
				FunctionReturnCreator.Finalize();

				FEdGraphPinType PinType;
				K2Schema->ConvertPropertyToPinType(Property, /*out*/ PinType);

				UEdGraphPin* ReturnPin = ReturnNode->CreateUserDefinedPin(TEXT("ReturnValue"), PinType, EGPD_Input);

				// Auto-connect the pins for entry and exit, so that by default the signature is properly generated
				UEdGraphPin* EntryNodeExec = K2Schema->FindExecutionPin(*EntryNode, EGPD_Output);
				UEdGraphPin* ResultNodeExec = K2Schema->FindExecutionPin(*ReturnNode, EGPD_Input);
				EntryNodeExec->MakeLinkTo(ResultNodeExec);

				FGraphNodeCreator<UK2Node_VariableGet> MemberGetCreator(*FunctionGraph);
				UK2Node_VariableGet* VarNode = MemberGetCreator.CreateNode();
				VarNode->VariableReference.SetSelfMember(PropertyName);
				MemberGetCreator.Finalize();

				ReturnPin->MakeLinkTo(VarNode->GetValuePin());

				// We need to flag the entry node to make sure that the compiled function is callable from Kismet2
				int32 ExtraFunctionFlags = ( FUNC_Private | FUNC_Const );
				K2Schema->AddExtraFunctionFlags(FunctionGraph, ExtraFunctionFlags);

				//Blueprint->FunctionGraphs.Add(FunctionGraph);

				ProcessOneFunctionGraph(FunctionGraph, true);
				//FEdGraphUtilities::MergeChildrenGraphsIn(Ubergraph, FunctionGraph, /*bRequireSchemaMatch=*/ true);
			}
		}
	}
}

template<typename TOBJ>
struct FCullTemplateObjectsHelper
{
	const TArray<TOBJ*>& Templates;

	FCullTemplateObjectsHelper(const TArray<TOBJ*>& InComponentTemplates)
		: Templates(InComponentTemplates)
	{}

	bool operator()(const UObject* const RemovalCandidate) const
	{
		return ( NULL != Templates.FindByKey(RemovalCandidate) );
	}
};


void FWidgetBlueprintCompilerContext::CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOutOldCDO)
{
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	const bool bRecompilingOnLoad = Blueprint->bIsRegeneratingOnLoad;
	auto RenameObjectToTransientPackage = [bRecompilingOnLoad](UObject* ObjectToRename, const FName BaseName,  bool bClearFlags)
	{
		ObjectToRename->SetFlags(RF_Transient);

		if (bClearFlags)
		{
			ObjectToRename->ClearFlags(RF_Public | RF_Standalone | RF_ArchetypeObject);
		}

        // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(ObjectToRename);

		const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;

		if (BaseName.IsNone())
		{
			ObjectToRename->Rename(nullptr, GetTransientPackage(), RenFlags);
		}
		else
		{
			FName TransientArchetypeName = MakeUniqueObjectName(GetTransientPackage(), ObjectToRename->GetClass(), BaseName);
			ObjectToRename->Rename(*TransientArchetypeName.ToString(), GetTransientPackage(), RenFlags);
		}
	};

	if ( !Blueprint->bIsRegeneratingOnLoad && bIsFullCompile )
	{
		if (UWidgetBlueprintGeneratedClass* WBC_ToClean = Cast<UWidgetBlueprintGeneratedClass>(ClassToClean))
		{
			ClearGeneratedClassWidgetTree(WBC_ToClean, bRecompilingOnLoad);
		}
	}

	// Remove widgets that are created but not referenced by the widget tree. This could happen when another referenced UserWidget is modified.
	{
		TArray<UWidget*> OuterWidgets = WidgetBP->GetAllSourceWidgets();
		TArray<UWidget*> TreeWidgets;
		if (WidgetBP->WidgetTree)
		{
			WidgetBP->WidgetTree->GetAllWidgets(TreeWidgets);
		}

		FMemMark Mark(FMemStack::Get());
		TArray<UWidget*, TMemStackAllocator<>> WidgetsToRemove;
		WidgetsToRemove.Reserve(OuterWidgets.Num());
		
		struct FNameSlotInfo
		{
			TScriptInterface<INamedSlotInterface> NamedSlotHost;
			FName SlotName;
		};
		TMap<UWidget*, FNameSlotInfo> WidgetToNamedSlotInfo;
		WidgetToNamedSlotInfo.Reserve(OuterWidgets.Num());

		for (UWidget* OuterWidget : OuterWidgets)
		{
			if (TScriptInterface<INamedSlotInterface> NamedSlotHost = TScriptInterface<INamedSlotInterface>(OuterWidget))
			{
				TArray<FName> SlotNames;
				NamedSlotHost->GetSlotNames(SlotNames);
				for (FName SlotName : SlotNames)
				{
					if (UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName))
					{
						FNameSlotInfo Info = { NamedSlotHost, SlotName };
						WidgetToNamedSlotInfo.Add(SlotContent, Info);						
					}
				}
			}

			if (!TreeWidgets.Contains(OuterWidget))
			{
				WidgetsToRemove.Push(OuterWidget);
			}
		}

		if (WidgetsToRemove.Num() != 0)
		{
			if (WidgetBP->WidgetTree->RootWidget == nullptr)
			{
				MessageLog.Note(*LOCTEXT("RootWidgetEmpty", "There is no valid Widgets in this Widget Hierarchy.").ToString());
			}
			else
			{
				MessageLog.Note(*FText::Format(LOCTEXT("RootWidgetNamedMessage", "Some Widgets will be removed since they are not part of the Widget Hierarchy. Root Widget is  '{0}'."), FText::FromName(WidgetBP->WidgetTree->RootWidget->GetFName())).ToString());
			}

			// Log first to have all the parents and named slot intact for logging
			for (const UWidget* WidgetToClean : WidgetsToRemove)
			{
				if (UPanelWidget* Parent = WidgetToClean->GetParent())
				{
					MessageLog.Note(*FText::Format(LOCTEXT("UnusedWidgetFoundAndRemovedWithParent", "Removing unused widget '{0}' (Parent: '{1}')."), FText::FromName(WidgetToClean->GetFName()), FText::FromName(Parent->GetFName())).ToString());				
				}
				else if (const FNameSlotInfo* Info = WidgetToNamedSlotInfo.Find(WidgetToClean))
				{
					UObject* NamedSlotWidget = Info->NamedSlotHost.GetObject();
					if (ensure(NamedSlotWidget))
					{
						MessageLog.Note(*FText::Format(LOCTEXT("UnusedWidgetFoundAndRemovedWithNamedSlot", "Removing unused widget '{0}' (Named Slot '{1} in '{2}')."), FText::FromName(WidgetToClean->GetFName()), FText::FromName(Info->SlotName), FText::FromName(NamedSlotWidget->GetFName())).ToString());
					}
				}
				else
				{
					MessageLog.Note(*FText::Format(LOCTEXT("UnusedWidgetFoundAndRemoved", "Removing unused widget '{0}'."), FText::FromName(WidgetToClean->GetFName())).ToString());
				}
			}

			// Remove Widget
			for (UWidget* WidgetToClean : WidgetsToRemove)
			{
				FString TransientCDOString = FString::Printf(TEXT("TRASH_%s"), *WidgetToClean->GetName());
				RenameObjectToTransientPackage(WidgetToClean, *TransientCDOString, true);
			}
		}
	}

	Super::CleanAndSanitizeClass(ClassToClean, InOutOldCDO);

	// Make sure our typed pointer is set
	check(ClassToClean == NewClass && NewWidgetBlueprintClass == NewClass);

	for (UWidgetAnimation* Animation : NewWidgetBlueprintClass->Animations)
	{
		RenameObjectToTransientPackage(Animation, FName(), false);
	}
	NewWidgetBlueprintClass->Animations.Empty();
	NewWidgetBlueprintClass->Bindings.Empty();
	NewWidgetBlueprintClass->Extensions.Empty();

	if (UWidgetBlueprintGeneratedClass* WidgetClassToClean = Cast<UWidgetBlueprintGeneratedClass>(ClassToClean))
	{
		UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [WidgetClassToClean, InOutOldCDO](UWidgetBlueprintExtension* InExtension)
			{
				InExtension->CleanAndSanitizeClass(WidgetClassToClean, InOutOldCDO);
			});
	}
}

void FWidgetBlueprintCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean)
{
	Super::SaveSubObjectsFromCleanAndSanitizeClass(SubObjectsToSave, ClassToClean);

	// Make sure our typed pointer is set
	check(ClassToClean == NewClass);
	NewWidgetBlueprintClass = CastChecked<UWidgetBlueprintGeneratedClass>((UObject*)NewClass);

	OldWidgetTree = nullptr;
	OldWidgetAnimations.Empty();
	if (NewWidgetBlueprintClass)
	{
		OldWidgetTree = NewWidgetBlueprintClass->GetWidgetTreeArchetype();
		OldWidgetAnimations.Append(NewWidgetBlueprintClass->Animations);
	}

	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	// We need to save the widget tree to survive the initial sub-object clean blitz, 
	// otherwise they all get renamed, and it causes early loading errors.
	SubObjectsToSave.AddObject(WidgetBP->WidgetTree);

	if (UUserWidget* ClassDefaultWidgetToClean = Cast<UUserWidget>(ClassToClean->GetDefaultObject(false)))
	{
		// We need preserve any named slots that have been slotted into the CDO.  This can happen when someone subclasses
		// from a widget with named slots.  Those named slots are exposed to the child classes widget tree as
		// containers they can slot stuff into.  Those widgets need to survive recompile.
		for (FNamedSlotBinding& CDONamedSlotBinding : ClassDefaultWidgetToClean->NamedSlotBindings)
		{
			SubObjectsToSave.AddObject(CDONamedSlotBinding.Content);
		}
	}

	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [&SubObjectsToSave, LocalClass = NewWidgetBlueprintClass](UWidgetBlueprintExtension* InExtension)
		{
			SubObjectsToSave.AddObjects(InExtension->SaveSubObjectsFromCleanAndSanitizeClass(LocalClass));
		});
}

void FWidgetBlueprintCompilerContext::CreateClassVariablesFromBlueprint()
{
	Super::CreateClassVariablesFromBlueprint();

	UWidgetBlueprint* WidgetBP = WidgetBlueprint();
	if (WidgetBP == nullptr)
	{
		return;
	}

	UClass* ParentClass = WidgetBP->ParentClass;

	// Build the set of variables based on the variable widgets in the first Widget Tree we find:
	// in the current blueprint, the parent blueprint, and so on, until we find one.
	TArray<UWidget*> Widgets;
	UWidgetBlueprint* WidgetBPToScan = WidgetBP;
	while (WidgetBPToScan != nullptr)
	{
		Widgets = WidgetBPToScan->GetAllSourceWidgets();
		if (Widgets.Num() != 0)
		{
			// We found widgets. Stop search, but still check if we have a parent for bind widget validation
			UWidgetBlueprint* ParentWidgetBP = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy
				? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy)
				: nullptr;

			if (ParentWidgetBP)
			{
				TArray<UWidget*> ParentOwnedWidgets = ParentWidgetBP->GetAllSourceWidgets();
				ParentOwnedWidgets.Sort([](const UWidget& Lhs, const UWidget& Rhs) { return Rhs.GetFName().LexicalLess(Lhs.GetFName()); });

				for (UWidget* ParentOwnedWidget : ParentOwnedWidgets)
				{
					// Look in the Parent class properties to find a property with the BindWidget meta tag of the same name and Type.
					FObjectPropertyBase* ExistingProperty = CastField<FObjectPropertyBase>(ParentClass->FindPropertyByName(ParentOwnedWidget->GetFName()));
					if (ExistingProperty &&
						FWidgetBlueprintEditorUtils::IsBindWidgetProperty(ExistingProperty) &&
						ParentOwnedWidget->IsA(ExistingProperty->PropertyClass))
					{
						ParentWidgetToBindWidgetMap.Add(ParentOwnedWidget, ExistingProperty);
					}
				}
			}

			break;
		}
		
		// Get the parent WidgetBlueprint
		WidgetBPToScan = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy 
			? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy)
			: nullptr;
	}

	// Add widget variables
	for ( UWidget* Widget : Widgets ) 
	{
		// Look in the Parent class properties to find a property with the BindWidget meta tag of the same name and Type.
		FObjectPropertyBase* ExistingProperty = CastField<FObjectPropertyBase>(ParentClass->FindPropertyByName(Widget->GetFName()));
		if (ExistingProperty && 
			FWidgetBlueprintEditorUtils::IsBindWidgetProperty(ExistingProperty) && 
			Widget->IsA(ExistingProperty->PropertyClass))
		{
			WidgetToMemberVariableMap.Add(Widget, ExistingProperty);
			continue;
		}

		// Check if the widget has a generated variable
		for (const FBPVariableDescription& VarDesc : WidgetBP->GeneratedVariables)
		{
			if (VarDesc.VarName == Widget->GetFName())
			{
				FProperty* WidgetProperty = UE::WidgetBlueprintCompiler::Private::FindChildProperty(NewClass, VarDesc.VarName);
				if (ensureMsgf(WidgetProperty, TEXT("The Widget Blueprint [%s] has a generated variable for the widget [%s] but we failed to find a property for it."), *WidgetBP->GetName(), *Widget->GetName()))
				{
					WidgetToMemberVariableMap.Add(Widget, WidgetProperty);
				}
						
				break;
			}
		}
	}

	WidgetBPToScan = WidgetBP;
	while (WidgetBPToScan != nullptr)
	{
		// Look for BindWidgetAnim properties in parent widgetblueprints
		for (UWidgetAnimation* Animation : WidgetBPToScan->Animations)
		{
			FObjectPropertyBase* ExistingProperty = CastField<FObjectPropertyBase>(ParentClass->FindPropertyByName(Animation->GetFName()));
			if (ExistingProperty &&
				FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(ExistingProperty) &&
				ExistingProperty->PropertyClass->IsChildOf(UWidgetAnimation::StaticClass()))
			{
				WidgetAnimToMemberVariableMap.Add(Animation, ExistingProperty);
				continue;
			}

			// Create variables for widget animation
			if (WidgetBPToScan == WidgetBP)
			{
				// Check if the animation has a generated variable
				for (const FBPVariableDescription& VarDesc : WidgetBP->GeneratedVariables)
				{
					if (VarDesc.VarName == Animation->GetFName())
					{
						FProperty* AnimationProperty = UE::WidgetBlueprintCompiler::Private::FindChildProperty(NewClass, VarDesc.VarName);
						if (ensureMsgf(AnimationProperty, TEXT("The Widget Blueprint [%s] has a generated variable for the animation [%s] but we failed to find a property for it."), *WidgetBP->GetName(), *Animation->GetName()))
						{
							WidgetAnimToMemberVariableMap.Add(Animation, AnimationProperty);
						}
						
						break;
					}
				}
			}
		}

		// Get the parent WidgetBlueprint
		WidgetBPToScan = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy
			? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy)
			: nullptr;
	}

	FWidgetBlueprintCompilerContext* Self = this;
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [Self](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->CreateClassVariablesFromBlueprint(FCreateVariableContext(*Self));
		});
}

void FWidgetBlueprintCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	FKismetCompilerContext::CopyTermDefaultsToDefaultObject(DefaultObject);

	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	UUserWidget* DefaultWidget = CastChecked<UUserWidget>(DefaultObject);
	UWidgetBlueprintGeneratedClass* WidgetClass = CastChecked<UWidgetBlueprintGeneratedClass>(DefaultObject->GetClass());

	const bool bRequiresInputEvents = GetAutomaticallyRegisterInputEvents();

	if ( DefaultWidget )
	{
		//TODO Once we handle multiple derived blueprint classes, we need to check parent versions of the class.
		const UFunction* ReceiveTickEvent = FKismetCompilerUtilities::FindOverriddenImplementableEvent(GET_FUNCTION_NAME_CHECKED(UUserWidget, Tick), NewWidgetBlueprintClass);
		if (ReceiveTickEvent)
		{
			DefaultWidget->bHasScriptImplementedTick = true;
		}
		else
		{
			DefaultWidget->bHasScriptImplementedTick = false;
		}

		//TODO Once we handle multiple derived blueprint classes, we need to check parent versions of the class.
		if ( const UFunction* ReceivePaintEvent = FKismetCompilerUtilities::FindOverriddenImplementableEvent(GET_FUNCTION_NAME_CHECKED(UUserWidget, OnPaint), NewWidgetBlueprintClass) )
		{
			DefaultWidget->bHasScriptImplementedPaint = true;
		}
		else
		{
			DefaultWidget->bHasScriptImplementedPaint = false;
		}
		
		// This "bAutomaticallyRegisterInputEvents" is set to true on node expansion of an input event. 
		DefaultWidget->bAutomaticallyRegisterInputOnConstruction = bRequiresInputEvents;
	}

	bool bClassOrParentsHaveLatentActions = false;
	bool bClassOrParentsHaveAnimations = false;
	bool bClassRequiresNativeTick = false;
	bool bParentRequiresInputReg = false;

	WidgetBP->UpdateTickabilityStatsAndInputRegistration(bClassOrParentsHaveLatentActions, bClassOrParentsHaveAnimations, bClassRequiresNativeTick, bParentRequiresInputReg);
	WidgetClass->SetClassRequiresNativeTick(bClassRequiresNativeTick);
	
	// If either this BP or any of its parents require the input registration then we want to use it
	DefaultWidget->bAutomaticallyRegisterInputOnConstruction = bRequiresInputEvents || bParentRequiresInputReg;	

	// If the widget is not tickable, warn the user that widgets with animations or implemented ticks will most likely not work
	if (DefaultWidget->GetDesiredTickFrequency() == EWidgetTickFrequency::Never)
	{
		if (bClassOrParentsHaveAnimations)
		{
			MessageLog.Warning(*LOCTEXT("NonTickableButAnimationsFound", "This widget has animations but the widget is set to never tick.  These animations will not function correctly.").ToString());
		}

		if (bClassOrParentsHaveLatentActions)
		{
			MessageLog.Warning(*LOCTEXT("NonTickableButLatentActionsFound", "This widget has latent actions but the widget is set to never tick.  These latent actions will not function correctly.").ToString());
		}

		if (bClassRequiresNativeTick)
		{
			MessageLog.Warning(*LOCTEXT("NonTickableButNativeTickFound", "This widget may require a native tick but the widget is set to never tick.  Native tick will not be called.").ToString());
		}

		if (DefaultWidget->bHasScriptImplementedTick)
		{
			MessageLog.Warning(*LOCTEXT("NonTickableButTickFound", "This widget has a blueprint implemented Tick event but the widget is set to never tick.  This tick event will never be called.").ToString());
		}
	}

	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [DefaultObject](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->CopyTermDefaultsToDefaultObject(DefaultObject);
		});
}

void FWidgetBlueprintCompilerContext::SanitizeBindings(UBlueprintGeneratedClass* Class)
{
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	// Fast recompilation leaves bindings pointing to the skeleton and not the generated class. Rebase.
	for (FDelegateEditorBinding& Binding : WidgetBP->Bindings)
	{
		Binding.SourcePath.Rebase(WidgetBP);
	}

	// 
	TArray<FDelegateEditorBinding> StaleBindings;
	for (const FDelegateEditorBinding& Binding : WidgetBP->Bindings)
	{
		if (!Binding.DoesBindingTargetExist(WidgetBP))
		{
			StaleBindings.Add(Binding);
		}
	}

	// 
	for (const FDelegateEditorBinding& Binding : StaleBindings)
	{
		WidgetBP->Bindings.Remove(Binding);
	}

	// 
	int32 AttributeBindings = 0;
	for (const FDelegateEditorBinding& Binding : WidgetBP->Bindings)
	{
		if (Binding.IsAttributePropertyBinding(WidgetBP))
		{
			AttributeBindings++;
		}
	}

	WidgetBP->PropertyBindings = AttributeBindings;
}

void FWidgetBlueprintCompilerContext::ValidateAndFixUpVariableGuids()
{
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	// If we don't yet have any tracked variable guids, populate them deterministically
	// The determinism is required so that stable guids are serialized when creating external references to this widget's variables before this widget is resaved
	if (WidgetBP->WidgetVariableNameToGuidMap.IsEmpty())
	{
		WidgetBP->ForEachSourceWidget([WidgetBP](UWidget* Widget)
		{
			ensureAlways(!WidgetBP->WidgetVariableNameToGuidMap.Contains(Widget->GetFName()));
			WidgetBP->WidgetVariableNameToGuidMap.Emplace(Widget->GetFName(), FGuid::NewDeterministicGuid(Widget->GetPathName()));
		});

		for (UWidgetAnimation* Animation : WidgetBP->Animations)
		{
			if (Animation)
			{
				ensureAlways(!WidgetBP->WidgetVariableNameToGuidMap.Contains(Animation->GetFName()));
				WidgetBP->WidgetVariableNameToGuidMap.Emplace(Animation->GetFName(), FGuid::NewDeterministicGuid(Animation->GetPathName()));
			}
		}
	}
	else
	{
		// Validate that our variable guids are properly tracked and fixup issues that may have been caused by missed cases

		// Verify all variables have a stored GUID
		TSet<FName> SeenVariableNames;
		WidgetBP->ForEachSourceWidget([WidgetBP, &SeenVariableNames](UWidget* Widget)
		{
			if (!ensureAlwaysMsgf(WidgetBP->WidgetVariableNameToGuidMap.Contains(Widget->GetFName()), TEXT("Widget [%s] was added but did not get a GUID"), *Widget->GetName()))
			{
				WidgetBP->WidgetVariableNameToGuidMap.Add(Widget->GetFName(), FGuid::NewGuid());
			}
			SeenVariableNames.Add(Widget->GetFName());
		});

		for (UWidgetAnimation* Animation : WidgetBP->Animations)
		{
			if (Animation)
			{
				if (!ensureAlwaysMsgf(WidgetBP->WidgetVariableNameToGuidMap.Contains(Animation->GetFName()), TEXT("Animation [%s] was added but did not get a GUID"), *Animation->GetName()))
				{
					WidgetBP->WidgetVariableNameToGuidMap.Add(Animation->GetFName(), FGuid::NewGuid());
				}
				SeenVariableNames.Add(Animation->GetFName());
			}
		}

		TMap<FGuid, FName> GuidToVariableNameMap;

		// Verify we're only storing GUIDs for variables we still have and that none collide
		for (auto It = WidgetBP->WidgetVariableNameToGuidMap.CreateIterator(); It; ++It)
		{
			if (!ensureAlwaysMsgf(It.Value().IsValid(), TEXT("Variable [%s] has an invalid GUID"), *It.Key().ToString()))
			{
				It.Value() = FGuid::NewGuid();
			}
			
			if (ensureAlwaysMsgf(!GuidToVariableNameMap.Contains(It.Value()), TEXT("The variables [%s] and [%s] have the same GUID, delete and recreate one of them to fix this error"), *It.Key().ToString(), *GuidToVariableNameMap[It.Value()].ToString()))
			{
				GuidToVariableNameMap.Add(It.Value(), It.Key());
			}

			if (!ensureAlwaysMsgf(SeenVariableNames.Contains(It.Key()), TEXT("Variable [%s] was deleted but still has a GUID referenced by WidgetBlueprint [%s]"), *It.Key().ToString(), *WidgetBP->GetName()))
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FWidgetBlueprintCompilerContext::FixAbandonedWidgetTree(UWidgetBlueprint* WidgetBP)
{
	UWidgetTree* WidgetTree = WidgetBP->WidgetTree;

	if (ensure(WidgetTree))
	{
		if (WidgetTree->GetName() != TEXT("WidgetTree"))
		{
			if (UWidgetTree* AbandonedWidgetTree = static_cast<UWidgetTree*>(FindObjectWithOuter(WidgetBP, UWidgetTree::StaticClass(), TEXT("WidgetTree"))))
			{
				AbandonedWidgetTree->ClearFlags(RF_DefaultSubObject);
				AbandonedWidgetTree->SetFlags(RF_Transient);
				AbandonedWidgetTree->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
			}

			WidgetTree->Rename(TEXT("WidgetTree"), nullptr, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
			WidgetTree->SetFlags(RF_DefaultSubObject);
		}
	}
}

void FWidgetBlueprintCompilerContext::UpdateGeneratedClassWidgetTree(UWidgetBlueprint* WidgetBP, UWidgetBlueprintGeneratedClass* BPGClass, UWidgetTree* OldWidgetTree, FCompilerResultsLog* MessageLog)
{
	FixAbandonedWidgetTree(WidgetBP);

	{
		TGuardValue<uint32> DisableInitializeFromWidgetTree(UUserWidget::GetInitializingFromWidgetTree(), 0);

		// Need to clear archetype flag before duplication as we check during dup to see if we should postload
		EObjectFlags PreviousFlags = WidgetBP->WidgetTree->GetFlags();
		WidgetBP->WidgetTree->ClearFlags(RF_ArchetypeObject);

		TMap<UObject*, UObject*> DupObjectsMap;
		FObjectDuplicationParameters DupParams(WidgetBP->WidgetTree, BPGClass);
		DupParams.DestName = DupParams.SourceObject->GetFName();
		DupParams.FlagMask = RF_AllFlags & ~RF_DefaultSubObject;
		DupParams.PortFlags |= PPF_DuplicateVerbatim; // Skip resetting text IDs

		// if we are recompiling the BP on load, skip post load and defer it to the loading process
		FUObjectSerializeContext* LinkerLoadingContext = nullptr;
		if (WidgetBP->bIsRegeneratingOnLoad)
		{
			FLinkerLoad* Linker = WidgetBP->GetLinker();
			LinkerLoadingContext = Linker ? FUObjectThreadContext::Get().GetSerializeContext() : nullptr;
			DupParams.bSkipPostLoad = true;
			DupParams.CreatedObjects = &DupObjectsMap;
		}

		UWidgetTree* NewWidgetTree = Cast<UWidgetTree>(StaticDuplicateObjectEx(DupParams));

		// if we have anything in here after duplicate, then hook them in the loading process so they get post loaded
		if (LinkerLoadingContext)
		{
			TArray<UObject*> DupObjects;
			DupObjectsMap.GenerateValueArray(DupObjects);
			LinkerLoadingContext->AddUniqueLoadedObjects(DupObjects);
		}

		BPGClass->SetWidgetTreeArchetype(NewWidgetTree);
		if (OldWidgetTree)
		{
			FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(OldWidgetTree, NewWidgetTree);
		}
		OldWidgetTree = nullptr;

		WidgetBP->WidgetTree->SetFlags(PreviousFlags);
	}

	{
		TValueOrError<void, UWidget*> HasReference = WidgetBP->HasCircularReferences();
		if (HasReference.HasError())
		{
			if (UWidget* FoundCircularWidget = BPGClass->GetWidgetTreeArchetype()->FindWidget(HasReference.GetError()->GetFName()))
			{
				BPGClass->GetWidgetTreeArchetype()->RemoveWidget(FoundCircularWidget);
			}
			if (MessageLog)
			{
				MessageLog->Error(*FText::Format(LOCTEXT("WidgetTreeCircularReference", "The WidgetTree '{0}' Contains circular references. See widget '{1}'"),
					FText::FromString(WidgetBP->WidgetTree->GetPathName()),
					FText::FromString(HasReference.GetError()->GetName())
				).ToString());
			}
		}
	}

	{
#if WITH_EDITOR
		BPGClass->NameClashingInHierarchy.Reset();
#endif
		TValueOrError<void, TSet<UWidget*>> HasConflictingWidgetNames = WidgetBP->HasConflictingWidgetNamesFromInheritance();
		if (HasConflictingWidgetNames.HasError())
		{
			TSet<UWidget*>& ConflictingNames = HasConflictingWidgetNames.GetError();
			for (UWidget* ConflictingWidget : ConflictingNames)
			{
#if WITH_EDITOR
				FName ConflictingWidgetName = ConflictingWidget->GetFName();
				BPGClass->NameClashingInHierarchy.Add(ConflictingWidgetName);
#endif
				if (UWidget* FoundConflictingWidget = BPGClass->GetWidgetTreeArchetype()->FindWidget(ConflictingWidget->GetFName()))
				{
					BPGClass->GetWidgetTreeArchetype()->RemoveWidget(FoundConflictingWidget);
				}

				if (MessageLog)
				{
					MessageLog->Error(*FText::Format(LOCTEXT("WidgetTreeDuplicateNames", "The WidgetTree '{0}' already contains a widget named '{1}'."),
						FText::FromString(WidgetBP->WidgetTree->GetPathName()),
						FText::FromString(ConflictingWidget->GetName())
					).ToString());
				}
			}
		}
	}
}

void FWidgetBlueprintCompilerContext::ClearGeneratedClassWidgetTree(UWidgetBlueprintGeneratedClass* WBC_ToClean, bool bRecompilingOnLoad)
{
	auto RenameObjectToTransientPackage = [bRecompilingOnLoad](UObject* ObjectToRename, const FName BaseName, bool bClearFlags)
		{
			ObjectToRename->SetFlags(RF_Transient);

			if (bClearFlags)
			{
				ObjectToRename->ClearFlags(RF_Public | RF_Standalone | RF_ArchetypeObject);
			}

			// Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
			FLinkerLoad::InvalidateExport(ObjectToRename);

			const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;

			if (BaseName.IsNone())
			{
				ObjectToRename->Rename(nullptr, GetTransientPackage(), RenFlags);
			}
			else
			{
				FName TransientArchetypeName = MakeUniqueObjectName(GetTransientPackage(), ObjectToRename->GetClass(), BaseName);
				ObjectToRename->Rename(*TransientArchetypeName.ToString(), GetTransientPackage(), RenFlags);
			}
		};

	if (WBC_ToClean)
	{
		if (UWidgetTree* OldArchetype = WBC_ToClean->GetWidgetTreeArchetype())
		{
			FString TransientArchetypeString = FString::Printf(TEXT("OLD_TEMPLATE_TREE%s"), *OldArchetype->GetName());
			RenameObjectToTransientPackage(OldArchetype, *TransientArchetypeString, true);

			TArray<UObject*> Children;
			ForEachObjectWithOuter(OldArchetype, [&Children](UObject* Child) {
				Children.Add(Child);
				}, EGetObjectsFlags::None);

			for (UObject* Child : Children)
			{
				RenameObjectToTransientPackage(Child, FName(), false);
			}

			WBC_ToClean->SetWidgetTreeArchetype(nullptr);
		}
	}
}

void FWidgetBlueprintCompilerContext::FinishCompilingClass(UClass* Class)
{
	if (Class == nullptr)
		return;

	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	if (WidgetBP == nullptr)
		return;

	UClass* ParentClass = WidgetBP->ParentClass;

	if (ParentClass == nullptr)
		return;
	
	const bool bIsSkeletonOnly = CompileOptions.CompileType == EKismetCompileType::SkeletonOnly;

	UWidgetBlueprintGeneratedClass* BPGClass = CastChecked<UWidgetBlueprintGeneratedClass>(Class);

	if (BPGClass == nullptr)
		return;

	// Don't do a bunch of extra work on the skeleton generated class.
	if ( !bIsSkeletonOnly )
	{
		if( !WidgetBP->bHasBeenRegenerated )
		{
			UBlueprint::ForceLoadMembers(WidgetBP->WidgetTree, WidgetBP);
		}

		UpdateGeneratedClassWidgetTree(WidgetBP,  BPGClass, OldWidgetTree, &MessageLog);


		int32 AnimIndex = 0;
		for ( const UWidgetAnimation* Animation : WidgetBP->Animations )
		{
			UWidgetAnimation* ClonedAnimation = DuplicateObject<UWidgetAnimation>(Animation, BPGClass, *( Animation->GetName() + TEXT("_INST") ));
			//ClonedAnimation->SetFlags(RF_Public); // Needs to be marked public so that it can be referenced from widget instances.
			
			if (OldWidgetAnimations.IsValidIndex(AnimIndex) && OldWidgetAnimations[AnimIndex])
			{
				FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(OldWidgetAnimations[AnimIndex], ClonedAnimation);
			}

			BPGClass->Animations.Add(ClonedAnimation);
			AnimIndex++;
		}
		OldWidgetAnimations.Empty();

		// Only check bindings on a full compile.  Also don't check them if we're regenerating on load,
		// that has a nasty tendency to fail because the other dependent classes that may also be blueprints
		// might not be loaded yet.
		const bool bIsLoading = WidgetBP->bIsRegeneratingOnLoad;
		if ( bIsFullCompile )
		{
			SanitizeBindings(BPGClass);

			// Convert all editor time property bindings into a list of bindings
			// that will be applied at runtime.  Ensure all bindings are still valid.
			for ( const FDelegateEditorBinding& EditorBinding : WidgetBP->Bindings )
			{
				if ( bIsLoading || EditorBinding.IsBindingValid(Class, WidgetBP, MessageLog) )
				{
					BPGClass->Bindings.Add(EditorBinding.ToRuntimeBinding(WidgetBP));
				}
			}

			const EPropertyBindingPermissionLevel PropertyBindingRule = WidgetBP->GetRelevantSettings()->CompilerOption_PropertyBindingRule(WidgetBP);
			if (PropertyBindingRule != EPropertyBindingPermissionLevel::Allow)
			{
				if (WidgetBP->Bindings.Num() > 0)
				{
					for (const FDelegateEditorBinding& EditorBinding : WidgetBP->Bindings)
					{
						if (EditorBinding.IsAttributePropertyBinding(WidgetBP))
						{
							FText NoPropertyBindingsAllowedError =
								FText::Format(LOCTEXT("NoPropertyBindingsAllowed", "Property Bindings have been disabled for this widget.  You should remove the binding from {0}.{1}"),
								FText::FromString(EditorBinding.ObjectName),
								FText::FromName(EditorBinding.PropertyName));

							switch (PropertyBindingRule)
							{
							case EPropertyBindingPermissionLevel::PreventAndWarn:
								MessageLog.Warning(*NoPropertyBindingsAllowedError.ToString());
								break;
							case EPropertyBindingPermissionLevel::PreventAndError:
								MessageLog.Error(*NoPropertyBindingsAllowedError.ToString());
								break;
							}
						}
					}
				}
			}

			if (!WidgetBP->GetRelevantSettings()->CompilerOption_AllowBlueprintTick(WidgetBP))
			{
				const UFunction* ReceiveTickEvent = FKismetCompilerUtilities::FindOverriddenImplementableEvent(GET_FUNCTION_NAME_CHECKED(UUserWidget, Tick), NewWidgetBlueprintClass);
				if (ReceiveTickEvent)
				{
					MessageLog.Error(*LOCTEXT("TickNotAllowedForWidget", "Blueprint implementable ticking has been disabled for this widget in the Widget Designer (Team) - Project Settings").ToString());
				}
			}

			if (!WidgetBP->GetRelevantSettings()->CompilerOption_AllowBlueprintPaint(WidgetBP))
			{
				if (const UFunction* ReceivePaintEvent = FKismetCompilerUtilities::FindOverriddenImplementableEvent(GET_FUNCTION_NAME_CHECKED(UUserWidget, OnPaint), NewWidgetBlueprintClass))
				{
					MessageLog.Error(*LOCTEXT("PaintNotAllowedForWidget", "Blueprint implementable painting has been disabled for this widget in the Widget Designer (Team) - Project Settings.").ToString());
				}
			}

			// It's possible we may encounter some rules that haven't had a chance to load yet during early loading phases
			// They're automatically removed from the returned set.
			TArray<UWidgetCompilerRule*> CustomRules = WidgetBP->GetRelevantSettings()->CompilerOption_Rules(WidgetBP);
			for (UWidgetCompilerRule* CustomRule : CustomRules)
			{
				CustomRule->ExecuteRule(WidgetBP, MessageLog);
			}
		}

		// Add all the names of the named slot widgets to the slot names structure.
		{
		#if WITH_EDITOR
			BPGClass->NamedSlotsWithID.Reset();
			BPGClass->NamedSlotsWithContentInSameTree.Reset();
		#endif
			BPGClass->NamedSlots.Reset();
			BPGClass->InstanceNamedSlots.Reset();

			TArray<FName> NamedSlotsPerWidgetBlueprint;

			UWidgetBlueprint* WidgetBPIt = WidgetBP;
			while (WidgetBPIt)
			{
				NamedSlotsPerWidgetBlueprint.Reset();
				WidgetBPIt->ForEachSourceWidget([&] (const UWidget* Widget) {
					if (const UNamedSlot* NamedSlot = Cast<UNamedSlot>(Widget))
					{
						NamedSlotsPerWidgetBlueprint.Add(Widget->GetFName());

					#if WITH_EDITOR
						BPGClass->NamedSlotsWithID.Add(TPair<FName, FGuid>(Widget->GetFName(), NamedSlot->GetSlotGUID()));

						// A namedslot whose content is in the same blueprint class is treated as a regular panel widget.
						// We need to keep track of these to later remove them from the hierarchy.
						if (NamedSlot->GetChildrenCount() > 0)
						{
							BPGClass->NamedSlotsWithContentInSameTree.Add(NamedSlot->GetFName());
						}
					#endif

						if (NamedSlot->bExposeOnInstanceOnly)
						{
							BPGClass->InstanceNamedSlots.Add(Widget->GetFName());
						}
					}
				});
				
				// Here we reverse this array to maintain the order of sibling namedslots once the final array BPGClass->NamedSlots is reversed.
				Algo::Reverse(NamedSlotsPerWidgetBlueprint);
				BPGClass->NamedSlots.Append(NamedSlotsPerWidgetBlueprint);

				WidgetBPIt = Cast<UWidgetBlueprint>(WidgetBPIt->ParentClass->ClassGeneratedBy);
			}

			// We iterate widget blueprints from child to parent, but we need the final namedslot array to be sorted from parent to child, so we reverse it.
			Algo::Reverse(BPGClass->NamedSlots);

			BPGClass->AvailableNamedSlots = BPGClass->NamedSlots;

			// Remove any named slots from the available slots that has content for it.
			BPGClass->GetNamedSlotArchetypeContent([BPGClass](FName SlotName, UWidget* Content)
			{
				// If we find content for this slot, remove it from the available set.
				BPGClass->AvailableNamedSlots.Remove(SlotName);
			});

			// Remove any available subclass named slots that are marked as instance named slot.
			for (const FName& InstanceNamedSlot : BPGClass->InstanceNamedSlots)
			{
				BPGClass->AvailableNamedSlots.Remove(InstanceNamedSlot);
			}

			// Now add any available named slot that doesn't have anything in it also.
			for (const FName& AvailableNamedSlot : BPGClass->AvailableNamedSlots)
			{
				BPGClass->InstanceNamedSlots.AddUnique(AvailableNamedSlot);
			}
		}

		// Make sure that we don't have dueling widget hierarchies
		if (UWidgetBlueprintGeneratedClass* SuperBPGClass = Cast<UWidgetBlueprintGeneratedClass>(BPGClass->GetSuperClass()))
		{
			if (SuperBPGClass->ClassGeneratedBy) // ClassGeneratedBy can be null for cooked widget blueprints
			{
				UWidgetBlueprint* SuperBlueprint = Cast<UWidgetBlueprint>(SuperBPGClass->ClassGeneratedBy);
				if (ensure(SuperBlueprint) && SuperBlueprint->WidgetTree != nullptr)
				{
					if ((SuperBlueprint->WidgetTree->RootWidget != nullptr) && (WidgetBlueprint()->WidgetTree->RootWidget != nullptr))
					{
						// We both have a widget tree, terrible things will ensue
						// @todo: nickd - we need to switch this back to a warning in engine, but note for games
						MessageLog.Note(*LOCTEXT("ParentAndChildBothHaveWidgetTrees", "This widget @@ and parent class widget @@ both have a widget hierarchy, which is not supported.  Only one of them should have a widget tree.").ToString(),
							WidgetBP, SuperBPGClass->ClassGeneratedBy);
					}
				}
			}
		}

		// Do validation that as we subclass trees, we never stomp the slotted content of a parent widget.
		// doing that is not valid, as it would invalidate variables that were set?  This check could be
		// made more complex to only worry about cases with variables being generated, but that's a whole lot
		// extra, so for now lets just limit it to be safe.
		{
			TMap<FName, UWidget*> NamedSlotContentMap;
			// Make sure that we don't have dueling widget hierarchies
			UWidgetBlueprintGeneratedClass* NamedSlotClass = BPGClass;
			while (NamedSlotClass)
			{
				UWidgetTree* Tree = NamedSlotClass->GetWidgetTreeArchetype();
			
				TArray<FName> SlotNames;
				Tree->GetSlotNames(SlotNames);

				for (FName SlotName : SlotNames)
				{
					if (UWidget* ContentInSlot = Tree->GetContentForSlot(SlotName))
					{
						if (NamedSlotClass->NamedSlotsWithContentInSameTree.Contains(SlotName))
						{
							UClass* SubClassWithSlotFilled = ContentInSlot->GetTypedOuter<UClass>();
							MessageLog.Error(
								*FText::Format(
									 LOCTEXT("NamedSlotAlreadyFilledInOriginalTree", "The Named Slot '{0}' already has content in the widget blueprint it was created in but the subclass @@ tried to slot @@ into it. Please remove at least one of the contents."),
									FText::FromName(SlotName)
								).ToString(),
								SubClassWithSlotFilled, ContentInSlot);
						}
						if (!NamedSlotContentMap.Contains(SlotName))
						{
							NamedSlotContentMap.Add(SlotName, ContentInSlot);
						}
						else
						{
							UClass* SubClassWithSlotFilled = ContentInSlot->GetTypedOuter<UClass>();
							UClass* ParentClassWithSlotFilled = NamedSlotClass;
							MessageLog.Error(
								*FText::Format(
									LOCTEXT("NamedSlotAlreadyFilled", "The Named Slot '{0}' already contains @@ from the class @@ but the subclass @@ tried to slot @@ into it. Please remove the content @@ from the class @@ to fix this error."),
									FText::FromName(SlotName)
								).ToString(),
								ContentInSlot, ParentClassWithSlotFilled,
								SubClassWithSlotFilled, NamedSlotContentMap.FindRef(SlotName),
								ContentInSlot, ParentClassWithSlotFilled);
						}
					}
				}
			
				NamedSlotClass = Cast<UWidgetBlueprintGeneratedClass>(NamedSlotClass->GetSuperClass());
			}
		}
	}

	if (bIsSkeletonOnly || WidgetBP->SkeletonGeneratedClass != Class)
	{
		bool bCanCallPreConstruct = true;

		// Check that all BindWidget properties are present and of the appropriate type
		for (TFObjectPropertyBase<UWidget*>* WidgetProperty : TFieldRange<TFObjectPropertyBase<UWidget*>>(ParentClass))
		{
			bool bIsOptional = false;

			if (FWidgetBlueprintEditorUtils::IsBindWidgetProperty(WidgetProperty, bIsOptional))
			{
				const FText OptionalBindingAvailableNote = LOCTEXT("OptionalWidgetNotBound", "An optional widget binding \"{0}\" of type @@ is available.");
				const FText RequiredWidgetNotBoundError = LOCTEXT("RequiredWidgetNotBound", "A required widget binding \"{0}\" of type @@ was not found.");
				const FText IncorrectWidgetTypeError = LOCTEXT("IncorrectWidgetTypes", "The widget @@ is of type @@, but the bind widget property is of type @@.");

				UWidget* const* Widget = WidgetToMemberVariableMap.FindKey(WidgetProperty);

				// If at first we don't find the binding, search the parent binding map
				if (!Widget)
				{
					Widget = ParentWidgetToBindWidgetMap.FindKey(WidgetProperty);
				}

				if (!Widget)
				{
					if (bIsOptional)
					{
						MessageLog.Note(*FText::Format(OptionalBindingAvailableNote, FText::FromName(WidgetProperty->GetFName())).ToString(), WidgetProperty->PropertyClass);
					}
					else if (Blueprint->bIsNewlyCreated)
					{
						MessageLog.Warning(*FText::Format(RequiredWidgetNotBoundError, FText::FromName(WidgetProperty->GetFName())).ToString(), WidgetProperty->PropertyClass);
						bCanCallPreConstruct = false;
					}
					else
					{
						MessageLog.Error(*FText::Format(RequiredWidgetNotBoundError, FText::FromName(WidgetProperty->GetFName())).ToString(), WidgetProperty->PropertyClass);
						bCanCallPreConstruct = false;
					}
				}
				else if (!(*Widget)->IsA(WidgetProperty->PropertyClass))
				{
					if (Blueprint->bIsNewlyCreated)
					{
						MessageLog.Warning(*IncorrectWidgetTypeError.ToString(), *Widget, (*Widget)->GetClass(), WidgetProperty->PropertyClass);
						bCanCallPreConstruct = false;
					}
					else
					{
						MessageLog.Error(*IncorrectWidgetTypeError.ToString(), *Widget, (*Widget)->GetClass(), WidgetProperty->PropertyClass);
						bCanCallPreConstruct = false;
					}
				}
			}
		}

		if (UWidgetBlueprintGeneratedClass* BPGC = Cast<UWidgetBlueprintGeneratedClass>(WidgetBP->GeneratedClass))
		{
			BPGC->bCanCallPreConstruct = bCanCallPreConstruct;
		}

		// Check that all BindWidgetAnim properties are present
		for (TFObjectPropertyBase<UWidgetAnimation*>* WidgetAnimProperty : TFieldRange<TFObjectPropertyBase<UWidgetAnimation*>>(ParentClass))
		{
			bool bIsOptional = false;

			if (FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(WidgetAnimProperty, bIsOptional))
			{
				const FText OptionalBindingAvailableNote = LOCTEXT("OptionalWidgetAnimNotBound", "An optional widget animation binding @@ is available.");
				const FText RequiredWidgetAnimNotBoundError = LOCTEXT("RequiredWidgetAnimNotBound", "A required widget animation binding @@ was not found.");

				UWidgetAnimation* const* WidgetAnim = WidgetAnimToMemberVariableMap.FindKey(WidgetAnimProperty);
				if (!WidgetAnim)
				{
					if (bIsOptional)
					{
						MessageLog.Note(*OptionalBindingAvailableNote.ToString(), WidgetAnimProperty);
					}
					else if (Blueprint->bIsNewlyCreated)
					{
						MessageLog.Warning(*RequiredWidgetAnimNotBoundError.ToString(), WidgetAnimProperty);
					}
					else
					{
						MessageLog.Error(*RequiredWidgetAnimNotBoundError.ToString(), WidgetAnimProperty);
					}
				}

				if (!WidgetAnimProperty->HasAnyPropertyFlags(CPF_Transient))
				{
					const FText BindWidgetAnimTransientError = LOCTEXT("BindWidgetAnimTransient", "The property @@ uses BindWidgetAnim, but isn't Transient!");
					MessageLog.Error(*BindWidgetAnimTransientError.ToString(), WidgetAnimProperty);
				}
			}
		}
	}

	if (BPGClass)
	{
		BPGClass->bCanCallInitializedWithoutPlayerContext = WidgetBP->bCanCallInitializedWithoutPlayerContext;
	}

	Super::FinishCompilingClass(Class);

	CA_ASSUME(BPGClass);
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [BPGClass](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->FinishCompilingClass(BPGClass);
		});
}

class FBlueprintCompilerLog : public IWidgetCompilerLog
{
public:
	FBlueprintCompilerLog(FCompilerResultsLog& InMessageLog, TSubclassOf<UUserWidget> InClassContext)
		: MessageLog(InMessageLog)
		, ClassContext(InClassContext)
	{
	}

	virtual TSubclassOf<UUserWidget> GetContextClass() const override
	{
		return ClassContext;
	}

	virtual void InternalLogMessage(TSharedRef<FTokenizedMessage>& InMessage) override
	{
		MessageLog.AddTokenizedMessage(InMessage);
	}

private:
	// Compiler message log (errors, warnings, notes)
	FCompilerResultsLog& MessageLog;
	TSubclassOf<UUserWidget> ClassContext;
};

void FWidgetBlueprintCompilerContext::ValidateWidgetAnimations()
{
	UWidgetBlueprintGeneratedClass* WidgetClass = NewWidgetBlueprintClass;
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();
	UUserWidget* UserWidget = WidgetClass->GetDefaultObject<UUserWidget>();
	FBlueprintCompilerLog BlueprintLog(MessageLog, WidgetClass);
	
	UWidgetTree* LatestWidgetTree = FWidgetBlueprintEditorUtils::FindLatestWidgetTree(WidgetBP, UserWidget);
	
	for (const UWidgetAnimation* InAnimation : WidgetBP->Animations)
	{
		for (const FWidgetAnimationBinding& Binding : InAnimation->AnimationBindings)
		{
			// Look for the object bindings within the widget
			UObject* FoundObject = Binding.FindRuntimeObject(*LatestWidgetTree, *UserWidget, InAnimation, nullptr);

			// If any of the FoundObjects is null, we do not play the animation.
			if (FoundObject == nullptr)
			{
				FoundObject = Binding.FindRuntimeObject(*WidgetBP->WidgetTree, *UserWidget, InAnimation, nullptr);
				if (FoundObject == nullptr)
				{
					// Notify the user of the null track in the editor
					const FText AnimationNullTrackMessage = LOCTEXT("AnimationNullTrack", "UMG Animation '{0}' from '{1}' is trying to animate a non-existent widget through binding '{2}'. Please re-bind or delete this object from the animation.");
					BlueprintLog.Warning(FText::Format(AnimationNullTrackMessage, InAnimation->GetDisplayName(), FText::FromString(UserWidget->GetClass()->GetName()), FText::FromName(Binding.WidgetName)));
				}
			}
		}
	}
}

void FWidgetBlueprintCompilerContext::OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context)
{
	Super::OnPostCDOCompiled(Context);

	if (Context.bIsSkeletonOnly)
	{
		return;
	}

	WidgetToMemberVariableMap.Empty();
	WidgetAnimToMemberVariableMap.Empty();
	ParentWidgetToBindWidgetMap.Empty();

	UWidgetBlueprintGeneratedClass* WidgetClass = NewWidgetBlueprintClass;
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	if (!Blueprint->bIsRegeneratingOnLoad && bIsFullCompile)
	{
		FBlueprintCompilerLog BlueprintLog(MessageLog, WidgetClass);
		WidgetClass->GetDefaultObject<UUserWidget>()->ValidateBlueprint(*WidgetBP->WidgetTree, BlueprintLog);
		ValidateWidgetAnimations();
	}

	ValidateDesiredFocusWidgetName();
}


void FWidgetBlueprintCompilerContext::ValidateDesiredFocusWidgetName()
{
	if (UWidgetBlueprintGeneratedClass* WidgetClass = NewWidgetBlueprintClass)
	{
		UWidgetBlueprint* WidgetBP = WidgetBlueprint();
		UUserWidget* UserWidgetCDO = WidgetClass->GetDefaultObject<UUserWidget>();
		if (WidgetBP && UserWidgetCDO)
		{
			FName DesiredFocusWidgetName = UserWidgetCDO->GetDesiredFocusWidgetName();

			if (!DesiredFocusWidgetName.IsNone())
			{
				// Check the current tree first as it could includes named slot content
				if (WidgetBP->WidgetTree->FindWidget(DesiredFocusWidgetName))
				{
					return;
				}

				// If there is a root widget, also check the latest merged tree (which may include parent-inherited widgets)
				if (WidgetBP->WidgetTree->RootWidget)
				{
					const UWidgetTree* LatestWidgetTree = FWidgetBlueprintEditorUtils::FindLatestWidgetTree(WidgetBP, UserWidgetCDO);
					if (LatestWidgetTree->FindWidget(DesiredFocusWidgetName))
					{
						return;
					}
				}

				FBlueprintCompilerLog BlueprintLog(MessageLog, WidgetClass);
				// Notify that the desired focus widget is not found in the Widget tree, so it's invalid.
				const FText InvalidDesiredFocusWidgetNameMessage = LOCTEXT("InvalidDesiredFocusWidgetName", "User Widget '{0}' Desired Focus is set to a non-existent widget '{1}'. Select a valid desired focus for this User Widget.");
				BlueprintLog.Warning(FText::Format(InvalidDesiredFocusWidgetNameMessage, FText::FromString(UserWidgetCDO->GetClass()->GetName()), FText::FromName(DesiredFocusWidgetName)));
			}
		}
	}
}

void FWidgetBlueprintCompilerContext::EnsureProperGeneratedClass(UClass*& TargetUClass)
{
	if ( TargetUClass && !( (UObject*)TargetUClass )->IsA(UWidgetBlueprintGeneratedClass::StaticClass()) )
	{
		FKismetCompilerUtilities::ConsignToOblivion(TargetUClass, Blueprint->bIsRegeneratingOnLoad);
		TargetUClass = nullptr;
	}
}

void FWidgetBlueprintCompilerContext::PopulateBlueprintGeneratedVariables()
{
	Super::PopulateBlueprintGeneratedVariables();

	ValidateAndFixUpVariableGuids();
	
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();

	// Widget Variables
	{
		TArray<UWidget*> SortedWidgets = WidgetBP->GetAllSourceWidgets();
		SortedWidgets.Sort( []( const UWidget& Lhs, const UWidget& Rhs ) { return Rhs.GetFName().LexicalLess(Lhs.GetFName()); } );

		for (UWidget* Widget : SortedWidgets)
		{
			// All UNamedSlot widgets are automatically variables, so that we can properly look them up quickly with FindField
			// in UserWidgets.
			// In the event there are bindings for a widget, but it's not marked as a variable, make it one, but hide it from the UI.
			// we do this so we can use FindField to locate it at runtime.
			const bool bShouldGenerateVariable = Widget->bIsVariable
				|| Widget->IsA<UNamedSlot>()
				|| WidgetBP->Bindings.ContainsByPredicate([&Widget] (const FDelegateEditorBinding& Binding) {
					return Binding.ObjectName == Widget->GetName();
				});

			if (!bShouldGenerateVariable)
			{
				continue;
			}

			// Look in the Parent class properties to find a property with the BindWidget meta tag of the same name and Type.
			FObjectPropertyBase* ExistingProperty = WidgetBP->ParentClass ? CastField<FObjectPropertyBase>(WidgetBP->ParentClass->FindPropertyByName(Widget->GetFName())) : nullptr;
			if (ExistingProperty && 
				FWidgetBlueprintEditorUtils::IsBindWidgetProperty(ExistingProperty) && 
				Widget->IsA(ExistingProperty->PropertyClass))
			{
				continue;
			}

			// This code was added to fix the problem of recompiling dependent widgets, not using the newest
			// class thus resulting in REINST failures in dependent blueprints.
			UClass* WidgetClass = Widget->GetClass();
			if ( UBlueprintGeneratedClass* BPWidgetClass = Cast<UBlueprintGeneratedClass>(WidgetClass) )
			{
				WidgetClass = BPWidgetClass->GetAuthoritativeClass();
			}
			
			ensure(WidgetBP->WidgetVariableNameToGuidMap.Contains(Widget->GetFName()));

			FBPVariableDescription WidgetVariableDesc;
			WidgetVariableDesc.VarName = Widget->GetFName();
			WidgetVariableDesc.VarGuid = WidgetBP->WidgetVariableNameToGuidMap.FindRef(Widget->GetFName());
			WidgetVariableDesc.VarType = FEdGraphPinType(UEdGraphSchema_K2::PC_Object, NAME_None, WidgetClass, EPinContainerType::None, false, FEdGraphTerminalType());
			WidgetVariableDesc.FriendlyName = Widget->IsGeneratedName() ? Widget->GetName() : Widget->GetLabelText().ToString();
			WidgetVariableDesc.PropertyFlags = (CPF_PersistentInstance | CPF_ExportObject | CPF_InstancedReference | CPF_RepSkip);
		
			// Only show variables if they're explicitly marked as variables.
			if (Widget->bIsVariable)
			{
				WidgetVariableDesc.PropertyFlags |= (CPF_BlueprintVisible | CPF_BlueprintReadOnly | CPF_DisableEditOnInstance);

				// Only include Category metadata for variables (i.e. a visible/editable property); otherwise, UHT will raise a warning if this Blueprint is nativized.
				const FString& CategoryName = Widget->GetCategoryName();
				WidgetVariableDesc.SetMetaData(TEXT("Category"), *(CategoryName.IsEmpty() ? WidgetBP->GetName() : CategoryName));
			}

			AddGeneratedVariable(MoveTemp(WidgetVariableDesc));
		}
	}

	// Animation Variables
	{
		for (UWidgetAnimation* Animation : WidgetBP->Animations)
		{
			// BindWidgetAnims already have properties
			FObjectPropertyBase* ExistingProperty = WidgetBP->ParentClass ? CastField<FObjectPropertyBase>(WidgetBP->ParentClass->FindPropertyByName(Animation->GetFName())) : nullptr;
			if (ExistingProperty &&
				FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(ExistingProperty) &&
				ExistingProperty->PropertyClass->IsChildOf(UWidgetAnimation::StaticClass()))
			{
				continue;
			}

			ensure(WidgetBP->WidgetVariableNameToGuidMap.Contains(Animation->GetFName()));
			FBPVariableDescription AnimVariableDesc;
			AnimVariableDesc.VarName = Animation->GetFName();
			AnimVariableDesc.VarGuid = WidgetBP->WidgetVariableNameToGuidMap.FindRef(Animation->GetFName());
			AnimVariableDesc.VarType = FEdGraphPinType(UEdGraphSchema_K2::PC_Object, NAME_None, Animation->GetClass(), EPinContainerType::None, true, FEdGraphTerminalType());
			AnimVariableDesc.FriendlyName =  Animation->GetDisplayName().ToString();
			AnimVariableDesc.PropertyFlags = (CPF_Transient | CPF_BlueprintVisible | CPF_BlueprintReadOnly | CPF_RepSkip);
			AnimVariableDesc.SetMetaData(TEXT("Category"), TEXT("Animations"));
			ensure(!WidgetBP->GeneratedVariables.ContainsByPredicate([&AnimVariableDesc](const FBPVariableDescription& VariableDescription) { return VariableDescription.VarName == AnimVariableDesc.VarName; }));
			ensure(!AnimVariableDesc.VarGuid.IsValid() || !WidgetBP->GeneratedVariables.ContainsByPredicate([&AnimVariableDesc](const FBPVariableDescription& VariableDescription) { return VariableDescription.VarGuid.IsValid() && VariableDescription.VarGuid == AnimVariableDesc.VarGuid; }));
			AddGeneratedVariable(MoveTemp(AnimVariableDesc));
		}
	}

	FWidgetBlueprintCompilerContext* Self = this;
	UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [Self](UWidgetBlueprintExtension* InExtension)
		{
			InExtension->PopulateGeneratedVariables(FPopulateGeneratedVariablesContext(*Self));
		});
}

void FWidgetBlueprintCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	NewWidgetBlueprintClass = FindObject<UWidgetBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if ( NewWidgetBlueprintClass == nullptr )
	{
		NewWidgetBlueprintClass = NewObject<UWidgetBlueprintGeneratedClass>(Blueprint->GetOutermost(), FName(*NewClassName), RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewWidgetBlueprintClass);
	}
	NewClass = NewWidgetBlueprintClass;
}

void FWidgetBlueprintCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* ClassToUse)
{
	NewWidgetBlueprintClass = CastChecked<UWidgetBlueprintGeneratedClass>(ClassToUse);
}

void FWidgetBlueprintCompilerContext::PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags)
{
	Super::PrecompileFunction(Context, InternalFlags);

	VerifyEventReplysAreNotEmpty(Context);
}

void FWidgetBlueprintCompilerContext::VerifyEventReplysAreNotEmpty(FKismetFunctionContext& Context)
{
	TArray<UK2Node_FunctionResult*> FunctionResults;
	Context.SourceGraph->GetNodesOfClass<UK2Node_FunctionResult>(FunctionResults);

	UScriptStruct* EventReplyStruct = FEventReply::StaticStruct();
	FEdGraphPinType EventReplyPinType(UEdGraphSchema_K2::PC_Struct, NAME_None, EventReplyStruct, EPinContainerType::None, /*bIsReference =*/false, /*InValueTerminalType =*/FEdGraphTerminalType());

	for ( UK2Node_FunctionResult* FunctionResult : FunctionResults )
	{
		for ( UEdGraphPin* ReturnPin : FunctionResult->Pins )
		{
			if ( ReturnPin->PinType == EventReplyPinType )
			{
				const bool IsUnconnectedEventReply = ReturnPin->Direction == EEdGraphPinDirection::EGPD_Input && ReturnPin->LinkedTo.Num() == 0;
				if ( IsUnconnectedEventReply )
				{
					MessageLog.Warning(*LOCTEXT("MissingEventReply_Warning", "Event Reply @@ should not be empty.  Return a reply such as Handled or Unhandled.").ToString(), ReturnPin);
				}
			}
		}
	}
}

bool FWidgetBlueprintCompilerContext::ValidateGeneratedClass(UBlueprintGeneratedClass* Class)
{
	const bool bSuperResult = Super::ValidateGeneratedClass(Class);
	const bool bResult = UWidgetBlueprint::ValidateGeneratedClass(Class);

	UWidgetBlueprintGeneratedClass* WidgetClass = Cast<UWidgetBlueprintGeneratedClass>(Class);
	bool bExtension = WidgetClass != nullptr;
	if (bExtension)
	{
		UWidgetBlueprintExtension::ForEachExtension(WidgetBlueprint(), [&bExtension, WidgetClass](UWidgetBlueprintExtension* InExtension)
			{
				bExtension = InExtension->ValidateGeneratedClass(WidgetClass) && bExtension;
			});
	}

	return bSuperResult && bResult && bExtension;
}

void FWidgetBlueprintCompilerContext::AddExtension(UWidgetBlueprintGeneratedClass* Class, UWidgetBlueprintGeneratedClassExtension* Extension)
{
	check(Class);
	check(Extension);
	Class->Extensions.Add(Extension);
}

void FWidgetBlueprintCompilerContext::AddGeneratedVariable(FBPVariableDescription&& VariableDescription) const
{
	UWidgetBlueprint* WidgetBP = WidgetBlueprint();
	ensureAlwaysMsgf(!WidgetBP->GeneratedVariables.ContainsByPredicate([&VariableDescription](const FBPVariableDescription& TestVar) { return TestVar.VarName == VariableDescription.VarName; }),
		TEXT("Widget Blueprint [%s] already contains generated variable with name [%s]"), *GetNameSafe(WidgetBP), *VariableDescription.VarName.ToString());
	ensureAlwaysMsgf(!VariableDescription.VarGuid.IsValid() || !WidgetBP->GeneratedVariables.ContainsByPredicate([&VariableDescription](const FBPVariableDescription& TestVar) { return TestVar.VarGuid.IsValid() && TestVar.VarGuid == VariableDescription.VarGuid; }),
		TEXT("Attempting to add generated variable [%s] to Widget Blueprint [%s] that has the same GUID as another variable"), *VariableDescription.VarName.ToString(), *GetNameSafe(WidgetBP));
	
	WidgetBP->GeneratedVariables.Emplace(MoveTemp(VariableDescription));
}

#undef LOCTEXT_NAMESPACE
