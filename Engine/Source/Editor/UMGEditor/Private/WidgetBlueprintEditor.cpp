// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintEditor.h"
#include "MovieSceneBinding.h"
#include "MovieSceneFolder.h"
#include "MovieScene.h"
#include "Animation/WidgetAnimation.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "Engine/SimpleConstructionScript.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "WidgetBlueprint.h"
#include "StatusBarSubsystem.h"
#include "Editor.h"
#include "WidgetBlueprintToolMenuContext.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR

#include "Algo/AllOf.h"

#include "Components/PanelSlot.h"
#include "Components/PanelWidget.h"
#include "Settings/WidgetDesignerSettings.h"

#include "Tracks/MovieScenePropertyTrack.h"
#include "ISequencerModule.h"
#include "SequencerSettings.h"
#include "ObjectEditorUtils.h"

#include "PropertyCustomizationHelpers.h"

#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Blueprint/WidgetTree.h"
#include "WidgetBlueprintEditorUtils.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "BlueprintModes/WidgetDesignerApplicationMode.h"
#include "BlueprintModes/WidgetGraphApplicationMode.h"
#include "BlueprintModes/WidgetPreviewApplicationMode.h"
#include "WidgetModeManager.h"

#include "WidgetBlueprintEditorToolbar.h"
#include "Components/CanvasPanel.h"
#include "Framework/Commands/GenericCommands.h"
#include "Kismet2/CompilerResultsLog.h"
#include "HAL/FileManager.h"
#include "IMessageLogListing.h"
#include "WidgetGraphSchema.h"

#include "Animation/MovieSceneWidgetMaterialTrack.h"
#include "Animation/WidgetMaterialTrackUtilities.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#include "ScopedTransaction.h"

#include "Designer/SDesignerView.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UMGEditorActions.h"
#include "UMGEditorModule.h"
#include "GameProjectGenerationModule.h"
#include "Tools/ToolCompatible.h"

#include "Preview/PreviewMode.h"
#include "Palette/SPaletteViewModel.h"
#include "Library/SLibraryViewModel.h"

#include "DesktopPlatformModule.h"
#include "Engine/MemberReference.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IDesktopPlatform.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageUtils.h"
#include "Serialization/BufferArchive.h"
#include "Widgets/SVirtualWindow.h"
#include "TabFactory/AnimationTabSummoner.h"
#include "TabFactory/DesignerTabSummoner.h"
#include "ToolPalette/WidgetEditorModeUILayer.h"
#include "BlueprintEditorTabs.h"

#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"
#include "GraphEditorActions.h"
#include "MovieSceneDynamicBindingCustomization.h"
#include "MovieSceneSequenceEditor.h"
#include "UIComponentWidgetBlueprintExtension.h"

#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentUserWidgetExtension.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Editor
{
	bool bDisplayWidgetVariableGuids = false;
	FAutoConsoleVariableRef CVarUMGEditorDisplayWidgetVariableGuids(
		TEXT("UMG.Editor.DisplayWidgetVariableGuids"),
		bDisplayWidgetVariableGuids,
		TEXT("Toggles displaying guids for variables generated by this blueprint (Widgets and Animations)."));
}

FWidgetBlueprintEditor::FWidgetBlueprintEditor()
	: PreviewScene(FPreviewScene::ConstructionValues().AllowAudioPlayback(true).ShouldSimulatePhysics(true))
	, PreviewBlueprint(nullptr)
	, bIsSimulateEnabled(false)
	, bIsRealTime(true)
	, bIsSequencerDrawerOpen(false)
	, bRefreshGeneratedClassAnimations(false)
	, bUpdatingSequencerSelection(false)
	, bUpdatingExternalSelection(false)
	, bIsPreviewWidgetSelected(false)
{
	PreviewScene.GetWorld()->SetBegunPlay(false);

	// Register sequencer menu extenders.
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>( "Sequencer" );
	{
		int32 NewIndex = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().Add(
			FAssetEditorExtender::CreateRaw(this, &FWidgetBlueprintEditor::GetAddTrackSequencerExtender));
		SequencerAddTrackExtenderHandle = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates()[NewIndex].GetHandle();
	}
}

FWidgetBlueprintEditor::~FWidgetBlueprintEditor()
{
	UWidgetBlueprint* Blueprint = GetWidgetBlueprintObj();
	if ( Blueprint )
	{
		Blueprint->OnChanged().RemoveAll(this);
		Blueprint->OnCompiled().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
			Sequencer->OnMovieSceneBindingsPasted().RemoveAll(this);
			Sequencer->Close();
			Sequencer.Reset();
		}
	}

	// Un-Register sequencer menu extenders.
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().RemoveAll([this](const FAssetEditorExtender& Extender)
	{
		return SequencerAddTrackExtenderHandle == Extender.GetHandle();
	});

	if (Blueprint)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->GetDebugObjectDelegateForClass(Blueprint->GeneratedClass)->RemoveAll(this);
		}
	}
}

void FWidgetBlueprintEditor::InitWidgetBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode)
{
	bShowDashedOutlines = GetDefault<UWidgetDesignerSettings>()->bShowOutlines;
	bRespectLocks = GetDefault<UWidgetDesignerSettings>()->bRespectLocks;

	TSharedPtr<FWidgetBlueprintEditor> ThisPtr(SharedThis(this));

	PaletteViewModel = MakeShared<FPaletteViewModel>(ThisPtr);
	PaletteViewModel->RegisterToEvents();

	LibraryViewModel = MakeShared<FLibraryViewModel>(ThisPtr);
	LibraryViewModel->RegisterToEvents();

	WidgetToolbar = MakeShared<FWidgetBlueprintEditorToolbar>(ThisPtr);

	BindToolkitCommands();

	InitBlueprintEditor(Mode, InitToolkitHost, InBlueprints, bShouldOpenInDefaultsMode);

	// We only show compile tab results on error
	TSharedPtr<SDockTab> CompileResultsTab = GetToolkitHost()->GetTabManager()->FindExistingLiveTab(FBlueprintEditorTabs::CompilerResultsID);
	if (CompileResultsTab)
	{
		CompileResultsTab->RequestCloseTab();
	}

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FWidgetBlueprintEditor::OnObjectsReplaced);

	// for change selected widgets on sequencer tree view
	UWidgetBlueprint* Blueprint = GetWidgetBlueprintObj();

	UpdatePreview(GetWidgetBlueprintObj(), true);

	DesignerCommandList = MakeShared<FUICommandList>();

	DesignerCommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::DeleteSelectedWidgets),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanDeleteSelectedWidgets)
		);

	DesignerCommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CopySelectedWidgets),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanCopySelectedWidgets)
		);

	DesignerCommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CutSelectedWidgets),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanCutSelectedWidgets)
		);

	DesignerCommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::PasteWidgets),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanPasteWidgets)
		);

	DesignerCommandList->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::DuplicateSelectedWidgets),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanDuplicateSelectedWidgets)
		);

	DesignerCommandList->MapAction(FGraphEditorCommands::Get().FindReferences,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OnFindWidgetReferences, false, EGetFindReferenceSearchStringFlags::Legacy),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanFindWidgetReferences));
	
	DesignerCommandList->MapAction(FGraphEditorCommands::Get().FindReferencesByNameLocal,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OnFindWidgetReferences, false, EGetFindReferenceSearchStringFlags::None),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanFindWidgetReferences));

	DesignerCommandList->MapAction(FGraphEditorCommands::Get().FindReferencesByNameGlobal,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OnFindWidgetReferences, true, EGetFindReferenceSearchStringFlags::None),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanFindWidgetReferences));
	
	DesignerCommandList->MapAction(FGraphEditorCommands::Get().FindReferencesByClassMemberLocal,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OnFindWidgetReferences, false, EGetFindReferenceSearchStringFlags::UseSearchSyntax),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanFindWidgetReferences));
	
	DesignerCommandList->MapAction(FGraphEditorCommands::Get().FindReferencesByClassMemberGlobal,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OnFindWidgetReferences, true, EGetFindReferenceSearchStringFlags::UseSearchSyntax),
		FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanFindWidgetReferences));

	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShared<FWidgetEditorModeUILayer>(PinnedToolkitHost.Get());

	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		AssetEditorSubsystem->GetDebugObjectDelegateForClass(Blueprint->GeneratedClass)
			->AddSP(this, &FWidgetBlueprintEditor::OnDebugObjectAccess);
	}
}

void FWidgetBlueprintEditor::InitalizeExtenders()
{
	Super::InitalizeExtenders();

	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	AddMenuExtender(UMGEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	AddMenuExtender(CreateMenuExtender());

	TArrayView<IUMGEditorModule::FWidgetEditorToolbarExtender> ToolbarExtenderDelegates = UMGEditorModule.GetAllWidgetEditorToolbarExtenders();
	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if (ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
		}
	}
}

TSharedPtr<FExtender> FWidgetBlueprintEditor::CreateMenuExtender()
{
	TSharedPtr<FExtender> MenuExtender = MakeShareable(new FExtender);

	// Extend the File menu with asset actions
	MenuExtender->AddMenuExtension(
		"FileLoadAndSave",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateSP(this, &FWidgetBlueprintEditor::FillFileMenu));
	
	MenuExtender->AddMenuExtension(
		"AssetEditorActions",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateSP(this, &FWidgetBlueprintEditor::FillAssetMenu));

	MenuExtender->AddMenuExtension(
		"FileLoadAndSave",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateSP(this, &FWidgetBlueprintEditor::CustomizeWidgetCompileOptions));

	return MenuExtender;
}

void FWidgetBlueprintEditor::FillFileMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Import/Export"), LOCTEXT("Import/Export", "Import/Export"));
	MenuBuilder.AddMenuEntry(FUMGEditorCommands::Get().ExportAsPNG);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("WidgetBlueprint"), LOCTEXT("WidgetBlueprint", "Widget Blueprint"));
	MenuBuilder.AddMenuEntry(FUMGEditorCommands::Get().CreateNativeBaseClass);
	MenuBuilder.EndSection();
}

void FWidgetBlueprintEditor::FillAssetMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Thumbnail"), LOCTEXT("Thumbnail", "Thumbnail"));
	MenuBuilder.AddMenuEntry(FUMGEditorCommands::Get().SetImageAsThumbnail);
	MenuBuilder.AddMenuEntry(FUMGEditorCommands::Get().ClearCustomThumbnail);
	MenuBuilder.EndSection();
}

void FWidgetBlueprintEditor::CustomizeWidgetCompileOptions(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddSubMenu(
		LOCTEXT("CreateCompileTab", "Create Compile Tab"),
		LOCTEXT("CreateCompileTab_ToolTip", "Displays Compile tab when hidden based on compilation results."),
		FNewMenuDelegate::CreateStatic(&FWidgetBlueprintEditor::AddCreateCompileTabSubMenu));

	InMenuBuilder.AddSubMenu(
		LOCTEXT("DismissCompileTab", "Dismiss Compile Tab"),
		LOCTEXT("DismissCompileTab_ToolTip", "Dismisses compile tab based on compilation results."),
		FNewMenuDelegate::CreateStatic(&FWidgetBlueprintEditor::AddDismissCompileTabSubMenu));
}

void FWidgetBlueprintEditor::AddCreateCompileTabSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FUMGEditorCommands& Commands = FUMGEditorCommands::Get();
	InMenuBuilder.AddMenuEntry(Commands.CreateOnCompile_ErrorsAndWarnings);
	InMenuBuilder.AddMenuEntry(Commands.CreateOnCompile_Errors);
	InMenuBuilder.AddMenuEntry(Commands.CreateOnCompile_Warnings);
	InMenuBuilder.AddMenuEntry(Commands.CreateOnCompile_Never);
}

void FWidgetBlueprintEditor::AddDismissCompileTabSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FUMGEditorCommands& Commands = FUMGEditorCommands::Get();
	InMenuBuilder.AddMenuEntry(Commands.DismissOnCompile_ErrorsAndWarnings);
	InMenuBuilder.AddMenuEntry(Commands.DismissOnCompile_Errors);
	InMenuBuilder.AddMenuEntry(Commands.DismissOnCompile_Warnings);
	InMenuBuilder.AddMenuEntry(Commands.DismissOnCompile_Never);
}

void FWidgetBlueprintEditor::BindToolkitCommands()
{
	FUMGEditorCommands::Register();

	GetToolkitCommands()->MapAction(FUMGEditorCommands::Get().CreateNativeBaseClass,
		FUIAction(
			FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::OpenCreateNativeBaseClassDialog),
			FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CanCreateNativeBaseClass),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateSP(this, &FWidgetBlueprintEditor::IsCreateNativeBaseClassVisible)
		)
	);

	GetToolkitCommands()->MapAction(FUMGEditorCommands::Get().ExportAsPNG,
		FUIAction(
			FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::TakeSnapshot),
			FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::IsPreviewWidgetInitialized)
		)
	);

	GetToolkitCommands()->MapAction(FUMGEditorCommands::Get().SetImageAsThumbnail,
		FUIAction(
			FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::CaptureThumbnail)
		)
	);

	GetToolkitCommands()->MapAction(FUMGEditorCommands::Get().ClearCustomThumbnail,
		FUIAction(
			FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::ClearThumbnail),
			FCanExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::IsImageUsedAsThumbnail)
		)
	);

	GetToolkitCommands()->MapAction(FUMGEditorCommands::Get().OpenAnimDrawer,
		FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::ToggleAnimDrawer)
	);

	auto MapCreateOnCompileAction = [&](const TSharedPtr<FUICommandInfo>& InUICommand, EDisplayOnCompile InCreateOnCompile)
	{
		ToolkitCommands->MapAction(
			InUICommand,
			FExecuteAction::CreateStatic(&FWidgetBlueprintEditor::SetCreateOnCompileSetting, InCreateOnCompile),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FWidgetBlueprintEditor::IsCreateOnCompileSet, InCreateOnCompile)
		);
	};
	MapCreateOnCompileAction(FUMGEditorCommands::Get().CreateOnCompile_ErrorsAndWarnings, EDisplayOnCompile::DoC_ErrorsOrWarnings);
	MapCreateOnCompileAction(FUMGEditorCommands::Get().CreateOnCompile_Errors, EDisplayOnCompile::DoC_ErrorsOnly);
	MapCreateOnCompileAction(FUMGEditorCommands::Get().CreateOnCompile_Warnings, EDisplayOnCompile::DoC_WarningsOnly);
	MapCreateOnCompileAction(FUMGEditorCommands::Get().CreateOnCompile_Never, EDisplayOnCompile::DoC_Never);

	auto MapDismissOnCompileAction = [&](const TSharedPtr<FUICommandInfo>& InUICommand, EDisplayOnCompile InDismissOnCompile)
	{
		ToolkitCommands->MapAction(
			InUICommand,
			FExecuteAction::CreateStatic(&FWidgetBlueprintEditor::SetDismissOnCompileSetting, InDismissOnCompile),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FWidgetBlueprintEditor::IsDismissOnCompileSet, InDismissOnCompile)
		);
	};
	MapDismissOnCompileAction(FUMGEditorCommands::Get().DismissOnCompile_ErrorsAndWarnings, EDisplayOnCompile::DoC_ErrorsOrWarnings);
	MapDismissOnCompileAction(FUMGEditorCommands::Get().DismissOnCompile_Errors, EDisplayOnCompile::DoC_ErrorsOnly);
	MapDismissOnCompileAction(FUMGEditorCommands::Get().DismissOnCompile_Warnings, EDisplayOnCompile::DoC_WarningsOnly);
	MapDismissOnCompileAction(FUMGEditorCommands::Get().DismissOnCompile_Never, EDisplayOnCompile::DoC_Never);
}

void FWidgetBlueprintEditor::TakeSnapshot()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(ParentWindow);
		TArray<FString> SaveFilenames;
		const bool bOpened = DesktopPlatform->SaveFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ExportWidgetBlueprintDialogTitle", "Save Widget Blueprint Screenshot").ToString(),
			FPaths::GameAgnosticSavedDir(),
			TEXT(""),
			TEXT("PNG (*.png)|*.png"),
			EFileDialogFlags::None,
			SaveFilenames
		);
		if (SaveFilenames.Num() > 0)
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*SaveFilenames[0]));
			if (Ar)
			{
				TSharedPtr<SWidget> WindowContent;
				UUserWidget* PreviewWidget = GetPreview();

				UTextureRenderTarget2D* RenderTarget2D = NewObject<UTextureRenderTarget2D>();
				TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> ScaleAndOffset = FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTarget(PreviewWidget, RenderTarget2D);

				if (!ScaleAndOffset.IsSet())
				{
					FMessageLog("Blueprint").Warning(LOCTEXT("ExportWidgetBlueprint_ImageSourceFailedToCreate", "ExportWidgetBlueprint: Failed to create image source."));
					return;
				}

				FBufferArchive Buffer;
				bool bSuccess = FImageUtils::ExportRenderTarget2DAsPNG(RenderTarget2D, Buffer);
				if (bSuccess)
				{
					Ar->Serialize(const_cast<uint8*>(Buffer.GetData()), Buffer.Num());
				}
			}
		}
	}
}

void FWidgetBlueprintEditor::CaptureThumbnail()
{
	TSharedPtr<SWidget> WindowContent;
	UUserWidget* PreviewWidget = GetPreview();

	if (!PreviewWidget)
	{
		return;
	}

	UTextureRenderTarget2D* RenderTarget2D = NewObject<UTextureRenderTarget2D>();
	TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> ScaleAndOffset = FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetForThumbnail(PreviewWidget, RenderTarget2D, FVector2D(256.f, 256.f), TOptional<FVector2D>(), EThumbnailPreviewSizeMode::MatchDesignerMode);

	if (!ScaleAndOffset.IsSet())
	{
		return;
	}

	FImage Image;
	if ( !FImageUtils::GetRenderTargetImage(RenderTarget2D,Image) )
	{
		return;
	}

	UTexture2D* ThumbnailTexture = FImageUtils::CreateTexture2DFromImage(Image);
	FWidgetBlueprintEditorUtils::SetTextureAsAssetThumbnail(GetWidgetBlueprintObj(), ThumbnailTexture);
}

void FWidgetBlueprintEditor::ClearThumbnail() 
{
	GetWidgetBlueprintObj()->ThumbnailImage = nullptr;
}

bool FWidgetBlueprintEditor::IsImageUsedAsThumbnail()
{
	return GetWidgetBlueprintObj()->ThumbnailImage != nullptr;
}

bool FWidgetBlueprintEditor::IsPreviewWidgetInitialized()
{
	return GetPreview() != nullptr;
}

FName FWidgetBlueprintEditor::GetToolkitContextFName() const
{
	return GetToolkitFName();
}

FName FWidgetBlueprintEditor::GetToolkitFName() const
{
	return FName("WidgetBlueprintEditor");
}

FText FWidgetBlueprintEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Widget Editor");
}

FString FWidgetBlueprintEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Widget Editor ").ToString();
}

FLinearColor FWidgetBlueprintEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.25f, 0.35f, 0.5f);
}

void FWidgetBlueprintEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	Super::InitToolMenuContext(MenuContext);

	UWidgetBlueprintToolMenuContext* Context = NewObject<UWidgetBlueprintToolMenuContext>();
	Context->WidgetBlueprintEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FWidgetBlueprintEditor::SetCreateOnCompileSetting(EDisplayOnCompile InCreateOnCompile)
{
	UWidgetDesignerSettings* Settings = GetMutableDefault<UWidgetDesignerSettings>();
	Settings->CreateOnCompile = InCreateOnCompile;
	Settings->SaveConfig();
}

void FWidgetBlueprintEditor::SetDismissOnCompileSetting(EDisplayOnCompile InDismissOnCompile)
{
	UWidgetDesignerSettings* Settings = GetMutableDefault<UWidgetDesignerSettings>();
	Settings->DismissOnCompile = InDismissOnCompile;
	Settings->SaveConfig();
}

bool FWidgetBlueprintEditor::IsCreateOnCompileSet(EDisplayOnCompile InCreateOnCompile)
{
	const UWidgetDesignerSettings* Settings = GetDefault<UWidgetDesignerSettings>();
	return Settings->CreateOnCompile == InCreateOnCompile;
}

bool FWidgetBlueprintEditor::IsDismissOnCompileSet(EDisplayOnCompile InDismissOnCompile)
{
	const UWidgetDesignerSettings* Settings = GetDefault<UWidgetDesignerSettings>();
	return Settings->DismissOnCompile == InDismissOnCompile;
}

void FWidgetBlueprintEditor::OpenCreateNativeBaseClassDialog()
{
	FGameProjectGenerationModule::Get().OpenAddCodeToProjectDialog(
		FAddToProjectConfig()
		.DefaultClassPrefix(TEXT(""))
		.DefaultClassName(GetWidgetBlueprintObj()->GetName() + TEXT("Base"))
		.ParentClass(GetWidgetBlueprintObj()->ParentClass)
		.ParentWindow(FGlobalTabmanager::Get()->GetRootWindow())
		.OnAddedToProject(FOnAddedToProject::CreateSP(this, &FWidgetBlueprintEditor::OnCreateNativeBaseClassSuccessfully))
	);
}

void FWidgetBlueprintEditor::OnCreateNativeBaseClassSuccessfully(const FString& InClassName, const FString& InClassPath, const FString& InModuleName)
{
	UClass* NewNativeClass = FindObject<UClass>(FTopLevelAssetPath(*InClassPath, *InClassName));
	if (NewNativeClass)
	{
		ReparentBlueprint_NewParentChosen(NewNativeClass);
	}
}

void FWidgetBlueprintEditor::RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated/* = false*/)
{
	//Super::RegisterApplicationModes(InBlueprints, bShouldOpenInDefaultsMode);

	if (InBlueprints.Num() == 1)
	{
		TSharedPtr<FWidgetBlueprintEditor> ThisPtr(SharedThis(this));

		// Create the modes and activate one (which will populate with a real layout)
		TArray< TSharedRef<FApplicationMode> > TempModeList;
		TempModeList.Add(MakeShared<FWidgetDesignerApplicationMode>(ThisPtr));
		TempModeList.Add(MakeShared<FWidgetGraphApplicationMode>(ThisPtr));

		if (FWidgetBlueprintApplicationModes::IsPreviewModeEnabled())
		{
			TempModeList.Add(MakeShared<UE::UMG::Editor::FWidgetPreviewApplicationMode>(ThisPtr));
			PreviewMode = MakeShared<UE::UMG::Editor::FPreviewMode>();
		}

		for (TSharedRef<FApplicationMode>& AppMode : TempModeList)
		{
			AddApplicationMode(AppMode->GetModeName(), AppMode);
		}

		SetCurrentMode(FWidgetBlueprintApplicationModes::DesignerMode);
	}
	else
	{
		//// We either have no blueprints or many, open in the defaults mode for multi-editing
		//AddApplicationMode(
		//	FBlueprintEditorApplicationModes::BlueprintDefaultsMode,
		//	MakeShareable(new FBlueprintDefaultsApplicationMode(SharedThis(this))));
		//SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
	}
}

void FWidgetBlueprintEditor::SelectWidgets(const TSet<FWidgetReference>& Widgets, bool bAppendOrToggle)
{
	if (!bAppendOrToggle && SelectedWidgets.Array() == Widgets.Array())
	{
		return;
	}

	TSet<FWidgetReference> TempSelection;
	for ( const FWidgetReference& Widget : Widgets )
	{
		if ( Widget.IsValid() )
		{
			TempSelection.Add(Widget);
		}
	}

	OnSelectedWidgetsChanging.Broadcast();

	// Finally change the selected widgets after we've updated the details panel 
	// to ensure values that are pending are committed on focus loss, and migrated properly
	// to the old selected widgets.
	if ( !bAppendOrToggle )
	{
		SelectedWidgets.Empty();
	}
	SelectedObjects.Empty();
	SelectedNamedSlot.Reset();
	bIsPreviewWidgetSelected = false;

	for ( const FWidgetReference& Widget : TempSelection )
	{
		if ( bAppendOrToggle && SelectedWidgets.Contains(Widget) )
		{
			SelectedWidgets.Remove(Widget);
		}
		else
		{
			SelectedWidgets.Add(Widget);
		}
	}

	OnSelectedWidgetsChanged.Broadcast();
}

void FWidgetBlueprintEditor::SelectObjects(const TSet<UObject*>& Objects)
{
	OnSelectedWidgetsChanging.Broadcast();

	SelectedWidgets.Empty();
	SelectedObjects.Empty();
	SelectedNamedSlot.Reset();
	bIsPreviewWidgetSelected = false;

	for ( UObject* Obj : Objects )
	{
		SelectedObjects.Add(Obj);
		if (Obj == GetPreview())
		{
			bIsPreviewWidgetSelected = true;
		}
	}

	OnSelectedWidgetsChanged.Broadcast();
}

bool FWidgetBlueprintEditor::IsBindingSelected(const FMovieSceneBinding& InBinding)
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	if (Widgets.Num() == 0)
	{
		return true;
	}

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UMovieSceneSequence* AnimationSequence = ActiveSequencer->GetFocusedMovieSceneSequence();
	UObject* BindingContext = GetAnimationPlaybackContext();
	TArray<UObject*, TInlineAllocator<1>> BoundObjects;
	AnimationSequence->LocateBoundObjects(InBinding.GetObjectGuid(), UE::UniversalObjectLocator::FResolveParams(BindingContext), ActiveSequencer->GetSharedPlaybackState(), BoundObjects);

	if (BoundObjects.Num() == 0)
	{
		return false;
	}
	else if (Cast<UPanelSlot>(BoundObjects[0]))
	{
		return Widgets.Contains(GetReferenceFromPreview(Cast<UPanelSlot>(BoundObjects[0])->Content));
	}
	else
	{
		return Widgets.Contains(GetReferenceFromPreview(Cast<UWidget>(BoundObjects[0])));
	}
}

void FWidgetBlueprintEditor::SetSelectedNamedSlot(TOptional<FNamedSlotSelection> InSelectedNamedSlot)
{
	OnSelectedWidgetsChanging.Broadcast();

	SelectedWidgets.Empty();
	SelectedObjects.Empty();
	SelectedNamedSlot.Reset();
	bIsPreviewWidgetSelected = false;

	SelectedNamedSlot = InSelectedNamedSlot;
	if (InSelectedNamedSlot.IsSet())
	{
		if (InSelectedNamedSlot->NamedSlotHostWidget.IsValid())
		{
			SelectedWidgets.Add(InSelectedNamedSlot->NamedSlotHostWidget);
		}
	}

	OnSelectedWidgetsChanged.Broadcast();
}

void FWidgetBlueprintEditor::CleanSelection()
{
	TSet<FWidgetReference> TempSelection;

	TArray<UWidget*> WidgetsInTree;
	GetWidgetBlueprintObj()->WidgetTree->GetAllWidgets(WidgetsInTree);
	TSet<UWidget*> TreeWidgetSet(WidgetsInTree);

	for ( FWidgetReference& WidgetRef : SelectedWidgets )
	{
		if ( WidgetRef.IsValid() )
		{
			if ( TreeWidgetSet.Contains(WidgetRef.GetTemplate()) )
			{
				TempSelection.Add(WidgetRef);
			}
		}
	}

	if ( TempSelection.Num() != SelectedWidgets.Num() )
	{
		SelectWidgets(TempSelection, false);
	}
	else if ( bIsPreviewWidgetSelected )
	{
		TSet<UObject*> ValidObjects = { GetPreview() };
		for ( const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects )
		{
			if ( SelectedObject.IsValid() )
			{
				ValidObjects.Add(SelectedObject.Get());
			}
		}

		SelectObjects(ValidObjects);
	}
}

const TSet<FWidgetReference>& FWidgetBlueprintEditor::GetSelectedWidgets() const
{
	return SelectedWidgets;
}

const TSet< TWeakObjectPtr<UObject> >& FWidgetBlueprintEditor::GetSelectedObjects() const
{
	return SelectedObjects;
}

TOptional<FNamedSlotSelection> FWidgetBlueprintEditor::GetSelectedNamedSlot() const
{
	return SelectedNamedSlot;
}

void FWidgetBlueprintEditor::InvalidatePreview(bool bViewOnly)
{
	if ( bViewOnly )
	{
		OnWidgetPreviewUpdated.Broadcast();
	}
	else
	{
		bPreviewInvalidated = true;
	}
}

void FWidgetBlueprintEditor::OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled )
{
	DestroyPreview();

	Super::OnBlueprintChangedImpl(InBlueprint, bIsJustBeingCompiled);

	if ( InBlueprint )
	{
		RefreshPreview();
	}
}

void FWidgetBlueprintEditor::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	// Remove dead references and update references
	for ( int32 HandleIndex = WidgetHandlePool.Num() - 1; HandleIndex >= 0; HandleIndex-- )
	{
		TSharedPtr<FWidgetHandle> Ref = WidgetHandlePool[HandleIndex].Pin();

		if ( Ref.IsValid() )
		{
			UObject* const* NewObject = ReplacementMap.Find(Ref->Widget.Get());
			if ( NewObject )
			{
				Ref->Widget = Cast<UWidget>(*NewObject);
			}
		}
		else
		{
			WidgetHandlePool.RemoveAtSwap(HandleIndex);
		}
	}
}

bool FWidgetBlueprintEditor::CanDeleteSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	return Widgets.Num() > 0 && !FWidgetBlueprintEditorUtils::IsAnySelectedWidgetLocked(Widgets);
}

void FWidgetBlueprintEditor::DeleteSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	FWidgetBlueprintEditorUtils::DeleteWidgets(GetWidgetBlueprintObj(), FWidgetBlueprintEditorUtils::ResolveWidgetTemplates(Widgets), FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::WarnAndAskUser);

	// Clear the selection now that the widget has been deleted.
	TSet<FWidgetReference> Empty;
	SelectWidgets(Empty, false);
}

bool FWidgetBlueprintEditor::CanCopySelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	return Widgets.Num() > 0;
}

void FWidgetBlueprintEditor::CopySelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	FWidgetBlueprintEditorUtils::CopyWidgets(GetWidgetBlueprintObj(), Widgets);
}

bool FWidgetBlueprintEditor::CanCutSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	return Widgets.Num() > 0 && !FWidgetBlueprintEditorUtils::IsAnySelectedWidgetLocked(Widgets);
}

void FWidgetBlueprintEditor::CutSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	FWidgetBlueprintEditorUtils::CutWidgets(SharedThis(this), GetWidgetBlueprintObj(), Widgets);
}

const UWidgetAnimation* FWidgetBlueprintEditor::RefreshCurrentAnimation()
{
	return CurrentAnimation.Get();
}

bool FWidgetBlueprintEditor::CanPasteWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();

	if (FWidgetBlueprintEditorUtils::IsAnySelectedWidgetLocked(Widgets))
	{
		return false;
	}

	if (!FWidgetBlueprintEditorUtils::DoesClipboardTextContainWidget(GetWidgetBlueprintObj()))
	{
		return false;
	}

	if (!FWidgetBlueprintEditorUtils::CanPasteWidgetsExtension(Widgets))
	{
		return false;
	}

	if ( Widgets.Num() == 1 )
	{
		// Always return true here now since we want to support pasting widgets as siblings
		return true;
	}
	else if ( GetWidgetBlueprintObj()->WidgetTree->RootWidget == nullptr )
	{
		return true;
	}
	else
	{
		TOptional<FNamedSlotSelection> NamedSlotSelection = GetSelectedNamedSlot();
		if ( NamedSlotSelection.IsSet() )
		{
			INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(NamedSlotSelection->NamedSlotHostWidget.GetTemplate());
			if ( NamedSlotHost == nullptr )
			{
				return false;
			}
			else if ( NamedSlotHost->GetContentForSlot(NamedSlotSelection->SlotName) != nullptr )
			{
				return false;
			}

			return true;
		}
	}

	return false;
}

void FWidgetBlueprintEditor::PasteWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	FWidgetReference Target = Widgets.Num() > 0 ? *Widgets.CreateIterator() : FWidgetReference();
	FName SlotName = NAME_None;

	TOptional<FNamedSlotSelection> NamedSlotSelection = GetSelectedNamedSlot();
	if ( NamedSlotSelection.IsSet() )
	{
		Target = NamedSlotSelection->NamedSlotHostWidget;
		SlotName = NamedSlotSelection->SlotName;
	}

	TArray<UWidget*> PastedWidgets = FWidgetBlueprintEditorUtils::PasteWidgets(SharedThis(this), GetWidgetBlueprintObj(), Target, SlotName, PasteDropLocation);

	PasteDropLocation = PasteDropLocation + FVector2D(25, 25);

	TSet<FWidgetReference> PastedWidgetRefs;
	for (UWidget* Widget : PastedWidgets)
	{
		PastedWidgetRefs.Add(GetReferenceFromPreview(Widget));
	}
	SelectWidgets(PastedWidgetRefs, false);
}

bool FWidgetBlueprintEditor::CanDuplicateSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	if (Widgets.Num() == 1)
	{
		FWidgetReference Target = *Widgets.CreateIterator();
		UPanelWidget* ParentWidget = Target.GetTemplate()->GetParent();
		return ParentWidget && ParentWidget->CanAddMoreChildren();
	}
	return false;
}

void FWidgetBlueprintEditor::DuplicateSelectedWidgets()
{
	TSet<FWidgetReference> Widgets = GetSelectedWidgets();
	TArray<UWidget*> DuplicatedWidgets = FWidgetBlueprintEditorUtils::DuplicateWidgets(SharedThis(this), GetWidgetBlueprintObj(), Widgets);

	TSet<FWidgetReference> DuplicatedWidgetRefs;
	for (UWidget* Widget : DuplicatedWidgets)
	{
		DuplicatedWidgetRefs.Add(GetReferenceFromPreview(Widget));
	}
	SelectWidgets(DuplicatedWidgetRefs, false);
}

void FWidgetBlueprintEditor::OnFindWidgetReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags)
{
	FWidgetReference WidgetReference = *GetSelectedWidgets().CreateConstIterator();
	const FString VariableName = WidgetReference.GetTemplate()->GetName();

	FMemberReference MemberReference;
	MemberReference.SetSelfMember(*VariableName);
	const FString SearchTerm = EnumHasAnyFlags(Flags, EGetFindReferenceSearchStringFlags::UseSearchSyntax) ? MemberReference.GetReferenceSearchString(GetBlueprintObj()->SkeletonGeneratedClass) : FString::Printf(TEXT("\"%s\""), *VariableName);

	SetCurrentMode(FWidgetBlueprintApplicationModes::GraphMode);
	
	const bool bSetFindWithinBlueprint = !bSearchAllBlueprints;
	SummonSearchUI(bSetFindWithinBlueprint, SearchTerm);
}

bool FWidgetBlueprintEditor::CanFindWidgetReferences() const
{
	return GetSelectedWidgets().Num() == 1 && GetSelectedWidgets().CreateConstIterator()->GetTemplate()->bIsVariable;
}

bool FWidgetBlueprintEditor::CanCreateNativeBaseClass() const
{
	return ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed() && IsParentClassNative();
}


bool FWidgetBlueprintEditor::IsCreateNativeBaseClassVisible() const
{
	return ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed();
}

void FWidgetBlueprintEditor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	// Tick the preview scene world.
	// Allow full tick only if preview simulation is enabled and we're not currently in an active SIE or PIE session
	if (bIsSimulateEnabled && GEditor->PlayWorld == nullptr && !GEditor->bIsSimulatingInEditor)
	{
		PreviewScene.GetWorld()->Tick(bIsRealTime ? LEVELTICK_All : LEVELTICK_TimeOnly, DeltaTime);
	}
	else
	{
		PreviewScene.GetWorld()->Tick(bIsRealTime ? LEVELTICK_ViewportsOnly : LEVELTICK_TimeOnly, DeltaTime);
	}

	// Whenever animations change the generated class animations need to be updated since they are copied on compile.  This
	// update is deferred to tick since some edit operations (e.g. drag/drop) cause large numbers of changes to the data.
	if ( bRefreshGeneratedClassAnimations )
	{
		TArray<TObjectPtr<UWidgetAnimation>>& PreviewAnimations = Cast<UWidgetBlueprintGeneratedClass>( PreviewBlueprint->GeneratedClass )->Animations;
		PreviewAnimations.Empty();
		for ( UWidgetAnimation* WidgetAnimation : PreviewBlueprint->Animations )
		{
			PreviewAnimations.Add( DuplicateObject<UWidgetAnimation>( WidgetAnimation, PreviewBlueprint->GeneratedClass ) );
		}
		bRefreshGeneratedClassAnimations = false;
	}

	// Note: The weak ptr can become stale if the actor is reinstanced due to a Blueprint change, etc. In that case we 
	//       look to see if we can find the new instance in the preview world and then update the weak ptr.
	if ( PreviewWidgetPtr.IsStale(true) || bPreviewInvalidated )
	{
		bPreviewInvalidated = false;
		RefreshPreview();
	}

	// Update the palette view model.
	if (PaletteViewModel->NeedUpdate())
	{
		PaletteViewModel->Update();
	}

	if (LibraryViewModel->NeedUpdate())
	{
		LibraryViewModel->Update();
	}
}

static bool MigratePropertyValue(UObject* SourceObject, UObject* DestinationObject, FEditPropertyChain::TDoubleLinkedListNode* PropertyChainNode, FProperty* MemberProperty, bool bIsModify)
{
	FProperty* CurrentProperty = PropertyChainNode->GetValue();
	FEditPropertyChain::TDoubleLinkedListNode* NextNode = PropertyChainNode->GetNextNode();

	if ( !ensure(SourceObject && DestinationObject) )
	{
		return false;
	}

	ensure(SourceObject->GetClass() == DestinationObject->GetClass());

	// If the current property is an array, map or set, short-circuit current progress so that we copy the whole container.
	if ( CastField<FArrayProperty>(CurrentProperty) || CastField<FMapProperty>(CurrentProperty) || CastField<FSetProperty>(CurrentProperty) || CastField<FStructProperty>(CurrentProperty))
	{
		NextNode = nullptr;
	}

	if ( FObjectProperty* CurrentObjectProperty = CastField<FObjectProperty>(CurrentProperty) )
	{
		UObject* NewSourceObject = CurrentObjectProperty->GetObjectPropertyValue_InContainer(SourceObject);
		UObject* NewDestionationObject = CurrentObjectProperty->GetObjectPropertyValue_InContainer(DestinationObject);

		if ( NewSourceObject == nullptr || NewDestionationObject == nullptr )
		{
			NextNode = nullptr;
		}
	}
	
	if ( NextNode == nullptr )
	{
		if (bIsModify)
		{
			if (DestinationObject)
			{
				DestinationObject->SetFlags(RF_Transactional);
				DestinationObject->Modify();
			}
			return true;
		}
		else
		{
			// Check to see if there's an edit condition property we also need to migrate.
			bool bDummyNegate = false;
			FBoolProperty* EditConditionProperty = PropertyCustomizationHelpers::GetEditConditionProperty(MemberProperty, bDummyNegate);
			if ( EditConditionProperty != nullptr )
			{
				FObjectEditorUtils::MigratePropertyValue(SourceObject, EditConditionProperty, DestinationObject, EditConditionProperty);
			}

			return FObjectEditorUtils::MigratePropertyValue(SourceObject, MemberProperty, DestinationObject, MemberProperty);
		}
	}

	if ( FObjectProperty* CurrentObjectProperty = CastField<FObjectProperty>(CurrentProperty) )
	{
		UObject* NewSourceObject = CurrentObjectProperty->GetObjectPropertyValue_InContainer(SourceObject);
		UObject* NewDestionationObject = CurrentObjectProperty->GetObjectPropertyValue_InContainer(DestinationObject);

		return MigratePropertyValue(NewSourceObject, NewDestionationObject, NextNode, NextNode->GetValue(), bIsModify);
	}

	// ExportText/ImportText works on all property types
	return MigratePropertyValue(SourceObject, DestinationObject, NextNode, MemberProperty, bIsModify);
}

void FWidgetBlueprintEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Super::AddReferencedObjects( Collector );
	Collector.AddReferencedObject(PreviewWidgetPtr);
}

static void PropagateDefaultPropertyChange(const FPropertyChangedEvent& PropertyChangedEvent, const FEditPropertyChain& PropertyThatChanged, UObject* UnmodifiedCDOObject, UObject* ModifedProxyObject)
{
	if (!UnmodifiedCDOObject || !ModifedProxyObject || !UnmodifiedCDOObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	// This is a kit bashed version of what happens in FPropertyNode::PropagatePropertyChange to migrate the CDO edits to the live instances.
	// We don't have PropertyHandles outside of the details view, and so we need to do the equivalent work with property paths.
	FCachedPropertyPath ChangedPropertyPath(PropertyChangedEvent, PropertyThatChanged);

	FCachedPropertyPath ResolvedCDOPath = FCachedPropertyPath::MakeUnresolvedCopy(ChangedPropertyPath);
	ResolvedCDOPath.Resolve(UnmodifiedCDOObject);

	if (ResolvedCDOPath.IsFullyResolved())
	{
		FProperty* Prop = ResolvedCDOPath.GetFProperty();
		FProperty* ParentProp = nullptr;
		if (ResolvedCDOPath.GetNumSegments() > 1)
		{
			ParentProp = CastField<FProperty>(ResolvedCDOPath.GetSegment(ResolvedCDOPath.GetNumSegments() - 2).GetField().ToField());
		}

		FArrayProperty* ParentArrayProp = CastField<FArrayProperty>(ParentProp);
		FMapProperty* ParentMapProp = CastField<FMapProperty>(ParentProp);
		FSetProperty* ParentSetProp = CastField<FSetProperty>(ParentProp);

		if (ParentArrayProp && ParentArrayProp->Inner != Prop)
		{
			ParentArrayProp = nullptr;
		}

		if (ParentMapProp && ParentMapProp->KeyProp != Prop && ParentMapProp->ValueProp != Prop)
		{
			ParentMapProp = nullptr;
		}

		if (ParentSetProp && ParentSetProp->ElementProp != Prop)
		{
			ParentSetProp = nullptr;
		}

		FCachedPropertyPath ResolvedComparisonPath = ResolvedCDOPath;
		if (ParentArrayProp || ParentMapProp || ParentSetProp)
		{
			ResolvedComparisonPath = ResolvedComparisonPath.MakeParentPath();
		}

		TArray<UObject*> ArchetypeInstances;
		UnmodifiedCDOObject->GetArchetypeInstances(ArchetypeInstances);

		void* CDOPristineValueAddr = ResolvedComparisonPath.GetCachedAddress();

		FString OldValue;
		FString NewValue;

		bool bHasCopyableValue = true;
		bHasCopyableValue &= PropertyPathHelpers::GetPropertyValueAsString(UnmodifiedCDOObject, ChangedPropertyPath, OldValue);
		bHasCopyableValue &= PropertyPathHelpers::GetPropertyValueAsString(ModifedProxyObject, ChangedPropertyPath, NewValue);

		if (bHasCopyableValue)
		{
			FCachedPropertyPath Instance_ComparisonPath = FCachedPropertyPath::MakeUnresolvedCopy(ResolvedComparisonPath);
			for (UObject* ArchetypeInstance : ArchetypeInstances)
			{
				if (ArchetypeInstance != ModifedProxyObject)
				{
					Instance_ComparisonPath.Resolve(ArchetypeInstance);

					if (Instance_ComparisonPath.IsFullyResolved())
					{
						void* Instance_DestValueAddr = Instance_ComparisonPath.GetCachedAddress();
						const bool bShouldImport = ResolvedComparisonPath.GetFProperty()->Identical(CDOPristineValueAddr, Instance_DestValueAddr, PPF_DeepComparison);

						if (bShouldImport)
						{
							// Note: We use the original change path to apply the actual edit - the comparison path might be shortened due to checking
							// maps or arrays or sets, but if the array|map|set matches as a whole, then we can mod a specific index.
							PropertyPathHelpers::SetPropertyValueFromString(ArchetypeInstance, ChangedPropertyPath, NewValue, PPF_InstanceSubobjects);
						}
					}
				}
			}
		}
	}
}

void FWidgetBlueprintEditor::MigrateFromChain(const FPropertyChangedEvent* PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged, bool bIsModify)
{
	UWidgetBlueprint* Blueprint = GetWidgetBlueprintObj();

	if (const UUserWidget* PreviewUserWidget = GetPreview())
	{
		FEditPropertyChain::TDoubleLinkedListNode* PropertyChainNode = PropertyThatChanged->GetHead();
		check(PropertyChainNode);

		const UClass* OwnerClass = PropertyChainNode->GetValue()->GetOwnerClass();
		if (!OwnerClass)
		{
			return;
		}

		// The root widget is only ever in the SelectedObjects array, the root widget is a copy of the CDO instanced for preview purposes.
		// If it's modified, we need to copy the values back to the CDO - but we ALSO need to do what the details panel
		// normally does when modifying a CDO, which is to copy the values to any live instances of the widget with matching
		// previous values so that defaults stay in-sync with the CDO.
		for (TWeakObjectPtr<UObject> SelectedObjectRef : SelectedObjects)
		{
			UObject* SelectedObjectPtr = SelectedObjectRef.Get();
			ensure(SelectedObjectPtr);
			if (!SelectedObjectPtr)
			{
				continue;
			}

			// Ignore selected objects that aren't the root widget
			if (!PreviewUserWidget->IsA(SelectedObjectPtr->GetClass()))
			{
				continue;
			}

			// dealing with root widget here
			if (UObject* WidgetCDO = SelectedObjectPtr->GetClass()->GetDefaultObject(true))
			{
				// Only do the migration if the property on the head of the linked list lives in the CDO
				// We want to skip the migration if the property isn't leading to this CDO, such as when we call 
				// FDetailCategoryImpl::AddExternalObjects to inject external objects in the details panel of the CDO.
				if (SelectedObjectPtr->IsA(OwnerClass) && WidgetCDO->IsA(OwnerClass) && PropertyThatChanged)
				{
					if (PropertyChangedEvent == nullptr)
					{
						WidgetCDO->SetEditChangePropagationFlags(EEditChangePropagationFlags::OnlyMarkRealignedInstancesAsDirty);
						WidgetCDO->PreEditChange(*PropertyThatChanged);
					}
					else
					{
						// We have to do this before we call MigratePropertyValue, because that will change the CDO, and we won't be
						// able to cheat like we do still having a pristine CDO copy. The details panel has to do shenanigans where it copies
						// the data into a temp property piece of memory, but we don't need to do that since we've got the actual unmodified
						// CDO still - but we're going to propagate the proxy's value that users modified instead of the CDO's value,
						// then MigratePropertyValue will update the CDO's value.
						PropagateDefaultPropertyChange(*PropertyChangedEvent, *PropertyThatChanged, WidgetCDO, SelectedObjectRef.Get());
					}
				
					MigratePropertyValue(SelectedObjectRef.Get(), WidgetCDO, PropertyChainNode, PropertyChainNode->GetValue(), bIsModify);

					if (PropertyChangedEvent)
					{
						TArray<UObject*> CDOArray = { WidgetCDO };
						FPropertyChangedEvent ChangeEvent(PropertyChangedEvent->Property, PropertyChangedEvent->ChangeType, CDOArray);
						ChangeEvent.SetActiveMemberProperty(PropertyChangedEvent->MemberProperty);

						TArray<TMap<FString, int32>> ArrayIndexForProperty;
						for (TDoubleLinkedList<FProperty*>::TIterator It(PropertyThatChanged->GetHead()); It; ++It)
						{
							FProperty* Property = *It;

							const FString PropertyName = Property->GetName();
							const int32 ArrayIndex = PropertyChangedEvent->GetArrayIndex(PropertyName);
							if (ArrayIndex != INDEX_NONE)
							{
								ArrayIndexForProperty.AddDefaulted();
								ArrayIndexForProperty.Last().Add(PropertyName, ArrayIndex);
							}
						}
						ChangeEvent.ObjectIteratorIndex = 0;
						ChangeEvent.SetArrayIndexPerObject(ArrayIndexForProperty);

						FPropertyChangedChainEvent ChainEvent(*PropertyThatChanged, ChangeEvent);
						WidgetCDO->PostEditChangeChainProperty(ChainEvent);
						WidgetCDO->SetEditChangePropagationFlags(EEditChangePropagationFlags::None);
					}
				}
			}
		}

		// Here we handle migrating values on Widgets in the WidgetTree. So we need to migrate the property to the
		// WidgetBlueprint WidgetTree and instead of the Preview Widget
		for ( FWidgetReference& WidgetRef : SelectedWidgets )
		{
			if ( UWidget* PreviewWidget = WidgetRef.GetPreview())
			{
				FName PreviewWidgetName = PreviewWidget->GetFName();
				if (UWidget* TemplateWidget = Blueprint->WidgetTree->FindWidget(PreviewWidgetName))
				{
					ensure(TemplateWidget == WidgetRef.GetTemplate());
					
					// Only do the migration if the property on the head of the linked list lives in this UWidget
					// We want to skip the migration if the property isn't leading to this UWidget, such as when we call 
					// FDetailCategoryImpl::AddExternalObjects to inject external objects in the details panel of this UWidget.
					if (PreviewWidget->IsA(OwnerClass) && TemplateWidget->IsA(OwnerClass))
					{
						MigratePropertyValue(PreviewWidget, TemplateWidget, PropertyChainNode, PropertyChainNode->GetValue(), bIsModify);
					}
					// If the Object being modified is a UUIComponent, we want to migrate to the Extension
					else if(OwnerClass->IsChildOf<UUIComponent>())
					{
						// We migrate the value to the Widget Blueprint Extension component
						UUIComponentUserWidgetExtension* ComponentExtension = PreviewUserWidget->GetExtension<UUIComponentUserWidgetExtension>();
						UUIComponentWidgetBlueprintExtension* ComponentWidgetBlueprintExtension = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(Blueprint);
						if (ComponentExtension && ComponentWidgetBlueprintExtension)
						{						
							UUIComponent* Component = ComponentExtension->GetComponent(OwnerClass, TemplateWidget->GetFName());
							UUIComponent* WPComponent = ComponentWidgetBlueprintExtension->GetComponent(OwnerClass, TemplateWidget->GetFName());							

							if(WPComponent && Component)
							{
								MigratePropertyValue(Component, WPComponent, PropertyChainNode, PropertyChainNode->GetValue(), bIsModify);
							}
						}
					}
				}
			}
		}
	}
}

void FWidgetBlueprintEditor::PostUndo(bool bSuccessful)
{
	Super::PostUndo(bSuccessful);
	InvalidatePreview();

	OnWidgetBlueprintTransaction.Broadcast();
}

void FWidgetBlueprintEditor::PostRedo(bool bSuccessful)
{
	Super::PostRedo(bSuccessful);
	InvalidatePreview();

	OnWidgetBlueprintTransaction.Broadcast();
}

TSharedRef<SWidget> FWidgetBlueprintEditor::CreateSequencerTabWidget()
{
	TSharedRef<SOverlay> SequencerOverlayRef =
		SNew(SOverlay)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Sequencer")));
	TabSequencerOverlay = SequencerOverlayRef;

	TSharedPtr<STextBlock> NoAnimationTextBlockPtr;
	if (!NoAnimationTextBlockTab.IsValid())
	{
		NoAnimationTextBlockPtr =
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "UMGEditor.NoAnimationFont")
			.Text(LOCTEXT("NoAnimationSelected", "No Animation Selected"));
		NoAnimationTextBlockTab = NoAnimationTextBlockPtr;
	}

	SequencerOverlayRef->AddSlot(0)
	[
		GetTabSequencer()->GetSequencerWidget()
	];

	SequencerOverlayRef->AddSlot(1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
	[
		NoAnimationTextBlockTab.Pin().ToSharedRef()
	];

	return SequencerOverlayRef;
}

TSharedRef<SWidget> FWidgetBlueprintEditor::CreateSequencerDrawerWidget()
{
	TSharedRef<SOverlay> SequencerOverlayRef =
		SNew(SOverlay)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Sequencer")));
	DrawerSequencerOverlay = SequencerOverlayRef;

	TSharedPtr<STextBlock> NoAnimationTextBlockPtr;
	if (!NoAnimationTextBlockDrawer.IsValid())
	{
		NoAnimationTextBlockPtr =
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "UMGEditor.NoAnimationFont")
			.Text(LOCTEXT("NoAnimationSelected", "No Animation Selected"));
		NoAnimationTextBlockDrawer = NoAnimationTextBlockPtr;
	}

	SequencerOverlayRef->AddSlot(0)
	[
		GetDrawerSequencer()->GetSequencerWidget()
	];

	SequencerOverlayRef->AddSlot(1)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
	[
		NoAnimationTextBlockDrawer.Pin().ToSharedRef()
	];

	return SequencerOverlayRef;
}

TSharedRef<SWidget> FWidgetBlueprintEditor::OnGetWidgetAnimSequencer()
{
	if (!AnimDrawerWidget.IsValid())
	{
		FAnimationTabSummoner AnimDrawerSummoner(SharedThis(this), true);
		FWorkflowTabSpawnInfo SpawnInfo;
		AnimDrawerWidget = AnimDrawerSummoner.CreateTabBody(SpawnInfo);
	}

	return AnimDrawerWidget.ToSharedRef();
}

void FWidgetBlueprintEditor::AddExternalEditorWidget(FName ID, TSharedRef<SWidget> InExternalWidget)
{
	if (!ExternalEditorWidgets.Contains(ID))
	{
		ExternalEditorWidgets.Add(ID, InExternalWidget);
	}
}

int32 FWidgetBlueprintEditor::RemoveExternalEditorWidget(FName ID)
{
	return ExternalEditorWidgets.Remove(ID);
}

TSharedPtr<SWidget> FWidgetBlueprintEditor::GetExternalEditorWidget(FName ID)
{
	TSharedPtr<SWidget>* ExternalWidget = ExternalEditorWidgets.Find(ID);

	if (ExternalWidget)
	{
		return *ExternalWidget;
	}

	return nullptr;
}

void FWidgetBlueprintEditor::ToggleAnimDrawer()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->TryToggleDrawer(FAnimationTabSummoner::WidgetAnimSequencerDrawerID);
}

void FWidgetBlueprintEditor::NotifyWidgetAnimListChanged()
{
	OnWidgetAnimationsUpdated.Broadcast();
	
	// Check if any animations viewed are invalid, if so select null animation
	// This can happen when a secondardary sequencer deletes our animatio
	for (TWeakPtr<ISequencer>& SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(Sequencer->GetFocusedMovieSceneSequence());
			if (!GetWidgetBlueprintObj()->Animations.Contains(WidgetAnimation))
			{
				Sequencer->ResetToNewRootSequence(*UWidgetAnimation::GetNullAnimation());
				Sequencer->GetSequencerWidget()->SetEnabled(false);
				Sequencer->SetAutoChangeMode(EAutoChangeMode::None);
			}
		}
	}
}


void FWidgetBlueprintEditor::OnWidgetAnimSequencerOpened(FName StatusBarWithDrawerName)
{
	OnWidgetAnimDrawerSequencerOpened(StatusBarWithDrawerName);
}

void FWidgetBlueprintEditor::OnWidgetAnimSequencerDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	OnWidgetAnimDrawerSequencerDismissed(NewlyFocusedWidget);
}

void FWidgetBlueprintEditor::OnWidgetAnimDrawerSequencerOpened(FName StatusBarWithDrawerName)
{
	bIsSequencerDrawerOpen = true;

	if (TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer())
	{
		UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
		if (WidgetAnimation)
		{
			ChangeViewedAnimation(*WidgetAnimation);
		}
	}

	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			Sequencer->RefreshTree();
		}
	}

	if (DrawerSequencer)
	{
		FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), DrawerSequencer->GetSequencerWidget());
	}
}

void FWidgetBlueprintEditor::OnWidgetAnimDrawerSequencerDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	if (TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer())
	{
		UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
		if (WidgetAnimation)
		{
			ChangeViewedAnimation(*WidgetAnimation);
		}
		ActiveSequencer->GetSequencerWidget()->SetEnabled(false);
		ActiveSequencer->SetAutoChangeMode(EAutoChangeMode::None);
	}
	bIsSequencerDrawerOpen = false;

	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			Sequencer->RefreshTree();
		}
	}

	SetKeyboardFocus();
}

void FWidgetBlueprintEditor::OnWidgetAnimTabSequencerClosed(TSharedRef<SDockTab> ClosedTab)
{
	// Deselected any animation when closing the tab 
	ChangeViewedAnimation(*UWidgetAnimation::GetNullAnimation());
	if (TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer())
	{
		ActiveSequencer->GetSequencerWidget()->SetEnabled(false);
		ActiveSequencer->SetAutoChangeMode(EAutoChangeMode::None);
	}
}

void FWidgetBlueprintEditor::OnWidgetAnimTabSequencerOpened()
{
	if (TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer())
	{
		if (UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence()))
		{
			ChangeViewedAnimation(*WidgetAnimation);
		}
	}
}

UWidgetBlueprint* FWidgetBlueprintEditor::GetWidgetBlueprintObj() const
{
	return Cast<UWidgetBlueprint>(GetBlueprintObj());
}

UUserWidget* FWidgetBlueprintEditor::GetPreview() const
{
	if ( PreviewWidgetPtr.IsStale(true) )
	{
		return nullptr;
	}

	return PreviewWidgetPtr.Get();
}

FPreviewScene* FWidgetBlueprintEditor::GetPreviewScene()
{
	return &PreviewScene;
}

bool FWidgetBlueprintEditor::IsSimulating() const
{
	return bIsSimulateEnabled;
}

void FWidgetBlueprintEditor::SetIsSimulating(bool bSimulating)
{
	bIsSimulateEnabled = bSimulating;
}

FWidgetReference FWidgetBlueprintEditor::GetReferenceFromTemplate(UWidget* TemplateWidget)
{
	TSharedRef<FWidgetHandle> Reference = MakeShareable(new FWidgetHandle(TemplateWidget));
	WidgetHandlePool.Add(Reference);

	return FWidgetReference(SharedThis(this), Reference);
}

FWidgetReference FWidgetBlueprintEditor::GetReferenceFromPreview(UWidget* PreviewWidget)
{
	UUserWidget* PreviewRoot = GetPreview();
	if ( PreviewRoot )
	{
		UWidgetBlueprint* Blueprint = GetWidgetBlueprintObj();

		if ( PreviewWidget )
		{
			FName Name = PreviewWidget->GetFName();
			return GetReferenceFromTemplate(Blueprint->WidgetTree->FindWidget(Name));
		}
	}

	return FWidgetReference(SharedThis(this), TSharedPtr<FWidgetHandle>());
}

TSharedPtr<ISequencer>& FWidgetBlueprintEditor::GetSequencer()
{
	return bIsSequencerDrawerOpen ? GetDrawerSequencer() : GetTabSequencer();
}

TSharedPtr<ISequencer> FWidgetBlueprintEditor::CreateSequencerWidgetInternal()
{
	const float InTime = 0.f;
	const float OutTime = 5.0f;

	FSequencerViewParams ViewParams(TEXT("UMGSequencerSettings"));
	{
		ViewParams.OnGetAddMenuContent = FOnGetAddMenuContent::CreateSP(this, &FWidgetBlueprintEditor::OnGetAnimationAddMenuContent);
		ViewParams.OnBuildCustomContextMenuForGuid = FOnBuildCustomContextMenuForGuid::CreateSP(this, &FWidgetBlueprintEditor::OnBuildCustomContextMenuForGuid);
	}

	FSequencerInitParams SequencerInitParams;
	{
		UWidgetAnimation* NullAnimation = UWidgetAnimation::GetNullAnimation();
		FFrameRate TickResolution = NullAnimation->MovieScene->GetTickResolution();
		FFrameNumber StartFrame = (InTime * TickResolution).FloorToFrame();
		FFrameNumber EndFrame = (OutTime * TickResolution).CeilToFrame();
		NullAnimation->MovieScene->SetPlaybackRange(StartFrame, (EndFrame - StartFrame).Value);
		FMovieSceneEditorData& EditorData = NullAnimation->MovieScene->GetEditorData();
		EditorData.WorkStart = InTime;
		EditorData.WorkEnd = OutTime;

		SequencerInitParams.ViewParams = ViewParams;
		SequencerInitParams.RootSequence = NullAnimation;
		SequencerInitParams.bEditWithinLevelEditor = false;
		SequencerInitParams.ToolkitHost = GetToolkitHost();
		SequencerInitParams.PlaybackContext = TAttribute<UObject*>(this, &FWidgetBlueprintEditor::GetAnimationPlaybackContext);
		SequencerInitParams.EventContexts = TAttribute<TArray<UObject*>>(this, &FWidgetBlueprintEditor::GetAnimationEventContexts);

		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
		SequencerInitParams.HostCapabilities.bSupportsAddFromContentBrowser = true;
		SequencerInitParams.HostCapabilities.bSupportsSidebar = true;
		SequencerInitParams.HostCapabilities.bSupportsViewportSelectability = true;
	};

	TSharedPtr<ISequencer> Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
	// Never recompile the blueprint on evaluate as this can create an insidious loop
	Sequencer->GetSequencerSettings()->SetCompileDirectorOnEvaluate(false);
	Sequencer->OnMovieSceneDataChanged().AddSP(this, &FWidgetBlueprintEditor::OnMovieSceneDataChanged);
	Sequencer->OnMovieSceneBindingsPasted().AddSP(this, &FWidgetBlueprintEditor::OnMovieSceneBindingsPasted);
	// Change selected widgets in the sequencer tree view
	Sequencer->GetSelectionChangedObjectGuids().AddSP(this, &FWidgetBlueprintEditor::SyncSelectedWidgetsWithSequencerSelection);
	OnSelectedWidgetsChanged.AddSP(this, &FWidgetBlueprintEditor::SyncSequencerSelectionToSelectedWidgets);

	// Allow sequencer to test which bindings are selected
	Sequencer->OnGetIsBindingVisible().BindRaw(this, &FWidgetBlueprintEditor::IsBindingSelected);
	Sequencers.AddUnique(Sequencer);

	return Sequencer;
}

TSharedPtr<ISequencer>& FWidgetBlueprintEditor::GetTabSequencer()
{
	if(!TabSequencer.IsValid())
	{
		TabSequencer = CreateSequencerWidgetInternal();

		bIsSequencerDrawerOpen = false;
		ChangeViewedAnimation(*UWidgetAnimation::GetNullAnimation());
	}

	return TabSequencer;
}

TSharedPtr<ISequencer>& FWidgetBlueprintEditor::GetDrawerSequencer()
{
	if(!DrawerSequencer.IsValid())
	{
		DrawerSequencer = CreateSequencerWidgetInternal();

		bIsSequencerDrawerOpen = true;
		ChangeViewedAnimation(*UWidgetAnimation::GetNullAnimation());
	}

	return DrawerSequencer;
}

void FWidgetBlueprintEditor::DockInLayoutClicked()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();

	const FName AnimationsTabName = FName(TEXT("Animations"));
	if (TSharedPtr<SDockTab> ExistingTab = GetToolkitHost()->GetTabManager()->TryInvokeTab(AnimationsTabName))
	{
		ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
	}
}

void FWidgetBlueprintEditor::ChangeViewedAnimation( UWidgetAnimation& InAnimationToView )
{
	CurrentAnimation = &InAnimationToView;
	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (SequencerPtr.IsValid())
		{
			TSharedPtr<ISequencer>  Sequencer = SequencerPtr.Pin();
			Sequencer->ResetToNewRootSequence(InAnimationToView);
			if (&InAnimationToView == UWidgetAnimation::GetNullAnimation())
			{
				Sequencer->GetSequencerWidget()->SetEnabled(false);
				Sequencer->SetAutoChangeMode(EAutoChangeMode::None);
			}
			else
			{
				Sequencer->GetSequencerWidget()->SetEnabled(true);
			}
		}
	}

	auto ToggleSequencerInteraction = [this](TWeakPtr<SOverlay> SequencerOverlay, TWeakPtr<STextBlock> NoAnimationTextBlock, UWidgetAnimation& InAnimationToView)
	{
		if (SequencerOverlay.IsValid() && NoAnimationTextBlock.IsValid())
		{
			TSharedPtr<SOverlay> SequencerOverlayPin = SequencerOverlay.Pin();
			TSharedPtr<STextBlock> NoAnimationTextBlockPin = NoAnimationTextBlock.Pin();

			if (&InAnimationToView == UWidgetAnimation::GetNullAnimation())
			{
				const FName CurveEditorTabName = FName(TEXT("SequencerGraphEditor"));
				TSharedPtr<SDockTab> ExistingTab = GetToolkitHost()->GetTabManager()->FindExistingLiveTab(CurveEditorTabName);
				if (ExistingTab)
				{
					ExistingTab->RequestCloseTab();
				}

				// Disable sequencer from interaction
				NoAnimationTextBlockPin->SetVisibility(EVisibility::Visible);
				SequencerOverlayPin->SetVisibility(EVisibility::HitTestInvisible);
			}
			else
			{
				// Allow sequencer to be interacted with
				NoAnimationTextBlockPin->SetVisibility(EVisibility::Collapsed);
				SequencerOverlayPin->SetVisibility(EVisibility::SelfHitTestInvisible);
			}
		}
	};
	ToggleSequencerInteraction(TabSequencerOverlay, NoAnimationTextBlockTab, InAnimationToView);
	ToggleSequencerInteraction(DrawerSequencerOverlay, NoAnimationTextBlockDrawer, InAnimationToView);

	InvalidatePreview();
	OnSelectedAnimationChanged.Broadcast();
}

void FWidgetBlueprintEditor::RefreshPreview()
{
	// Rebuilding the preview can force objects to be recreated, so the selection may need to be updated.
	OnSelectedWidgetsChanging.Broadcast();

	UpdatePreview(GetWidgetBlueprintObj(), true);

	CleanSelection();

	// Fire the selection updated event to ensure everyone is watching the same widgets.
	OnSelectedWidgetsChanged.Broadcast();
}

void FWidgetBlueprintEditor::Compile()
{
	DestroyPreview();

	FBlueprintEditor::Compile();

	if (const UWidgetDesignerSettings* Settings = GetDefault<UWidgetDesignerSettings>())
	{
		// Check if we should create the compile tab
		bool bShouldCreateCompileTab = false;
		switch (Settings->CreateOnCompile)
		{
		case EDisplayOnCompile::DoC_ErrorsOrWarnings:
			bShouldCreateCompileTab = CachedNumErrors > 0 || CachedNumWarnings > 0;
			break;
		case EDisplayOnCompile::DoC_ErrorsOnly:
			bShouldCreateCompileTab = CachedNumErrors > 0;
			break;
		case EDisplayOnCompile::DoC_WarningsOnly:
			bShouldCreateCompileTab = CachedNumWarnings > 0;
			break;
		case EDisplayOnCompile::DoC_Never:
		default:
			break;
		}

		if (bShouldCreateCompileTab)
		{
			GetToolkitHost()->GetTabManager()->TryInvokeTab(FBlueprintEditorTabs::CompilerResultsID);
		}

		// Check if we should dismiss the compile tab
		bool bShouldDismissCompileTab = false;
		switch (Settings->DismissOnCompile)
		{
		case EDisplayOnCompile::DoC_ErrorsOrWarnings:
			bShouldDismissCompileTab = CachedNumErrors == 0 && CachedNumWarnings == 0;
			break;
		case EDisplayOnCompile::DoC_ErrorsOnly:
			bShouldDismissCompileTab = CachedNumErrors == 0;
			break;
		case EDisplayOnCompile::DoC_WarningsOnly:
			bShouldDismissCompileTab = CachedNumWarnings == 0;
			break;
		case EDisplayOnCompile::DoC_Never:
		default:
			break;
		}

		if (bShouldDismissCompileTab)
		{
			TSharedPtr<SDockTab> CompileResultsTab = GetToolkitHost()->GetTabManager()->FindExistingLiveTab(FBlueprintEditorTabs::CompilerResultsID);
			if (CompileResultsTab)
			{
				CompileResultsTab->RequestCloseTab();
			}
		}
	}
}

bool FWidgetBlueprintEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	bool bAllowClose = Super::OnRequestClose(InCloseReason);

	// Give any active modes a chance to shutdown while the toolkit host is still alive
	// Note: This along side with the default tool palette extension tab being closed 
	// is what prevents an unrecognized tab from spawning on layout restore
	if (bAllowClose)
	{
		GetEditorModeManager().ActivateDefaultMode();
	}

	return bAllowClose;
}

void FWidgetBlueprintEditor::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingStarted(Toolkit);
}

void FWidgetBlueprintEditor::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
}

void FWidgetBlueprintEditor::DestroyPreview()
{
	UUserWidget* PreviewUserWidget = GetPreview();
	if ( PreviewUserWidget != nullptr )
	{
		check(PreviewScene.GetWorld());

		// Establish the widget as being in design time before destroying it
		PreviewUserWidget->SetDesignerFlags(GetCurrentDesignerFlags());
		
		// Immediately release the preview ptr to let people know it's gone.
		PreviewWidgetPtr.Reset();

		// Immediately notify anyone with a preview out there they need to dispose of it right now,
		// otherwise the leak detection can't be trusted.
		OnWidgetPreviewUpdated.Broadcast();

		FWidgetBlueprintEditorUtils::DestroyUserWidget(PreviewUserWidget);
	}
}

void FWidgetBlueprintEditor::UpdatePreview(UBlueprint* InBlueprint, bool bInForceFullUpdate)
{
	UUserWidget* PreviewUserWidget = GetPreview();

	// Signal that we're going to be constructing editor components
	if ( InBlueprint != nullptr && InBlueprint->SimpleConstructionScript != nullptr )
	{
		InBlueprint->SimpleConstructionScript->BeginEditorComponentConstruction();
	}

	// If the Blueprint is changing
	if ( InBlueprint != PreviewBlueprint || bInForceFullUpdate )
	{
		// Destroy the previous actor instance
		DestroyPreview();

		// Save the Blueprint we're creating a preview for
		PreviewBlueprint = Cast<UWidgetBlueprint>(InBlueprint);

		PreviewUserWidget = FWidgetBlueprintEditorUtils::CreateUserWidgetFromBlueprint(
			PreviewScene.GetWorld(),
			PreviewBlueprint,
			FWidgetBlueprintEditorUtils::FCreateWidgetFromBlueprintParams{
				GetCurrentDesignerFlags(),
				PreviewScene.GetWorld()->GetFirstLocalPlayerFromController()
			});

		// Store a reference to the preview actor.
		PreviewWidgetPtr = PreviewUserWidget;
	}

	OnWidgetPreviewUpdated.Broadcast();

	// We've changed the binding context so drastically that we should just clear all knowledge of our previous cached bindings
	for (TWeakPtr<ISequencer>& SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			Sequencer->GetEvaluationState()->ClearObjectCaches(*Sequencer);
			Sequencer->ForceEvaluate();
		}
	}
}

FGraphAppearanceInfo FWidgetBlueprintEditor::GetGraphAppearance(UEdGraph* InGraph) const
{
	FGraphAppearanceInfo AppearanceInfo = Super::GetGraphAppearance(InGraph);

	if (FBlueprintEditorUtils::IsEditorUtilityBlueprint(GetBlueprintObj()))
	{
		AppearanceInfo.CornerText = LOCTEXT("EditorUtilityWidgetAppearanceCornerText", "EDITOR UTILITY WIDGET");
	}
	else if ( GetBlueprintObj()->IsA(UWidgetBlueprint::StaticClass()) )
	{
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "WIDGET BLUEPRINT");
	}

	return AppearanceInfo;
}

TSubclassOf<UEdGraphSchema> FWidgetBlueprintEditor::GetDefaultSchemaClass() const
{
	return UWidgetGraphSchema::StaticClass();
}

void FWidgetBlueprintEditor::ClearHoveredWidget()
{
	HoveredWidget = FWidgetReference();
	OnHoveredWidgetCleared.Broadcast();
}

void FWidgetBlueprintEditor::SetHoveredWidget(FWidgetReference& InHoveredWidget)
{
	if (InHoveredWidget != HoveredWidget)
	{
		HoveredWidget = InHoveredWidget;
		OnHoveredWidgetSet.Broadcast(InHoveredWidget);
	}
}

const FWidgetReference& FWidgetBlueprintEditor::GetHoveredWidget() const
{
	return HoveredWidget;
}

void FWidgetBlueprintEditor::AddPostDesignerLayoutAction(TFunction<void()> Action)
{
	QueuedDesignerActions.Add(MoveTemp(Action));
}

void FWidgetBlueprintEditor::OnEnteringDesigner()
{
	OnEnterWidgetDesigner.Broadcast();
}

TArray< TFunction<void()> >& FWidgetBlueprintEditor::GetQueuedDesignerActions()
{
	return QueuedDesignerActions;
}

EWidgetDesignFlags FWidgetBlueprintEditor::GetCurrentDesignerFlags() const
{
	EWidgetDesignFlags Flags = EWidgetDesignFlags::Designing;
	
	if ( bShowDashedOutlines )
	{
		Flags |= EWidgetDesignFlags::ShowOutline;
	}

	if ( const UWidgetDesignerSettings* Designer = GetDefault<UWidgetDesignerSettings>() )
	{
		if ( Designer->bExecutePreConstructEvent )
		{
			Flags |= EWidgetDesignFlags::ExecutePreConstruct;
		}
	}

	return Flags;
}

bool FWidgetBlueprintEditor::GetShowDashedOutlines() const
{
	return bShowDashedOutlines;
}

void FWidgetBlueprintEditor::SetShowDashedOutlines(bool Value)
{
	bShowDashedOutlines = Value;
}

bool FWidgetBlueprintEditor::GetIsRespectingLocks() const
{
	return bRespectLocks;
}

void FWidgetBlueprintEditor::SetIsRespectingLocks(bool Value)
{
	bRespectLocks = Value;
}

void FWidgetBlueprintEditor::CreateEditorModeManager()
{
	TSharedPtr<FWidgetModeManager> WidgetModeManager = MakeShared<FWidgetModeManager>();
	WidgetModeManager->OwningToolkit = SharedThis(this);
	EditorModeManager = WidgetModeManager;

}

class FObjectAndDisplayName
{
public:
	FObjectAndDisplayName(FText InDisplayName, UObject* InObject)
	{
		DisplayName = InDisplayName;
		Object = InObject;
	}

	bool operator<(FObjectAndDisplayName const& Other) const
	{
		return DisplayName.CompareTo(Other.DisplayName) < 0;
	}

	FText DisplayName;
	UObject* Object;

};

void GetBindableObjects(UWidgetTree* WidgetTree, TArray<FObjectAndDisplayName>& BindableObjects)
{
	// Add the 'this' widget so you can animate it.
	BindableObjects.Add(FObjectAndDisplayName(LOCTEXT("RootWidgetThis", "[[This]]"), WidgetTree->GetOuter()));

	WidgetTree->ForEachWidget([&BindableObjects] (UWidget* Widget) {
		
		// if the widget has a generated name this is just some unimportant widget, don't show it in the list?
		if (Widget->IsGeneratedName() && !Widget->bIsVariable)
		{
			return;
		}
		
		BindableObjects.Add(FObjectAndDisplayName(Widget->GetLabelText(), Widget));

		if (Widget->Slot && Widget->Slot->Parent)
		{
			FText SlotDisplayName = FText::Format(LOCTEXT("AddMenuSlotFormat", "{0} ({1})"), Widget->GetLabelText(), Widget->Slot->GetClass()->GetDisplayNameText());
			BindableObjects.Add(FObjectAndDisplayName(SlotDisplayName, Widget->Slot));
		}
	});
}

void FWidgetBlueprintEditor::OnGetAnimationAddMenuContent(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> InSequencer)
{
	if (CurrentAnimation.IsValid())
	{
		const TSet<FWidgetReference>& Selection = GetSelectedWidgets();
		for (const FWidgetReference& SelectedWidget : Selection)
		{
			if (UWidget* Widget = SelectedWidget.GetPreview())
			{
				FUIAction AddWidgetTrackAction(FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::AddObjectToAnimation, (UObject*)Widget));
				MenuBuilder.AddMenuEntry(Widget->GetLabelText(), FText(), FSlateIcon(), AddWidgetTrackAction);

				if (Widget->Slot && Widget->Slot->Parent)
				{
					FText SlotDisplayName = FText::Format(LOCTEXT("AddMenuSlotFormat", "{0} ({1})"), Widget->GetLabelText(), Widget->Slot->GetClass()->GetDisplayNameText());
					FUIAction AddSlotTrackAction(FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::AddObjectToAnimation, (UObject*)Widget->Slot));
					MenuBuilder.AddMenuEntry(SlotDisplayName, FText(), FSlateIcon(), AddSlotTrackAction);
				}
			}
		}

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllNamedWidgets", "All Named Widgets"),
			LOCTEXT("AllNamedWidgetsTooltip", "Select a widget or slot to create an animation track for"),
			FNewMenuDelegate::CreateRaw(this, &FWidgetBlueprintEditor::OnGetAnimationAddMenuContentAllWidgets),
			false,
			FSlateIcon()
		);
	}
}

void FWidgetBlueprintEditor::OnGetAnimationAddMenuContentAllWidgets(FMenuBuilder& MenuBuilder)
{
	TArray<FObjectAndDisplayName> BindableObjects;
	{
		GetBindableObjects(GetPreview()->WidgetTree, BindableObjects);
		BindableObjects.Sort();
	}

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	for (FObjectAndDisplayName& BindableObject : BindableObjects)
	{
		FGuid BoundObjectGuid = ActiveSequencer->FindObjectId(*BindableObject.Object, ActiveSequencer->GetFocusedTemplateID());
		if (BoundObjectGuid.IsValid() == false)
		{
			FUIAction AddMenuAction(FExecuteAction::CreateSP(this, &FWidgetBlueprintEditor::AddObjectToAnimation, BindableObject.Object));
			MenuBuilder.AddMenuEntry(BindableObject.DisplayName, FText(), FSlateIcon(), AddMenuAction);
		}
	}
}

void FWidgetBlueprintEditor::AddObjectToAnimation(UObject* ObjectToAnimate)
{
	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UMovieScene* MovieScene = ActiveSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "AddWidgetToAnimation", "Add widget to animation" ) );
	ActiveSequencer->GetFocusedMovieSceneSequence()->Modify();

	// @todo Sequencer - Make this kind of adding more explicit, this current setup seem a bit brittle.
	FGuid NewGuid = ActiveSequencer->GetHandleToObject(ObjectToAnimate);

	TArray<UMovieSceneFolder*> SelectedParentFolders;
	ActiveSequencer->GetSelectedFolders(SelectedParentFolders);

	if (SelectedParentFolders.Num() > 0)
	{
		SelectedParentFolders[0]->AddChildObjectBinding(NewGuid);
	}
}

TSharedRef<FExtender> FWidgetBlueprintEditor::GetAddTrackSequencerExtender( const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects )
{
	TSharedRef<FExtender> AddTrackMenuExtender( new FExtender() );
	AddTrackMenuExtender->AddMenuExtension(
		SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
		EExtensionHook::Before,
		CommandList,
		FMenuExtensionDelegate::CreateRaw( this, &FWidgetBlueprintEditor::ExtendSequencerAddTrackMenu, ContextSensitiveObjects ) );
	return AddTrackMenuExtender;
}

void FWidgetBlueprintEditor::OnBuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder, FGuid ObjectBinding)
{
	if (CurrentAnimation.IsValid())
	{
		TArray<FWidgetReference> ValidSelectedWidgets;
		for (FWidgetReference SelectedWidget : SelectedWidgets)
		{
			if (SelectedWidget.IsValid())
			{
				//need to make sure it's a widget, if not bound assume it is.
				UWidget* BoundWidget = nullptr;
				bool bNotBound = true;
				TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
				if (auto ObjectsView = ActiveSequencer->FindObjectsInCurrentSequence(ObjectBinding); ObjectsView.Num() > 0)
				{
					TWeakObjectPtr<> WeakObjectPtr = ObjectsView[0];
					BoundWidget = Cast<UWidget>(WeakObjectPtr.Get());
					bNotBound = false;
				}

				if (bNotBound || (BoundWidget && SelectedWidget.GetPreview()->GetTypedOuter<UWidgetTree>() == BoundWidget->GetTypedOuter<UWidgetTree>()))
				{
					ValidSelectedWidgets.Add(SelectedWidget);
				}
			}
		}
		
		if (ValidSelectedWidgets.Num() > 0)
		{
			MenuBuilder.AddMenuSeparator();
			
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddSelectedToBinding", "Add Selected"),
				LOCTEXT("AddSelectedToBindingToolTip", "Add selected objects to this track"),
				FSlateIcon(),
				FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::AddWidgetsToTrack, ValidSelectedWidgets, ObjectBinding)
			);
			
			if (ValidSelectedWidgets.Num() > 1)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ReplaceBindingWithSelected", "Replace with Selected"),
					LOCTEXT("ReplaceBindingWithSelectedToolTip", "Replace the object binding with selected objects"),
					FSlateIcon(),
					FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::ReplaceTrackWithWidgets, ValidSelectedWidgets, ObjectBinding)
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("ReplaceObject", "Replace with {0}"), FText::FromString(ValidSelectedWidgets[0].GetPreview()->GetName())),
					FText::Format(LOCTEXT("ReplaceObjectToolTip", "Replace the bound widget in this animation with {0}"), FText::FromString(ValidSelectedWidgets[0].GetPreview()->GetName())),
					FSlateIcon(),
					FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::ReplaceTrackWithWidgets, ValidSelectedWidgets, ObjectBinding)
				);
			}
			
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveSelectedFromBinding", "Remove Selected"),
				LOCTEXT("RemoveSelectedFromBindingToolTip", "Remove selected objects from this track"),
				FSlateIcon(),
				FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::RemoveWidgetsFromTrack, ValidSelectedWidgets, ObjectBinding)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveAllBindings", "Remove All"),
				LOCTEXT("RemoveAllBindingsToolTip", "Remove all bound objects from this track"),
				FSlateIcon(),
				FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::RemoveAllWidgetsFromTrack, ObjectBinding)
			);
			
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveMissing", "Remove Missing"),
				LOCTEXT("RemoveMissingToolTip", "Remove missing objects bound to this track"),
				FSlateIcon(),
				FExecuteAction::CreateRaw(this, &FWidgetBlueprintEditor::RemoveMissingWidgetsFromTrack, ObjectBinding)
			);

			if (CanAddDynamicPossessionMenu())
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("DynamicPossession", "Dynamic Possession"),
					LOCTEXT("DynamicPossessionToolTip", "Specify a Blueprint method that will find a compatible widget for this binding"),
					FNewMenuDelegate::CreateRaw(this, &FWidgetBlueprintEditor::AddDynamicPossessionMenu, ObjectBinding));
			}
		}
	}
}

void FWidgetBlueprintEditor::ExtendSequencerAddTrackMenu( FMenuBuilder& AddTrackMenuBuilder, const TArray<UObject*> ContextObjects )
{
	if ( ContextObjects.Num() == 1 )
	{
		UWidget* Widget = Cast<UWidget>( ContextObjects[0] );

		if ( Widget != nullptr && Widget->GetTypedOuter<UUserWidget>() == GetPreview() )
		{
			if( Widget->GetParent() != nullptr && Widget->Slot != nullptr )
			{
				AddTrackMenuBuilder.BeginSection( "Slot", LOCTEXT( "SlotSection", "Slot" ) );
				{
					FUIAction AddSlotAction( FExecuteAction::CreateRaw( this, &FWidgetBlueprintEditor::AddSlotTrack, Widget->Slot ) );
					FText AddSlotLabel = FText::Format(LOCTEXT("SlotLabelFormat", "{0} Slot"), FText::FromString(Widget->GetParent()->GetName()));
					FText AddSlotToolTip = FText::Format(LOCTEXT("SlotToolTipFormat", "Add {0} slot"), FText::FromString( Widget->GetParent()->GetName()));
					AddTrackMenuBuilder.AddMenuEntry(AddSlotLabel, AddSlotToolTip, FSlateIcon(), AddSlotAction);
				}
				AddTrackMenuBuilder.EndSection();
			}

			TArray<FWidgetMaterialPropertyPath> MaterialBrushPropertyPaths;
			WidgetMaterialTrackUtilities::GetMaterialBrushPropertyPaths( Widget, MaterialBrushPropertyPaths );
			if ( MaterialBrushPropertyPaths.Num() > 0 )
			{
				AddTrackMenuBuilder.BeginSection( "Materials", LOCTEXT( "MaterialsSection", "Materials" ) );
				{
					for (FWidgetMaterialPropertyPath& MaterialBrushPropertyPath : MaterialBrushPropertyPaths )
					{
						FString DisplayName = MaterialBrushPropertyPath.PropertyPath[0]->GetDisplayNameText().ToString();
						for ( int32 i = 1; i < MaterialBrushPropertyPath.PropertyPath.Num(); i++)
						{
							DisplayName.AppendChar( '.' );
							DisplayName.Append( MaterialBrushPropertyPath.PropertyPath[i]->GetDisplayNameText().ToString() );
						}
						DisplayName.AppendChar('.');
						DisplayName.Append(MaterialBrushPropertyPath.DisplayName);

						FText DisplayNameText = FText::FromString( DisplayName );
						FUIAction AddMaterialAction( FExecuteAction::CreateRaw( this, &FWidgetBlueprintEditor::AddMaterialTrack, Widget, MaterialBrushPropertyPath.PropertyPath, DisplayNameText ) );
						FText AddMaterialLabel = DisplayNameText;
						FText AddMaterialToolTip = FText::Format( LOCTEXT( "MaterialToolTipFormat", "Add a material track for the {0} property." ), DisplayNameText );
						AddTrackMenuBuilder.AddMenuEntry( AddMaterialLabel, AddMaterialToolTip, FSlateIcon(), AddMaterialAction );
					}
				}
				AddTrackMenuBuilder.EndSection();
			}
		}
	}
}

void FWidgetBlueprintEditor::AddWidgetsToTrack(const TArray<FWidgetReference> Widgets, FGuid ObjectId)
{
	const FScopedTransaction Transaction(LOCTEXT("AddSelectedWidgetsToTrack", "Add Widgets to Track"));

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	FText ExistingBindingName;
	TArray<FWidgetReference> WidgetsToAdd;
	for (const FWidgetReference& Widget : Widgets)
	{
		UWidget* PreviewWidget = Widget.GetPreview();

		// If this widget is already bound to the animation we cannot add it to 2 separate bindings
		FGuid SelectedWidgetId = ActiveSequencer->FindObjectId(*PreviewWidget, MovieSceneSequenceID::Root);
		if (!SelectedWidgetId.IsValid())
		{
			WidgetsToAdd.Add(Widget);
		}
		else if (ExistingBindingName.IsEmpty())
		{
			ExistingBindingName = MovieScene->GetObjectDisplayName(SelectedWidgetId);
		}
	}

	if (WidgetsToAdd.Num() == 0)
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("WidgetAlreadyBound", "Widget already bound to {0}"), ExistingBindingName));
		Info.FadeInDuration = 0.1f;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 2.5f;
		auto NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();
	}
	else
	{
		MovieScene->Modify();
		WidgetAnimation->Modify();

		for (const FWidgetReference& Widget : WidgetsToAdd)
		{
			UWidget* PreviewWidget = Widget.GetPreview();
			WidgetAnimation->BindPossessableObject(ObjectId, *PreviewWidget, GetAnimationPlaybackContext());
		}

		UpdateTrackName(ObjectId);
		SyncSequencersMovieSceneData();
	}
}

void FWidgetBlueprintEditor::RemoveWidgetsFromTrack(const TArray<FWidgetReference> Widgets, FGuid ObjectId)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveWidgetsFromTrack", "Remove Widgets from Track"));

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	TArray<FWidgetReference> WidgetsToRemove;

	for (const FWidgetReference& Widget : Widgets)
	{
		UWidget* PreviewWidget = Widget.GetPreview();
		FGuid WidgetId = ActiveSequencer->FindObjectId(*PreviewWidget, MovieSceneSequenceID::Root);
		if (WidgetId.IsValid() && WidgetId == ObjectId)
		{
			WidgetsToRemove.Add(Widget);
		}
	}

	if (WidgetsToRemove.Num() == 0)
	{
		FNotificationInfo Info(LOCTEXT("SelectedWidgetNotBound", "Selected Widget not Bound to Track"));
		Info.FadeInDuration = 0.1f;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 2.5f;
		auto NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();
	}
	else
	{
		MovieScene->Modify();
		WidgetAnimation->Modify();

		for (const FWidgetReference& Widget : WidgetsToRemove)
		{
			UWidget* PreviewWidget = Widget.GetPreview();
			WidgetAnimation->RemoveBinding(*PreviewWidget);

			ActiveSequencer->PreAnimatedState.RestorePreAnimatedState(*PreviewWidget);
		}

		UpdateTrackName(ObjectId);
		SyncSequencersMovieSceneData();
	}
}

void FWidgetBlueprintEditor::RemoveAllWidgetsFromTrack(FGuid ObjectId)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveAllWidgetsFromTrack", "Remove All Widgets from Track"));

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	UUserWidget* PreviewRoot = GetPreview();
	check(PreviewRoot);

	WidgetAnimation->Modify();
	MovieScene->Modify();

	// Restore object animation state
	for (TWeakObjectPtr<> WeakObject : ActiveSequencer->FindBoundObjects(ObjectId, MovieSceneSequenceID::Root))
	{
		if (UObject* Obj = WeakObject.Get())
		{
			ActiveSequencer->PreAnimatedState.RestorePreAnimatedState(*Obj);
		}
	}

	// Remove bindings
	for (int32 Index = WidgetAnimation->AnimationBindings.Num() - 1; Index >= 0; --Index)
	{
		if (WidgetAnimation->AnimationBindings[Index].AnimationGuid == ObjectId)
		{
			WidgetAnimation->AnimationBindings.RemoveAt(Index, EAllowShrinking::No);
		}
	}

	SyncSequencersMovieSceneData();
}

void FWidgetBlueprintEditor::RemoveMissingWidgetsFromTrack(FGuid ObjectId)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveMissingWidgetsFromTrack", "Remove Missing Widgets from Track"));

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	UUserWidget* PreviewRoot = GetPreview();
	check(PreviewRoot);

	WidgetAnimation->Modify();
	MovieScene->Modify();

	for (int32 Index = WidgetAnimation->AnimationBindings.Num() - 1; Index >= 0; --Index)
	{
		const FWidgetAnimationBinding& Binding = WidgetAnimation->AnimationBindings[Index];
		if (Binding.AnimationGuid == ObjectId && Binding.FindRuntimeObject(*PreviewRoot->WidgetTree, *PreviewRoot, WidgetAnimation, ActiveSequencer->GetSharedPlaybackState()) == nullptr)
		{
			WidgetAnimation->AnimationBindings.RemoveAt(Index, EAllowShrinking::No);
		}
	}

	UpdateTrackName(ObjectId);
}

void FWidgetBlueprintEditor::ReplaceTrackWithWidgets(TArray<FWidgetReference> Widgets, FGuid ObjectId)
{
	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	// Filter out anything in the input array that is currently bound to another object in the animation
	FText ExistingBindingName;
	for (int32 Index = Widgets.Num()-1; Index >= 0; --Index)
	{
		UWidget* PreviewWidget = Widgets[Index].GetPreview();
		FGuid WidgetId = ActiveSequencer->FindObjectId(*PreviewWidget, MovieSceneSequenceID::Root);
		if (WidgetId.IsValid() && WidgetId != ObjectId)
		{
			Widgets.RemoveAt(Index, EAllowShrinking::No);

			if (ExistingBindingName.IsEmpty())
			{
				ExistingBindingName = MovieScene->GetObjectDisplayName(WidgetId);
			}
		}
	}

	if (Widgets.Num() == 0)
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("WidgetAlreadyBound", "Widget already bound to {0}"), ExistingBindingName));
		Info.FadeInDuration = 0.1f;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 2.5f;
		auto NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "ReplaceTrackWithSelectedWidgets", "Replace Track with Selected Widgets" ) );


	WidgetAnimation->Modify();
	MovieScene->Modify();

	// Remove everything from the track
	RemoveAllWidgetsFromTrack(ObjectId);
	
	// Create a new guid for the first object
	FGuid NewGuid = ActiveSequencer->GetHandleToObject(Widgets[0].GetPreview());

	// Move binding contents and remove possessable
	MovieScene->MoveBindingContents(ObjectId, NewGuid);
	MovieScene->RemovePossessable(ObjectId);

	// Add all the remaining widgets to the new binding
	AddWidgetsToTrack(Widgets, NewGuid);

	UpdateTrackName(NewGuid);
	SyncSequencersMovieSceneData();
}

bool FWidgetBlueprintEditor::CanAddDynamicPossessionMenu()
{
	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();

	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	if (!WidgetAnimation)
	{
		return false;
	}

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	FMovieSceneSequenceEditor* SequenceEditor = SequencerModule.FindSequenceEditor(WidgetAnimation->GetClass());
	return SequenceEditor && SequenceEditor->SupportsDirectorBlueprint(WidgetAnimation);
}

void FWidgetBlueprintEditor::AddDynamicPossessionMenu(FMenuBuilder& MenuBuilder, FGuid ObjectId)
{
	using namespace UE::Sequencer;

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();

	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectId);
	if (!Possessable)
	{
		return;
	}

	FWidgetAnimationBinding* WidgetBinding = nullptr;

	for (int32 Index = 0; Index < WidgetAnimation->AnimationBindings.Num(); ++Index)
	{
		FWidgetAnimationBinding& Binding = WidgetAnimation->AnimationBindings[Index];
		if (Binding.AnimationGuid == ObjectId)
		{
			WidgetBinding = &Binding;
			break;
		}
	}

	if (WidgetBinding)
	{
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowScrollBar = false;
		}

		FStructureDetailsViewArgs StructureViewArgs;
		{
			StructureViewArgs.bShowObjects = false;
			StructureViewArgs.bShowAssets = true;
			StructureViewArgs.bShowClasses = true;
			StructureViewArgs.bShowInterfaces = false;
		}

		TSharedRef<IStructureDetailsView> StructureDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor")
			.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

		// Register details customizations for this instance
		StructureDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(
			FMovieSceneDynamicBinding::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneDynamicBindingCustomization::MakeInstance, MovieScene, ObjectId, 0));

		// We can't just show the FMovieSceneDynamicBinding struct in the details view, because Slate only uses
		// the above details view customization for *properties* (not for the root object). So here we put a copy of
		// our dynamic binding struct inside a container, and when the details view is done setting values on it,
		// we copy these values back to the original dynamic binding.
		TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FMovieSceneDynamicBindingContainer::StaticStruct());
		FMovieSceneDynamicBindingContainer* BufferContainer = (FMovieSceneDynamicBindingContainer*)StructOnScope->GetStructMemory();
		BufferContainer->DynamicBinding = WidgetBinding->DynamicBinding;
		StructureDetailsView->SetStructureData(StructOnScope);

		StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &FWidgetBlueprintEditor::OnFinishedChangingDynamicBindingProperties, StructOnScope, ObjectId);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("DynamicBindingHeader", "Dynamic Binding"));
		{
			TSharedRef<SWidget> Widget = StructureDetailsView->GetWidget().ToSharedRef();
			MenuBuilder.AddWidget(Widget, FText());
		}
		MenuBuilder.EndSection();
	}
}

void FWidgetBlueprintEditor::OnFinishedChangingDynamicBindingProperties(const FPropertyChangedEvent& ChangeEvent, TSharedPtr<FStructOnScope> ValueStruct, FGuid ObjectId)
{
	auto* Container = (FMovieSceneDynamicBindingContainer*)ValueStruct->GetStructMemory();

	using namespace UE::Sequencer;

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();

	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectId);
	if (!Possessable)
	{
		return;
	}

	FWidgetAnimationBinding* WidgetBinding = nullptr;

	for (int32 Index = 0; Index < WidgetAnimation->AnimationBindings.Num(); ++Index)
	{
		FWidgetAnimationBinding& Binding = WidgetAnimation->AnimationBindings[Index];
		if (Binding.AnimationGuid == ObjectId)
		{
			WidgetBinding = &Binding;
			break;
		}
	}

	if (WidgetBinding)
	{
		WidgetBinding->DynamicBinding = Container->DynamicBinding;
	}

	// Force refresh the binding
	if (FMovieSceneEvaluationState* EvaluationState = ActiveSequencer->GetSharedPlaybackState()->FindCapability<FMovieSceneEvaluationState>())
	{
		EvaluationState->Invalidate(ObjectId, ActiveSequencer->GetFocusedTemplateID());
	}
}

void FWidgetBlueprintEditor::OnDebugObjectAccess(UObject* DebugObject)
{
	if (DebugObject)
	{
		if (UWidgetBlueprint* WidgetBlueprint = GetWidgetBlueprintObj())
		{
			if (WidgetBlueprint->GetObjectBeingDebugged() != DebugObject)
			{
				WidgetBlueprint->SetObjectBeingDebugged(DebugObject);
				RefreshMyBlueprint();
			}
		}
	}
}

void FWidgetBlueprintEditor::AddSlotTrack( UPanelSlot* Slot )
{
	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	ActiveSequencer->GetHandleToObject( Slot );
}

void FWidgetBlueprintEditor::AddMaterialTrack( UWidget* Widget, TArray<FProperty*> MaterialPropertyPath, FText MaterialPropertyDisplayName )
{
	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	FGuid WidgetHandle = ActiveSequencer->GetHandleToObject( Widget );
	if ( WidgetHandle.IsValid() )
	{
		UMovieScene* MovieScene = ActiveSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return;
		}

		TArray<FName> MaterialPropertyNamePath;
		for ( FProperty* Property : MaterialPropertyPath )
		{
			MaterialPropertyNamePath.Add( Property->GetFName() );
		}
		if( MovieScene->FindTrack( UMovieSceneWidgetMaterialTrack::StaticClass(), WidgetHandle, WidgetMaterialTrackUtilities::GetTrackNameFromPropertyNamePath( MaterialPropertyNamePath ) ) == nullptr)
		{
			const FScopedTransaction Transaction( LOCTEXT( "AddWidgetMaterialTrack", "Add widget material track" ) );

			MovieScene->Modify();

			UMovieSceneWidgetMaterialTrack* NewTrack = Cast<UMovieSceneWidgetMaterialTrack>( MovieScene->AddTrack( UMovieSceneWidgetMaterialTrack::StaticClass(), WidgetHandle ) );
			NewTrack->Modify();
			NewTrack->SetBrushPropertyNamePath( MaterialPropertyNamePath );
			NewTrack->SetDisplayName( FText::Format( LOCTEXT( "TrackDisplayNameFormat", "{0}"), MaterialPropertyDisplayName ) );

			SyncSequencersMovieSceneData();
		}
	}
}

void FWidgetBlueprintEditor::OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	bRefreshGeneratedClassAnimations = true;
}

void FWidgetBlueprintEditor::OnMovieSceneBindingsPasted(const TArray<FMovieSceneBinding>& BindingsPasted)
{
	TArray<FObjectAndDisplayName> BindableObjects;
	{
		GetBindableObjects(GetPreview()->WidgetTree, BindableObjects);
	}

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UMovieSceneSequence* AnimationSequence = ActiveSequencer->GetFocusedMovieSceneSequence();
	UObject* BindingContext = GetAnimationPlaybackContext();

	// First, rebind top level possessables (without parents) - match binding pasted's name with the bindable object name
	for (const FMovieSceneBinding& BindingPasted : BindingsPasted)
	{
		FMovieScenePossessable* Possessable = AnimationSequence->GetMovieScene()->FindPossessable(BindingPasted.GetObjectGuid());
		FText BindingPastedName = AnimationSequence->GetMovieScene()->GetObjectDisplayName(BindingPasted.GetObjectGuid());
		if (Possessable && !Possessable->GetParent().IsValid())
		{
			for (FObjectAndDisplayName& BindableObject : BindableObjects)
			{
				if (BindableObject.DisplayName.EqualTo(BindingPastedName))
				{
					AnimationSequence->BindPossessableObject(BindingPasted.GetObjectGuid(), *BindableObject.Object, BindingContext);			
					break;
				}
			}
		}
	}

	// Second, bind child possessables - match the binding pasted's parent guid with the bindable slot's content guid
	for (const FMovieSceneBinding& BindingPasted : BindingsPasted)
	{
		FMovieScenePossessable* Possessable = AnimationSequence->GetMovieScene()->FindPossessable(BindingPasted.GetObjectGuid());
		FText BindingPastedName = AnimationSequence->GetMovieScene()->GetObjectDisplayName(BindingPasted.GetObjectGuid());
		if (Possessable && Possessable->GetParent().IsValid())
		{
			for (FObjectAndDisplayName& BindableObject : BindableObjects)
			{
				UPanelSlot* PanelSlot = Cast<UPanelSlot>(BindableObject.Object);
				if (PanelSlot && PanelSlot->Content)
				{
					FGuid ParentGuid = AnimationSequence->FindPossessableObjectId(*PanelSlot->Content, BindingContext);

					if (ParentGuid == Possessable->GetParent())
					{
						AnimationSequence->BindPossessableObject(BindingPasted.GetObjectGuid(), *BindableObject.Object, BindingContext);			
						break;
					}

					// Special case for canvas slots, they need to be added again
					if (BindableObject.Object->GetFName() == BindingPastedName.ToString())
					{
						// Create handle, to rebind correctly
						ActiveSequencer->GetHandleToObject(BindableObject.Object);
						// Remove the existing binding, as it is now replaced by the that was just added by getting the handle
						AnimationSequence->GetMovieScene()->RemovePossessable(BindingPasted.GetObjectGuid());
						break;
					}
				}
			}
		}
	}
}

void FWidgetBlueprintEditor::SyncSelectedWidgetsWithSequencerSelection(TArray<FGuid> ObjectGuids)
{
	if (bUpdatingSequencerSelection)
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingExternalSelection, true);

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UMovieSceneSequence* AnimationSequence = ActiveSequencer->GetFocusedMovieSceneSequence();
	UObject* BindingContext = GetAnimationPlaybackContext();
	TSet<FWidgetReference> SequencerSelectedWidgets;
	for (FGuid Guid : ObjectGuids)
	{
		TArray<UObject*, TInlineAllocator<1>> BoundObjects;
		AnimationSequence->LocateBoundObjects(Guid, UE::UniversalObjectLocator::FResolveParams(BindingContext), ActiveSequencer->GetSharedPlaybackState(), BoundObjects);
		if (BoundObjects.Num() == 0)
		{
			continue;
		}
		else if (Cast<UPanelSlot>(BoundObjects[0]))
		{
			SequencerSelectedWidgets.Add(GetReferenceFromPreview(Cast<UPanelSlot>(BoundObjects[0])->Content));
		}
		else
		{
			UWidget* BoundWidget = Cast<UWidget>(BoundObjects[0]);
			SequencerSelectedWidgets.Add(GetReferenceFromPreview(BoundWidget));
		}
	}
	if (SequencerSelectedWidgets.Num() != 0)
	{
		SelectWidgets(SequencerSelectedWidgets, false);
	}
}

void FWidgetBlueprintEditor::SyncSequencerSelectionToSelectedWidgets()
{
	if (bUpdatingExternalSelection)
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingSequencerSelection, true);

	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			if (Sequencer->GetSequencerSettings()->GetShowSelectedNodesOnly())
			{
				Sequencer->RefreshTree();
			}

			Sequencer->ExternalSelectionHasChanged();
		}
	}
}

void FWidgetBlueprintEditor::SyncSequencersMovieSceneData()
{
	for (TWeakPtr<ISequencer> SequencerPtr : Sequencers)
	{
		if (TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}
	}
}

void FWidgetBlueprintEditor::UpdateTrackName(FGuid ObjectId)
{
	UUserWidget* PreviewRoot = GetPreview();
	UObject* BindingContext = GetAnimationPlaybackContext();

	TSharedPtr<ISequencer>& ActiveSequencer = GetSequencer();
	UWidgetAnimation* WidgetAnimation = Cast<UWidgetAnimation>(ActiveSequencer->GetFocusedMovieSceneSequence());
	UMovieScene* MovieScene = WidgetAnimation->GetMovieScene();

	const TArray<FWidgetAnimationBinding>& WidgetBindings = WidgetAnimation->GetBindings();
	
	for (FWidgetAnimationBinding& Binding : WidgetAnimation->AnimationBindings)
	{
		if (Binding.AnimationGuid != ObjectId)
		{
			continue;
		}

		TArray<UObject*, TInlineAllocator<1>> BoundObjects;
		WidgetAnimation->LocateBoundObjects(ObjectId, UE::UniversalObjectLocator::FResolveParams(BindingContext), ActiveSequencer->GetSharedPlaybackState(), BoundObjects);

		if (BoundObjects.Num() > 0)
		{
			FString NewLabel = Binding.WidgetName.ToString();
			if (BoundObjects.Num() > 1)
			{
				NewLabel.Append(FString::Printf(TEXT(" (%d)"), BoundObjects.Num()));
			}

			MovieScene->SetObjectDisplayName(ObjectId, FText::FromString(NewLabel));
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE