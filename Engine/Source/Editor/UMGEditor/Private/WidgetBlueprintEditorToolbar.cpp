// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintEditorToolbar.h"
#include "Types/ISlateMetaData.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"

#if WITH_EDITOR
	#include "Styling/AppStyle.h"
#endif // WITH_EDITOR
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "BlueprintEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "BlueprintEditorContext.h"

#include "WidgetBlueprintEditor.h"
#include "WorkflowOrientedApp/SModeWidget.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"

#define LOCTEXT_NAMESPACE "UMG"

//////////////////////////////////////////////////////////////////////////
// SBlueprintModeSeparator

class SBlueprintModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SBlueprintModeSeparator) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
			);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintEditorToolbar

FWidgetBlueprintEditorToolbar::FWidgetBlueprintEditorToolbar(TSharedPtr<FWidgetBlueprintEditor>& InWidgetEditor)
	: WidgetEditor(InWidgetEditor)
{
}

void FWidgetBlueprintEditorToolbar::AddWidgetBlueprintEditorModesToolbar(TSharedPtr<FExtender> Extender)
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = WidgetEditor.Pin();

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		BlueprintEditorPtr->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FWidgetBlueprintEditorToolbar::FillWidgetBlueprintEditorModesToolbar));
}

void FWidgetBlueprintEditorToolbar::FillWidgetBlueprintEditorModesToolbar(FToolBarBuilder& ToolbarBuilder)
{
	TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPtr = WidgetEditor.Pin();
	UBlueprint* BlueprintObj = BlueprintEditorPtr->GetBlueprintObj();

	if( !BlueprintObj ||
		(!FBlueprintEditorUtils::IsLevelScriptBlueprint(BlueprintObj) 
		&& !FBlueprintEditorUtils::IsInterfaceBlueprint(BlueprintObj)
		&& !BlueprintObj->bIsNewlyCreated)
		)
	{
		TAttribute<FName> GetActiveMode(BlueprintEditorPtr.ToSharedRef(), &FBlueprintEditor::GetCurrentMode);
		FOnModeChangeRequested SetActiveMode = FOnModeChangeRequested::CreateSP(BlueprintEditorPtr.ToSharedRef(), &FBlueprintEditor::SetCurrentMode);

		// Left side padding
		BlueprintEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(4.0f, 1.0f)));

		BlueprintEditorPtr->AddToolbarWidget(
			SNew(SModeWidget, FWidgetBlueprintApplicationModes::GetLocalizedMode(FWidgetBlueprintApplicationModes::DesignerMode), FWidgetBlueprintApplicationModes::DesignerMode)
			.OnGetActiveMode(GetActiveMode)
			.OnSetActiveMode(SetActiveMode)
			.ToolTip(IDocumentation::Get()->CreateToolTip(
				LOCTEXT("DesignerModeButtonTooltip", "Switch to Blueprint Designer Mode"),
				nullptr,
				TEXT("Shared/Editors/BlueprintEditor"),
				TEXT("DesignerMode")))
			.IconImage(FAppStyle::GetBrush("UMGEditor.SwitchToDesigner"))
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("DesignerMode")))
		);

		BlueprintEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));

		BlueprintEditorPtr->AddToolbarWidget(
			SNew(SModeWidget, FWidgetBlueprintApplicationModes::GetLocalizedMode(FWidgetBlueprintApplicationModes::GraphMode), FWidgetBlueprintApplicationModes::GraphMode)
			.OnGetActiveMode(GetActiveMode)
			.OnSetActiveMode(SetActiveMode)
			.CanBeSelected(BlueprintEditorPtr.Get(), &FBlueprintEditor::IsEditingSingleBlueprint)
			.ToolTip(IDocumentation::Get()->CreateToolTip(
				LOCTEXT("GraphModeButtonTooltip", "Switch to Graph Editing Mode"),
				nullptr,
				TEXT("Shared/Editors/BlueprintEditor"),
				TEXT("GraphMode")))
			.IconImage(FAppStyle::GetBrush("FullBlueprintEditor.SwitchToScriptingMode"))
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("GraphMode")))
		);
		
		// Right side padding
		BlueprintEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));

		if (FWidgetBlueprintApplicationModes::IsPreviewModeEnabled())
		{
			BlueprintEditorPtr->AddToolbarWidget(
				SNew(SModeWidget, FWidgetBlueprintApplicationModes::GetLocalizedMode(FWidgetBlueprintApplicationModes::PreviewMode), FWidgetBlueprintApplicationModes::PreviewMode)
				.OnGetActiveMode(GetActiveMode)
				.OnSetActiveMode(SetActiveMode)
				.CanBeSelected(BlueprintEditorPtr.Get(), &FBlueprintEditor::IsEditingSingleBlueprint)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					LOCTEXT("PreviewModeButtonTooltip", "Switch to Preview Mode"),
					nullptr,
					TEXT("Shared/Editors/BlueprintEditor"),
					TEXT("DebugMode")))
				.IconImage(FAppStyle::GetBrush("BlueprintDebugger.TabIcon"))
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("PreviewMode")))
			);
		
			// Right side padding
			BlueprintEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));
		}
	}
}


void FWidgetBlueprintEditorToolbar::AddWidgetReflector(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("WidgetTools");
	Section.InsertPosition = FToolMenuInsert("SourceControl", EToolMenuInsertType::After);

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		"OpenWidgetReflector",
		FUIAction(
			FExecuteAction::CreateLambda([=] { FGlobalTabmanager::Get()->TryInvokeTab(FTabId("WidgetReflector")); }),
			FCanExecuteAction()
		)
		, LOCTEXT("OpenWidgetReflector", "Widget Reflector")
		, LOCTEXT("OpenWidgetReflectorToolTip", "Opens the Widget Reflector, a handy tool for diagnosing problems with live widgets.")
		, FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "WidgetReflector.Icon")
	));
}

void FWidgetBlueprintEditorToolbar::AddToolPalettes(UToolMenu* InMenu)
{
	// @TODO: DarenC - For now we only support one tool palette, switch this to a dropdown when we support multiple tool palettes.
	for (TSharedPtr<FUICommandInfo>& Command : WidgetEditor.Pin()->ToolPaletteCommands)
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("UMGToolPalette");
		Section.AddDynamicEntry(Command->GetCommandName(), FNewToolMenuSectionDelegate::CreateLambda([this, Command](FToolMenuSection& InSection)
			{
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Command));
			}));
	}
}

#undef LOCTEXT_NAMESPACE
