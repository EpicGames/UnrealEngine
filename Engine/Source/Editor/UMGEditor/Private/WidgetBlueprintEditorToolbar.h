// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FExtender;
class FToolBarBuilder;
class FWidgetBlueprintEditor;
class UToolMenu;

/**
 * Handles all of the toolbar related construction for the widget blueprint editor.
 */
class FWidgetBlueprintEditorToolbar : public TSharedFromThis<FWidgetBlueprintEditorToolbar>
{

public:
	/** Constructor */
	FWidgetBlueprintEditorToolbar(TSharedPtr<FWidgetBlueprintEditor>& InWidgetEditor);
	
	/**
	 * Builds the modes toolbar for the widget blueprint editor.
	 */
	void AddWidgetBlueprintEditorModesToolbar(TSharedPtr<FExtender> Extender);

	void AddWidgetReflector(UToolMenu* InMenu);

	void AddToolPalettes(UToolMenu* InMenu);

public:
	/**  */
	void FillWidgetBlueprintEditorModesToolbar(FToolBarBuilder& ToolbarBuilder);

	TWeakPtr<FWidgetBlueprintEditor> WidgetEditor;
};
