// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintEditorUtils.h"
#include "Components/PanelSlot.h"
#include "Components/PanelWidget.h"
#include "Components/ContentWidget.h"
#include "Engine/Texture2D.h"
#include "Interfaces/IPluginManager.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/TopLevelAssetPath.h"
#include "Blueprint/WidgetTree.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "UMGEditorProjectSettings.h"
#include "WidgetBlueprint.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Settings/ContentBrowserSettings.h"

#include "ClassViewerFilter.h"
#include "EditorClassUtils.h"
#include "Dialogs/Dialogs.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragDrop/WidgetTemplateDragDropOp.h"
#include "Exporters/Exporter.h"
#include "ObjectEditorUtils.h"
#include "Components/CanvasPanelSlot.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateBlueprintClass.h"
#include "Factories.h"
#include "UnrealExporter.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedTransaction.h"
#include "Components/CanvasPanel.h"
#include "Utility/WidgetSlotPair.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Components/Widget.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ScriptInterface.h"
#include "Components/NamedSlotInterface.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/UserInterfaceSettings.h"
#include "Input/HittestGrid.h"
#include "Interfaces/ISlateRHIRendererModule.h"
#include "Interfaces/ISlate3DRenderer.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Slate/WidgetRenderer.h"
#include "UMGEditorModule.h"
#include "Widgets/SVirtualWindow.h"
#include "GraphEditorActions.h"
#include "WidgetEditingProjectSettings.h"
#include "WidgetBlueprintOperationUtils.h"

#define LOCTEXT_NAMESPACE "UMG"

class FWidgetObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FWidgetObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		const bool bIsWidget = ObjectClass->IsChildOf(UWidget::StaticClass());
		const bool bIsSlot = ObjectClass->IsChildOf(UPanelSlot::StaticClass());
		const bool bIsSlotMetaData = ObjectClass->IsChildOf(UWidgetSlotPair::StaticClass());

		return bIsWidget || bIsSlot || bIsSlotMetaData;
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if ( UWidget* Widget = Cast<UWidget>(NewObject) )
		{
			NewWidgetMap.Add(Widget->GetFName(), Widget);
		}
		else if ( UWidgetSlotPair* SlotMetaData = Cast<UWidgetSlotPair>(NewObject) )
		{
			MissingSlotData.Add(SlotMetaData->GetWidgetName(), SlotMetaData);
		}
	}

	// FCustomizableTextObjectFactory (end)

public:

	// Name->Instance object mapping
	TMap<FName, UWidget*> NewWidgetMap;

	// Instance->OldSlotMetaData that didn't survive the journey because it wasn't copied.
	TMap<FName, UWidgetSlotPair*> MissingSlotData;
};

bool FWidgetBlueprintEditorUtils::VerifyWidgetRename(TSharedRef<class FWidgetBlueprintEditor> BlueprintEditor, FWidgetReference Widget, const FText& NewName, FText& OutErrorMessage)
{
	UWidgetBlueprint* Blueprint = BlueprintEditor->GetWidgetBlueprintObj();
	return FWidgetBlueprintOperationUtils::VerifyWidgetRename(Blueprint, Widget.GetTemplate(), NewName, OutErrorMessage);
}

void FWidgetBlueprintEditorUtils::SetDesiredFocus(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, const FName DesiredFocusWidgetName)
{
	SetDesiredFocus(BlueprintEditor->GetWidgetBlueprintObj(), DesiredFocusWidgetName);
}

void FWidgetBlueprintEditorUtils::SetDesiredFocus(UWidgetBlueprint* Blueprint, const FName& DesiredFocusWidgetName)
{
	if (Blueprint)
	{
		if (Blueprint->GeneratedClass)
		{
			if (UUserWidget* WidgetCDO = Blueprint->GeneratedClass->GetDefaultObject<UUserWidget>())
			{
				WidgetCDO->SetFlags(RF_Transactional);
				WidgetCDO->Modify();
				WidgetCDO->SetDesiredFocusWidget(DesiredFocusWidgetName);
			}
		}

		constexpr bool bFocusIfOpen = false;
		FWidgetBlueprintEditor* BlueprintEditor = static_cast<FWidgetBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, bFocusIfOpen));
		if (BlueprintEditor)
		{
			if (UUserWidget* PreviewWidget = BlueprintEditor->GetPreview())
			{
				// We need to change the PreviewWidget to make sure the DetailPanel show the right value.
				PreviewWidget->SetFlags(RF_Transactional);
				PreviewWidget->Modify();
				PreviewWidget->SetDesiredFocusWidget(DesiredFocusWidgetName);
			}
		}
	}
}


void FWidgetBlueprintEditorUtils::ReplaceDesiredFocus(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, const FName& OldName, const FName& NewName)
{
	ReplaceDesiredFocus(BlueprintEditor->GetWidgetBlueprintObj(), OldName, NewName);
}

void FWidgetBlueprintEditorUtils::ReplaceDesiredFocus(UWidgetBlueprint* Blueprint, const FName& OldName, const FName& NewName)
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		if (UUserWidget* WidgetCDO = Blueprint->GeneratedClass->GetDefaultObject<UUserWidget>())
		{
			// Verify if the Name changed is the Desired focus Widget name.
			if (WidgetCDO->GetDesiredFocusWidgetName() == OldName)
			{
				WidgetCDO->SetFlags(RF_Transactional);
				WidgetCDO->Modify();
				WidgetCDO->SetDesiredFocusWidget(NewName);
				
				constexpr bool bFocusIfOpen = false;
				FWidgetBlueprintEditor* BlueprintEditor = static_cast<FWidgetBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, bFocusIfOpen));
				if (BlueprintEditor)
				{
					if (UUserWidget* PreviewWidget = BlueprintEditor->GetPreview())
					{
						ensure(PreviewWidget->GetDesiredFocusWidgetName() == OldName);

						// We need to change the PreviewWidget to make sure the DetailPanel show the right value.
						PreviewWidget->SetFlags(RF_Transactional);
						PreviewWidget->Modify();
						PreviewWidget->SetDesiredFocusWidget(NewName);
					}
				}
			}
		}
	}
}

bool FWidgetBlueprintEditorUtils::RenameWidget(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, const FName& OldObjectName, const FString& NewDisplayName)
{
	UWidgetBlueprint* Blueprint = BlueprintEditor->GetWidgetBlueprintObj();
	check(Blueprint);

	UWidget* Widget = Blueprint->WidgetTree->FindWidget(OldObjectName);
	check(Widget);
	return FWidgetBlueprintOperationUtils::RenameWidget(Blueprint, Widget, NewDisplayName);
}

void FWidgetBlueprintEditorUtils::CreateWidgetContextMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, FVector2D TargetLocation)
{
	BlueprintEditor->PasteDropLocation = TargetLocation;

	TSet<FWidgetReference> Widgets = BlueprintEditor->GetSelectedWidgets();
	UWidgetBlueprint* BP = BlueprintEditor->GetWidgetBlueprintObj();

	MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
	{
		MenuBuilder.PushCommandList(BlueprintEditor->DesignerCommandList.ToSharedRef());
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			
			// Insert "Find References" sub-menu here
			MenuBuilder.AddSubMenu(
				LOCTEXT("FindReferences_Label", "Find References"),
				LOCTEXT("FindReferences_Tooltip", "Options for finding references to class members"),
				FNewMenuDelegate::CreateStatic(&FGraphEditorCommands::BuildFindReferencesMenu),
				false,
				FSlateIcon()
			);
		}
		MenuBuilder.PopCommandList();

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Actions");
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT( "EditBlueprint_Label", "Edit Widget Blueprint..." ),
			LOCTEXT( "EditBlueprint_Tooltip", "Open the selected Widget Blueprint(s) for edit." ),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic( &FWidgetBlueprintEditorUtils::ExecuteOpenSelectedWidgetsForEdit, Widgets ),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateStatic( &FWidgetBlueprintEditorUtils::CanOpenSelectedWidgetsForEdit, Widgets )
				)
			);

		if (!FWidgetBlueprintEditorUtils::IsAnySelectedWidgetLocked(Widgets))
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("WidgetTree_WrapWith", "Wrap With..."),
				LOCTEXT("WidgetTree_WrapWithToolTip", "Wraps the currently selected widgets inside of another container widget"),
				FNewMenuDelegate::CreateStatic(&FWidgetBlueprintEditorUtils::BuildWrapWithMenu, BlueprintEditor, BP, Widgets)
			);

			if (Widgets.Num() == 1)
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("WidgetTree_ReplaceWith", "Replace With..."),
					LOCTEXT("WidgetTree_ReplaceWithToolTip", "Replaces the currently selected widget, with another widget"),
					FNewMenuDelegate::CreateStatic(&FWidgetBlueprintEditorUtils::BuildReplaceWithMenu, BlueprintEditor, BP, Widgets)
				);
			}
		}
	}
	MenuBuilder.EndSection();

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	const TArrayView<const TSharedPtr<IWidgetContextMenuExtension>> ContextMenuExtensions = EditorModule.GetWidgetContextMenuExtensibilityManager()->GetExtensions();
	for (const TSharedPtr<IWidgetContextMenuExtension>& ContextMenuExtension : ContextMenuExtensions)
	{
		ContextMenuExtension->ExtendContextMenu(MenuBuilder, BlueprintEditor, TargetLocation);
	}
}

void FWidgetBlueprintEditorUtils::ExecuteOpenSelectedWidgetsForEdit( TSet<FWidgetReference> SelectedWidgets )
{
	for ( auto& Widget : SelectedWidgets )
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset( Widget.GetTemplate()->GetClass()->ClassGeneratedBy );
	}
}

bool FWidgetBlueprintEditorUtils::CanOpenSelectedWidgetsForEdit( TSet<FWidgetReference> SelectedWidgets )
{
	bool bCanOpenAllForEdit = SelectedWidgets.Num() > 0;
	for ( auto& Widget : SelectedWidgets )
	{
		auto Blueprint = Widget.GetTemplate()->GetClass()->ClassGeneratedBy;
		if ( !Blueprint || !Blueprint->IsA( UWidgetBlueprint::StaticClass() ) )
		{
			bCanOpenAllForEdit = false;
			break;
		}
	}

	return bCanOpenAllForEdit;
}

void FWidgetBlueprintEditorUtils::DeleteWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* Blueprint, TSet<FWidgetReference> Widgets, bool bSilentDelete /*=false*/)
{
	DeleteWidgets(Blueprint, ResolveWidgetTemplates(Widgets), bSilentDelete ? EDeleteWidgetWarningType::DeleteSilently : EDeleteWidgetWarningType::WarnAndAskUser);
}

void FWidgetBlueprintEditorUtils::DeleteWidgets(UWidgetBlueprint* Blueprint, TSet<UWidget*> Widgets, EDeleteWidgetWarningType WarningType)
{
	if ( Widgets.Num() > 0 )
	{
		// Check if the widgets are used in the graph
		FScopedTransaction Transaction(LOCTEXT("RemoveWidget", "Remove Widget"));
		TArray<UWidget*> UsedVariables;
		TArray<FText> WidgetNames;
		const bool bIncludeChildrenVariables = true;
		FindUsedVariablesForWidgets(Widgets, Blueprint, UsedVariables, WidgetNames, bIncludeChildrenVariables);

		if (WarningType == EDeleteWidgetWarningType::WarnAndAskUser && UsedVariables.Num()!= 0 && !ShouldContinueDeleteOperation(Blueprint, WidgetNames))
		{
			Transaction.Cancel();
			return;
		}

		Blueprint->WidgetTree->SetFlags(RF_Transactional);
		Blueprint->WidgetTree->Modify();
		Blueprint->Modify();

		bool bRemoved = false;
		for (UWidget* Item : Widgets)
		{
			UWidget* WidgetTemplate = Item;
			WidgetTemplate->SetFlags(RF_Transactional);
			const FName WidgetName = WidgetTemplate->GetFName();
			// Find and update all binding references in the widget blueprint
			for (int32 BindingIndex = Blueprint->Bindings.Num() - 1; BindingIndex >= 0; BindingIndex--)
			{
				FDelegateEditorBinding& Binding = Blueprint->Bindings[BindingIndex];
				if (Binding.ObjectName == WidgetTemplate->GetName())
				{
					Blueprint->Bindings.RemoveAt(BindingIndex);
				}
			}

			// Modify the widget's parent
			UPanelWidget* Parent = WidgetTemplate->GetParent();
			if ( Parent )
			{
				Parent->SetFlags(RF_Transactional);
				Parent->Modify();
			}
			
			// Modify the widget being removed.
			WidgetTemplate->Modify();

			bRemoved |= Blueprint->WidgetTree->RemoveWidget(WidgetTemplate);

			// If the widget we're removing doesn't have a parent it may be rooted in a named slot,
			// so check there as well.
			if ( WidgetTemplate->GetParent() == nullptr )
			{
				bRemoved |= FindAndRemoveNamedSlotContent(WidgetTemplate, Blueprint->WidgetTree);
			}

			if (UsedVariables.Contains(WidgetTemplate))
			{
				FBlueprintEditorUtils::RemoveVariableNodes(Blueprint, WidgetTemplate->GetFName());
			}

			// Rename the Desired Focus that fit the Widget Deleted
			ReplaceDesiredFocus(Blueprint, WidgetTemplate->GetFName(), FName());

			// Rename the removed widget to the transient package so that it doesn't conflict with future widgets sharing the same name.
			WidgetTemplate->Rename(nullptr, GetTransientPackage());

			// Deletion can sometimes happen from replacing a widget with another one with the same name, so only delete the variable data if we no longer have a widget with the same name
			const bool bHasWidgetWithSameName = Blueprint->GetAllSourceWidgets().ContainsByPredicate([WidgetName](const UWidget* Widget)
			{
				return WidgetName == Widget->GetFName();
			});
			if (!bHasWidgetWithSameName)
			{
				Blueprint->OnVariableRemoved(WidgetName);
			}

			// Rename all child widgets as well, to the transient package so that they don't conflict with future widgets sharing the same name.
			TArray<UWidget*> ChildWidgets;
			UWidgetTree::GetChildWidgets(WidgetTemplate, ChildWidgets);
			for ( UWidget* Widget : ChildWidgets )
			{
				const FName ChildWidgetName = Widget->GetFName();
				Widget->SetFlags(RF_Transactional);
				Widget->Modify();
				if (UsedVariables.Contains(Widget))
				{
					FBlueprintEditorUtils::RemoveVariableNodes(Blueprint, Widget->GetFName());
				}
				Widget->Rename(nullptr, GetTransientPackage());
				
				// Deletion can sometimes happen from replacing a widget with another one with the same name, so only delete the variable data if we no longer have a widget with the same name
				const bool bHasChildWidgetWithSameName = Blueprint->GetAllSourceWidgets().ContainsByPredicate([ChildWidgetName](const UWidget* Widget)
				{
					return ChildWidgetName == Widget->GetFName();
				});
				if (!bHasChildWidgetWithSameName)
				{
					Blueprint->OnVariableRemoved(ChildWidgetName);
				}
			}
		}

		//TODO UMG There needs to be an event for widget removal so that caches can be updated, and selection

		if ( bRemoved )
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
}

void FWidgetBlueprintEditorUtils::FindUsedVariablesForWidgets(const TSet<UWidget*>& Widgets, const UWidgetBlueprint* BP, TArray<UWidget*>& UsedVariables, TArray<FText>& WidgetNames, bool bIncludeVariablesOnChildren)
{
	TSet<UWidget*> AllWidgets;
	AllWidgets.Reserve(Widgets.Num());
	for (UWidget* Item : Widgets)
	{
		AllWidgets.Add(Item);
		if (bIncludeVariablesOnChildren)
		{
			TArray<UWidget*> ChildWidgets;
			UWidgetTree::GetChildWidgets(Item, ChildWidgets);
			AllWidgets.Append(ChildWidgets);
		}
	}

	for (UWidget* Widget : AllWidgets)
	{
		if (FBlueprintEditorUtils::IsVariableUsed(BP, Widget->GetFName()))
		{
			WidgetNames.Add(FText::FromName(Widget->GetFName()));
			UsedVariables.Add(Widget);
		}
	}
}

bool FWidgetBlueprintEditorUtils::ShouldContinueDeleteOperation(UWidgetBlueprint* BP, const TArray<FText>& WidgetNames)
{
	// If the Widget is used in the graph ask the user before we continue.
	if (WidgetNames.Num())
	{
		FText ConfirmDelete = FText::Format(LOCTEXT("ConfirmDeleteVariableInUse", "One or more widgets are in use in the graph! Do you really want to delete them? \n\n {0}"),
			FText::Join(LOCTEXT("ConfirmDeleteVariableInUsedDelimiter", " \n "), WidgetNames));

		// Warn the user that this may result in data loss
		FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("DeleteVar", "Delete widgets"), "DeleteWidgetsInUse_Warning");
		Info.ConfirmText = LOCTEXT("DeleteVariable_Yes", "Yes");
		Info.CancelText = LOCTEXT("DeleteVariable_No", "No");

		FSuppressableWarningDialog DeleteVariableInUse(Info);
		if (DeleteVariableInUse.ShowModal() == FSuppressableWarningDialog::Cancel)
		{
			return false;
		}
	}
			
	return true;
}


bool FWidgetBlueprintEditorUtils::ShouldContinueReplaceOperation(UWidgetBlueprint* BP, const TArray<FText>& WidgetNames)
{
	// If the Widget is used in the graph ask the user before we continue.
	if (WidgetNames.Num())
	{
		FText ConfirmDelete = FText::Format(LOCTEXT("ConfirmReplaceWidgetWithVariableInUse", "One or more widgets you want to replace are in use in the graph! Do you really want to replace them? \n\n {0}"),
			FText::Join(LOCTEXT("ConfirmDeleteVariableInUsedDelimiter", " \n "), WidgetNames));

		// Warn the user that this may result in data loss
		FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("ReplaceWidgetVar", "Replace widgets"), "ReaplaceWidgetsInUse_Warning");
		Info.ConfirmText = LOCTEXT("ReplaceWidget_Yes", "Yes");
		Info.CancelText = LOCTEXT("ReplaceWidget_No", "No");

		FSuppressableWarningDialog DeleteVariableInUse(Info);
		if (DeleteVariableInUse.ShowModal() == FSuppressableWarningDialog::Cancel)
		{
			return false;
		}
	}

	return true;
}

TScriptInterface<INamedSlotInterface> FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree)
{
	// If the named slot comes from a parent widget class, the WidgetTree will be the SlotHost 
	TArray<FName> SlotNames;
	WidgetTree->GetSlotNames(SlotNames);

	for (FName SlotName : SlotNames)
	{
		if (UWidget* SlotContent = WidgetTree->GetContentForSlot(SlotName))
		{
			if (SlotContent == WidgetTemplate)
			{
				return WidgetTree;
			}
		}
	}
	
	return TScriptInterface<INamedSlotInterface>(FindNamedSlotHostWidgetForContent(WidgetTemplate, WidgetTree));
}

UWidget* FWidgetBlueprintEditorUtils::FindNamedSlotHostWidgetForContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree)
{
	UWidget* HostWidget = nullptr;

	WidgetTree->ForEachWidget([&](UWidget* Widget) {

		if (HostWidget != nullptr)
		{
			return;
		}

		if (INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(Widget))
		{
			TArray<FName> SlotNames;
			NamedSlotHost->GetSlotNames(SlotNames);

			for (FName SlotName : SlotNames)
			{
				if (UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName))
				{
					if (SlotContent == WidgetTemplate)
					{
						HostWidget = Widget;
					}
				}
			}
		}
	});

	return HostWidget;
}

void FWidgetBlueprintEditorUtils::FindAllAncestorNamedSlotHostWidgetsForContent(TArray<FWidgetReference>& OutSlotHostWidgets, UWidget* WidgetTemplate, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor)
{
	OutSlotHostWidgets.Empty();
	UUserWidget* Preview = BlueprintEditor->GetPreview();
	UWidgetBlueprint* WidgetBP = BlueprintEditor->GetWidgetBlueprintObj();
	UWidgetTree* WidgetTree = (WidgetBP != nullptr) ? ToRawPtr(WidgetBP->WidgetTree) : nullptr;

	if (Preview != nullptr && WidgetTree != nullptr)
	{
		// Find the first widget up the chain with a null parent, they're the only candidates for this approach.
		while (WidgetTemplate && WidgetTemplate->GetParent())
		{
			WidgetTemplate = WidgetTemplate->GetParent();
		}

		UWidget* SlotHostWidget = FindNamedSlotHostWidgetForContent(WidgetTemplate, WidgetTree);
		while (SlotHostWidget != nullptr)
		{
			UWidget* SlotWidget = Preview->GetWidgetFromName(SlotHostWidget->GetFName());
			FWidgetReference WidgetRef;

			if (SlotWidget != nullptr)
			{
				WidgetRef = BlueprintEditor->GetReferenceFromPreview(SlotWidget);

				if (WidgetRef.IsValid())
				{
					OutSlotHostWidgets.Add(WidgetRef);
				}
			}

			WidgetTemplate = WidgetRef.GetTemplate();

			SlotHostWidget = nullptr;
			if (WidgetTemplate != nullptr)
			{
				// Find the first widget up the chain with a null parent, they're the only candidates for this approach.
				while (WidgetTemplate->GetParent())
				{
					WidgetTemplate = WidgetTemplate->GetParent();
				}

				SlotHostWidget = FindNamedSlotHostWidgetForContent(WidgetRef.GetTemplate(), WidgetTree);
			}
		}
	}
}

bool FWidgetBlueprintEditorUtils::RemoveNamedSlotHostContent(UWidget* WidgetTemplate, TScriptInterface<INamedSlotInterface> NamedSlotHost)
{
	return ReplaceNamedSlotHostContent(WidgetTemplate, NamedSlotHost, nullptr);
}

bool FWidgetBlueprintEditorUtils::ReplaceNamedSlotHostContent(UWidget* WidgetTemplate, TScriptInterface<INamedSlotInterface> NamedSlotHost, UWidget* NewContentWidget)
{
	TArray<FName> SlotNames;
	NamedSlotHost->GetSlotNames(SlotNames);

	for (FName SlotName : SlotNames)
	{
		if (UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName))
		{
			if (SlotContent == WidgetTemplate)
			{
				NamedSlotHost.GetObject()->Modify();
				if (UPanelWidget* NamedSlot = WidgetTemplate->GetParent())
				{
					// Make sure we also mark the named slot as modified to properly track changes in it.
					NamedSlot->Modify();
				}

				if (NewContentWidget)
				{
					NewContentWidget->Modify();
					if (UPanelWidget* Parent = NewContentWidget->GetParent())
					{
						Parent->Modify();
						NewContentWidget->RemoveFromParent();
					}
				}
				NamedSlotHost->SetContentForSlot(SlotName, NewContentWidget);
				return true;
			}
		}
	}

	return false;
}

bool FWidgetBlueprintEditorUtils::FindAndRemoveNamedSlotContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree)
{
	UWidget* NamedSlotHostWidget = FindNamedSlotHostWidgetForContent(WidgetTemplate, WidgetTree);
	if (TScriptInterface<INamedSlotInterface> NamedSlotHost = TScriptInterface<INamedSlotInterface>(NamedSlotHostWidget) )
	{
		NamedSlotHostWidget->Modify();
		return RemoveNamedSlotHostContent(WidgetTemplate, NamedSlotHost);
	}

	return false;
}

void FWidgetBlueprintEditorUtils::BuildWrapWithMenu(FMenuBuilder& Menu, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	TArray<UClass*> WrapperClasses;
	for ( TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt )
	{
		UClass* WidgetClass = *ClassIt;
		if ( FWidgetBlueprintEditorUtils::IsUsableWidgetClass(WidgetClass, BlueprintEditor) )
		{
			if ( WidgetClass->IsChildOf(UPanelWidget::StaticClass()) && !WidgetClass->HasAnyClassFlags(CLASS_HideDropDown) )
			{
				WrapperClasses.Add(WidgetClass);
			}
		}
	}

	WrapperClasses.Sort([] (UClass& Lhs, UClass& Rhs) { return Lhs.GetDisplayNameText().CompareTo(Rhs.GetDisplayNameText()) < 0; });

	Menu.BeginSection("WrapWith", LOCTEXT("WidgetTree_WrapWith", "Wrap With..."));
	{
		for ( UClass* WrapperClass : WrapperClasses )
		{
			Menu.AddMenuEntry(
				WrapperClass->GetDisplayNameText(),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakBlueprintEditor = TWeakPtr<FWidgetBlueprintEditor>(BlueprintEditor), BP, Widgets, WrapperClass]()
					{
						if (TSharedPtr<FWidgetBlueprintEditor> PinnedEditor = WeakBlueprintEditor.Pin())
						{
							TArray<UWidget*> TemplateWidgets;
							TemplateWidgets.Reserve(Widgets.Num());
							for (const FWidgetReference& Ref : Widgets)
							{
								if (UWidget* Template = Ref.GetTemplate())
								{
									TemplateWidgets.Add(Template);
								}
							}
							FWidgetBlueprintOperationUtils::WrapWidgets(BP, TemplateWidgets, WrapperClass);
						}
					}),
					FCanExecuteAction()
				));
		}
	}
	Menu.EndSection();
}

void FWidgetBlueprintEditorUtils::BuildReplaceWithMenu(FMenuBuilder& Menu, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	Menu.BeginSection("ReplaceWith", LOCTEXT("WidgetTree_ReplaceWith", "Replace With..."));
	{
		if ( Widgets.Num() == 1 )
		{
			FWidgetReference Widget = *Widgets.CreateIterator();
			UClass* WidgetClass = Widget.GetTemplate()->GetClass();
			TWeakObjectPtr<UClass> TemplateWidget = BlueprintEditor->GetSelectedTemplate();
			FAssetData SelectedUserWidget = BlueprintEditor->GetSelectedUserWidget();
			if (TemplateWidget.IsValid() || SelectedUserWidget.GetSoftObjectPath().IsValid() )
			{
				Menu.AddMenuEntry(
					FText::Format(LOCTEXT("WidgetTree_ReplaceWithSelection", "Replace With {0}"), FText::FromString(TemplateWidget.IsValid() ? TemplateWidget->GetName() : SelectedUserWidget.AssetName.ToString())),
					FText::Format(LOCTEXT("WidgetTree_ReplaceWithSelectionToolTip", "Replace this widget with a {0}"), FText::FromString(TemplateWidget.IsValid() ? TemplateWidget->GetName() : SelectedUserWidget.AssetName.ToString())),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FWidgetBlueprintEditorUtils::ReplaceWidgetWithSelectedTemplate, BlueprintEditor, BP, Widget),
						FCanExecuteAction::CreateStatic(&FWidgetBlueprintEditorUtils::CanBeReplacedWithTemplate, BlueprintEditor, BP, Widget)
					));
				Menu.AddMenuSeparator();
			}

			if ( WidgetClass->IsChildOf(UPanelWidget::StaticClass()) && Cast<UPanelWidget>(Widget.GetTemplate())->GetChildrenCount() == 1 )
			{
				Menu.AddMenuEntry(
					LOCTEXT("ReplaceWithChild", "Replace With Child"),
					LOCTEXT("ReplaceWithChildTooltip", "Remove this widget and insert the children of this widget into the parent."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([BP, Widget]()
						{
							UWidget* WidgetToReplace = Widget.GetTemplate();
							if (!BP || !WidgetToReplace)
							{
								return;
							}

							TSet<UWidget*> WidgetsToDelete{ WidgetToReplace };
							TArray<UWidget*> UsedVariables;
							TArray<FText> WidgetNames;
							const bool bIncludeChildrenVariables = false;
							FWidgetBlueprintEditorUtils::FindUsedVariablesForWidgets(WidgetsToDelete, BP, UsedVariables, WidgetNames, bIncludeChildrenVariables);

							if (UsedVariables.Num() != 0 && !FWidgetBlueprintEditorUtils::ShouldContinueReplaceOperation(BP, WidgetNames))
							{
								return;
							}

							FText IgnoredError;
							FWidgetBlueprintOperationUtils::ReplaceWidgetWithChild(BP, WidgetToReplace, IgnoredError);
						}),
						FCanExecuteAction()
					));

				Menu.AddMenuSeparator();
			}			
			if (TScriptInterface<INamedSlotInterface> NamedSlotHost = TScriptInterface<INamedSlotInterface>(Widget.GetTemplate()))
			{								
				TArray<FName> SlotNames;
				NamedSlotHost->GetSlotNames(SlotNames);
				for (const FName& SlotName : SlotNames)
				{
					const FText SlotNameTxt = FText::FromString(SlotName.ToString());
					if (UWidget* Content = NamedSlotHost->GetContentForSlot(SlotName))
					{
						Menu.AddMenuEntry(
							FText::Format(LOCTEXT("ReplaceWithNamedSlot", "Replace With '{0}'"), SlotNameTxt),
							FText::Format(LOCTEXT("ReplaceWithNamedSlotTooltip", "Remove this widget and insert '{0}' content into the parent."), SlotNameTxt),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([BP, Widget, SlotName]()
									{
										FText IgnoredError;
										FWidgetBlueprintOperationUtils::ReplaceWidgetWithNamedSlot(BP, Widget.GetTemplate(), SlotName, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::WarnAndAskUser, IgnoredError);
									}),
								FCanExecuteAction()
							));
					}
				}
				Menu.AddMenuSeparator();
			}
		}

		TArray<UClass*> ReplacementClasses;
		for ( TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt )
		{
			UClass* WidgetClass = *ClassIt;
			if ( FWidgetBlueprintEditorUtils::IsUsableWidgetClass(WidgetClass, BlueprintEditor) )
			{
				if ( WidgetClass->IsChildOf(UPanelWidget::StaticClass()) && !WidgetClass->HasAnyClassFlags(CLASS_HideDropDown) )
				{
					// Only allow replacement with panels that accept multiple children
					if ( WidgetClass->GetDefaultObject<UPanelWidget>()->CanHaveMultipleChildren() )
					{
						ReplacementClasses.Add(WidgetClass);
					}
				}
			}
		}

		ReplacementClasses.Sort([] (UClass& Lhs, UClass& Rhs) { return Lhs.GetDisplayNameText().CompareTo(Rhs.GetDisplayNameText()) < 0; });

		for ( UClass* ReplacementClass : ReplacementClasses )
		{
			Menu.AddMenuEntry(
				ReplacementClass->GetDisplayNameText(),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FWidgetBlueprintEditorUtils::ReplaceWidgets, BP, ResolveWidgetTemplates(Widgets), ReplacementClass, EReplaceWidgetNamingMethod::MaintainNameAndReferences)
				));
		}
	}
	Menu.EndSection();
}


bool FWidgetBlueprintEditorUtils::IsDesiredFocusWidget(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidget* Widget)
{
	return IsDesiredFocusWidget(BlueprintEditor->GetWidgetBlueprintObj(), Widget);
}

bool FWidgetBlueprintEditorUtils::IsDesiredFocusWidget(UWidgetBlueprint* Blueprint, UWidget* Widget)
{
	if (Blueprint && Blueprint->GeneratedClass && Widget)
	{
		if (UUserWidget* WidgetCDO = Blueprint->GeneratedClass->GetDefaultObject<UUserWidget>())
		{
			return (WidgetCDO && WidgetCDO->GetDesiredFocusWidgetName() == Widget->GetFName());
		}
	}

	return false;
}

void FWidgetBlueprintEditorUtils::ReplaceWidgetWithSelectedTemplate(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference Widget)
{
	// @Todo: Needs to deal with bound object in animation tracks

	UWidget* WidgetToReplace = Widget.GetTemplate();
	if (!WidgetToReplace)
	{
		return;
	}

	TSharedPtr<FWidgetTemplateClass> TemplateClass;
	if (UClass* ReplacementWidgetClass = BlueprintEditor->GetSelectedTemplate().Get())
	{
		TemplateClass = MakeShareable(new FWidgetTemplateClass(ReplacementWidgetClass));
	}
	else
	{	
		const FAssetData SelectedUserWidget = BlueprintEditor->GetSelectedUserWidget();		
		if (SelectedUserWidget.GetSoftObjectPath().IsValid())
		{
			TemplateClass = MakeShareable(new FWidgetTemplateBlueprintClass(SelectedUserWidget));
		}	
	}
	
	if (!TemplateClass.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ReplaceWidgets", "Replace Widgets"));

	TArray<UWidget*> UsedVariables;
	TArray<FText> WidgetNames;
	const bool bIncludeChildrenVariables = false;
	FindUsedVariablesForWidgets({WidgetToReplace}, BP, UsedVariables, WidgetNames, bIncludeChildrenVariables);

	if (UsedVariables.Num() != 0 && !ShouldContinueReplaceOperation(BP, WidgetNames))
	{
		Transaction.Cancel();
		return;
	}
	
	FWidgetBlueprintOperationUtils::ReplaceWidgetsWithTemplateClass(BP, {WidgetToReplace}, TemplateClass, EReplaceWidgetNamingMethod::MaintainNameAndReferences);
}

bool FWidgetBlueprintEditorUtils::CanBeReplacedWithTemplate(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference Widget)
{
	FAssetData SelectedUserWidget = BlueprintEditor->GetSelectedUserWidget();
	UWidget* ThisWidget = Widget.GetTemplate();
	UPanelWidget* ExistingPanel = Cast<UPanelWidget>(ThisWidget);

	UClass* WidgetClass = nullptr;
	// If selecting another widget blueprint
	if (SelectedUserWidget.GetSoftObjectPath().IsValid())
	{
		if (ExistingPanel && ExistingPanel->GetChildrenCount() != 0)
		{
			return false;
		}
		if (UWidget* NewWidget = FWidgetTemplateBlueprintClass(SelectedUserWidget).Create(BP->WidgetTree))
		{
			// If we are creating a UserWidget, check for Circular references
			if (UUserWidget* NewUserWidget = Cast<UUserWidget>(NewWidget))
			{
				const bool bFreeFromCircularRefs = BP->IsWidgetFreeFromCircularReferences(NewUserWidget);
				NewWidget->Rename(nullptr, GetTransientPackage());
				return bFreeFromCircularRefs;
			}
			WidgetClass = NewWidget->GetClass();
			NewWidget->Rename(nullptr, GetTransientPackage());
		}
	}

	// If we get here, the Widget selected is not a UserWidget and it's not a Blueprint.
	if (!WidgetClass)
	{
		WidgetClass = BlueprintEditor->GetSelectedTemplate().Get();
	}

	// If the Widget to replace is not a Panel we can replace it with anything	
	if (!ExistingPanel)
	{
		return true;
	}

	const bool bNewWidgetClassIsAPanel = WidgetClass->IsChildOf(UPanelWidget::StaticClass());
	// If the Widget to replace is a Panel and the new widget is not, we allow to replace it only if it's empty;
	if (!bNewWidgetClassIsAPanel)
	{
		return ExistingPanel->GetChildrenCount() == 0;
	}

	// If the Widget to replace is a Panel that can have multiple children, we allow to replace it with a Panel that can support multiple children only.
	if (ExistingPanel->GetClass()->GetDefaultObject<UPanelWidget>()->CanHaveMultipleChildren() && bNewWidgetClassIsAPanel)
	{
		const bool bChildAllowed = WidgetClass->GetDefaultObject<UPanelWidget>()->CanHaveMultipleChildren() || ExistingPanel->GetChildrenCount() == 0;
		return bChildAllowed;
	}
	
	return true;
}

void FWidgetBlueprintEditorUtils::ReplaceWidgets(UWidgetBlueprint* BP, TSet<UWidget*> Widgets, UClass* WidgetClass, EReplaceWidgetNamingMethod NewWidgetNamingMethod)
{
	FScopedTransaction Transaction(LOCTEXT("ReplaceWidgets", "Replace Widgets"));

	TArray<UWidget*> UsedVariables;
	TArray<FText> WidgetNames;
	const bool bIncludeChildrenVariables = false;
	FindUsedVariablesForWidgets(Widgets, BP, UsedVariables, WidgetNames, bIncludeChildrenVariables);

	if (UsedVariables.Num() != 0 && !ShouldContinueReplaceOperation(BP, WidgetNames))
	{
		Transaction.Cancel();
		return;
	}

	TSharedPtr<FWidgetTemplateClass> TemplateClass = MakeShareable(new FWidgetTemplateClass(WidgetClass));	
	
	FWidgetBlueprintOperationUtils::ReplaceWidgetsWithTemplateClass(BP, Widgets, TemplateClass, NewWidgetNamingMethod);
}


void FWidgetBlueprintEditorUtils::CutWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	CopyWidgets(BP, Widgets);
	DeleteWidgets(BP, ResolveWidgetTemplates(Widgets), EDeleteWidgetWarningType::WarnAndAskUser);
}

void FWidgetBlueprintEditorUtils::CopyWidgets(UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	FString ExportedText = CopyWidgetsInternal(BP, Widgets);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

FString FWidgetBlueprintEditorUtils::CopyWidgetsInternal(UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	TSet<UWidget*> TemplateWidgets;

	// Convert the set of widget references into the list of widget templates we're going to copy.
	for ( const FWidgetReference& Widget : Widgets )
	{
		UWidget* TemplateWidget = Widget.GetTemplate();
		TemplateWidgets.Add(TemplateWidget);
	}

	TArray<UWidget*> FinalWidgets;

	// Pair down copied widgets to the legitimate root widgets, if they're parent is not already
	// in the set we're planning to copy, then keep them in the list, otherwise remove widgets that
	// will already be handled when their parent copies into the array.
	for ( UWidget* TemplateWidget : TemplateWidgets )
	{
		bool bFoundParent = false;

		// See if the widget already has a parent in the set we're copying.
		for ( UWidget* PossibleParent : TemplateWidgets )
		{
			if ( PossibleParent != TemplateWidget )
			{
				if ( TemplateWidget->IsChildOf(PossibleParent) )
				{
					bFoundParent = true;
					break;
				}
			}
		}

		if ( !bFoundParent )
		{
			FinalWidgets.Add(TemplateWidget);
			UWidgetTree::GetChildWidgets(TemplateWidget, FinalWidgets);
		}
	}

	FString ExportedText;
	FWidgetBlueprintEditorUtils::ExportWidgetsToText(FinalWidgets, /*out*/ ExportedText);
	return ExportedText;
}

TArray<UWidget*> FWidgetBlueprintEditorUtils::DuplicateWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets)
{
	TArray<UWidget*> DuplicatedWidgets;

	FWidgetReference ParentWidgetRef = Widgets.Num() > 0 ? *Widgets.CreateIterator() : FWidgetReference();
	FName SlotName = NAME_None;

	TOptional<FNamedSlotSelection> NamedSlotSelection = BlueprintEditor->GetSelectedNamedSlot();
	if (NamedSlotSelection.IsSet())
	{
		ParentWidgetRef = NamedSlotSelection->NamedSlotHostWidget;
		SlotName = NamedSlotSelection->SlotName;
	}

	if (ParentWidgetRef.IsValid())
	{
		FString ExportedText = CopyWidgetsInternal(BP, Widgets);

		FScopedTransaction Transaction(FGenericCommands::Get().Duplicate->GetDescription());
		bool TransactionSuccesful = true;
		DuplicatedWidgets = PasteWidgetsInternal(BlueprintEditor, BP, ExportedText, ParentWidgetRef, SlotName, FVector2D::ZeroVector, true, TransactionSuccesful);
		if (!TransactionSuccesful)
		{
			BlueprintEditor->LogSimpleMessage(LOCTEXT("PasteWidgetsCancel", "Paste operation on widget cancelled."));
			Transaction.Cancel();
		}
	}

	return DuplicatedWidgets;
}

UUserWidget* FWidgetBlueprintEditorUtils::CreateUserWidgetFromBlueprint(UObject* Outer, UWidgetBlueprint* BP, const FCreateWidgetFromBlueprintParams& Params)
{
	check(Outer);
	check(BP);

	UUserWidget* CreatedUserWidget = nullptr;

	// Create the Widget, we have to do special swapping out of the widget tree.
	{
		// Assign the outer to the game instance if it exists, otherwise use the world
		{
			FMakeClassSpawnableOnScope TemporarilySpawnable(BP->GeneratedClass);
			CreatedUserWidget = NewObject<UUserWidget>(Outer, BP->GeneratedClass);
		}

		// The preview widget should not be transactional.
		CreatedUserWidget->ClearFlags(RF_Transactional);

		// Establish the widget as being in design time before initializing and before duplication
        // (so that IsDesignTime is reliable within both calls to Initialize)
        // The preview widget is also the outer widget that will update all child flags
		CreatedUserWidget->SetDesignerFlags(Params.FlagsToApply);

		if (ULocalPlayer* Player = Params.LocalPlayer)
		{
			CreatedUserWidget->SetPlayerContext(FLocalPlayerContext(Player));
		}

		UWidgetTree* LatestWidgetTree = FWidgetBlueprintEditorUtils::FindLatestWidgetTree(BP, CreatedUserWidget);

		TMap<FName, UWidget*> SortedNamedSlotContentToMerge;
		UWidgetBlueprint* WidgetBlueprintIterator = BP;
		TArray<TTuple<FName, UWidget*>> NamedSlotContentToMergeArray;

		while (WidgetBlueprintIterator)
		{
			TArray<FName> SlotNames;
			WidgetBlueprintIterator->WidgetTree->GetSlotNames(SlotNames);

			// We iterate widget blueprints from child to parent, but we need the final namedslot array to be sorted from parent to child.
			// Here, we iterate the slot names in reverse to maintain the order of namedslots per widget blueprint once the final array is reversed.
			for (int32 Index = SlotNames.Num() - 1; Index >= 0; Index--)
			{
				FName SlotName = SlotNames[Index];
				if (UWidget* Content = WidgetBlueprintIterator->WidgetTree->GetContentForSlot(SlotName))
				{
					NamedSlotContentToMergeArray.Add(TTuple<FName, UWidget*>(SlotName, Content));
				}
			}

			WidgetBlueprintIterator = Cast<UWidgetBlueprint>(WidgetBlueprintIterator->GeneratedClass->GetSuperClass()->ClassGeneratedBy);
		}

		// We iterate the array in reverse so that the final SortedNamedSlotContentToMerge map ends up sorted from outermost namedslot to innermost.
		for (int32 Index = NamedSlotContentToMergeArray.Num() - 1; Index >= 0; Index--)
		{
			TTuple<FName, UWidget*>& Element = NamedSlotContentToMergeArray[Index];
			SortedNamedSlotContentToMerge.Add(Element.Key, Element.Value);
		}
		 
		// Update the widget tree directly to match the blueprint tree.  That way the preview can update
		// without needing to do a full recompile.
		CreatedUserWidget->DuplicateAndInitializeFromWidgetTree(LatestWidgetTree, SortedNamedSlotContentToMerge);

		// Establish the widget as being in design time before initializing (so that IsDesignTime is reliable within Initialize)
        // We have to call it to make sure that all the WidgetTree had the DesignerFlags set correctly
		CreatedUserWidget->SetDesignerFlags(Params.FlagsToApply);
	}

	return CreatedUserWidget;
}

void FWidgetBlueprintEditorUtils::DestroyUserWidget(UUserWidget* UserWidget)
{
	check(UserWidget);

	TWeakPtr<SWidget> SlateWidgetWeak = UserWidget->GetCachedWidget();

	UserWidget->MarkAsGarbage();
	UserWidget->ReleaseSlateResources(true);

	ensure(!SlateWidgetWeak.IsValid());
}

bool FWidgetBlueprintEditorUtils::IsAnySelectedWidgetLocked(TSet<FWidgetReference> SelectedWidgets)
{
	for (const FWidgetReference& Widget : SelectedWidgets)
	{
		if (Widget.GetPreview()->IsLockedInDesigner())
		{
			return true;
		}
	}
	return false;
}

bool FWidgetBlueprintEditorUtils::CanPasteWidgetsExtension(TSet<FWidgetReference> SelectedWidgets)
{
	if (!SelectedWidgets.IsEmpty())
	{
		IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		const TArrayView<const TSharedPtr<IClipboardExtension>> ClipboardExtensions = EditorModule.GetClipboardExtensibilityManager()->GetExtensions();

		for (const TSharedPtr<IClipboardExtension>& ClipboardExtension : ClipboardExtensions)
		{
			if (ensure(ClipboardExtension.IsValid()))
			{
				for (const FWidgetReference& SelectedWidget : SelectedWidgets)
				{
					if (UWidget* TemplateWidget = SelectedWidget.GetTemplate())
					{
						if (!ClipboardExtension->CanWidgetAcceptPaste(TemplateWidget))
						{
							return false;
						}
					}
				}
			}
		}
	}

	return true;
}

UWidget* FWidgetBlueprintEditorUtils::GetWidgetTemplateFromDragDrop(UWidgetBlueprint* Blueprint, UWidgetTree* RootWidgetTree, TSharedPtr<FDragDropOperation>& DragDropOp)
{
	UWidget* Widget = nullptr;

	if (!DragDropOp.IsValid())
	{
		return nullptr;
	}

	if (DragDropOp->IsOfType<FWidgetTemplateDragDropOp>())
	{
		TSharedPtr<FWidgetTemplateDragDropOp> TemplateDragDropOp = StaticCastSharedPtr<FWidgetTemplateDragDropOp>(DragDropOp);
		Widget = TemplateDragDropOp->Template->Create(RootWidgetTree);

		// Check to make sure that this widget can be added to the current blueprint
		if (Cast<UUserWidget>(Widget) && !Blueprint->IsWidgetFreeFromCircularReferences(Cast<UUserWidget>(Widget)))
		{
			Widget = nullptr;
		}
	}
	else if (DragDropOp->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(DragDropOp);
		if (AssetDragDropOp->GetAssets().Num() > 0)
		{
			// Only handle first valid dragged widget, multi widget drag drop is not practically useful
			const FAssetData& AssetData = AssetDragDropOp->GetAssets()[0];
			FText OutErrorMessage;
			return FWidgetBlueprintOperationUtils::CreateWidgetFromAsset(Blueprint, AssetData, RootWidgetTree, OutErrorMessage);
		}
	}

	return Widget;
}

bool FWidgetBlueprintEditorUtils::ShouldPreventDropOnTargetExtensions(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp, FText& OutFailureText)
{
	if (Target)
	{
		IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		const TArrayView<const TSharedPtr<IWidgetDragDropExtension>> DragDropExtensions = EditorModule.GetWidgetDragDropExtensibilityManager()->GetExtensions();

		for (const TSharedPtr<IWidgetDragDropExtension>& DragDropExtension : DragDropExtensions)
		{
			if (ensure(DragDropExtension.IsValid()) && DragDropExtension->ShouldPreventDropOnTarget(Target, DragDropOp))
			{
				OutFailureText = DragDropExtension->GetDropFailureText(Target, DragDropOp);
				return true;
			}
		}
	}

	return false;
}

void FWidgetBlueprintEditorUtils::ExportWidgetsToText(TArray<UWidget*> WidgetsToExport, /*out*/ FString& ExportedText)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;

	// Validate all nodes are from the same scope and set all UUserWidget::WidgetTrees (and things outered to it) to be ignored
	TArray<UObject*> WidgetsToIgnore;
	UObject* LastOuter = nullptr;
	for ( UWidget* Widget : WidgetsToExport )
	{
		// The nodes should all be from the same scope
		UObject* ThisOuter = Widget->GetOuter();
		check((LastOuter == ThisOuter) || (LastOuter == nullptr));
		LastOuter = ThisOuter;

		if ( UUserWidget* UserWidget = Cast<UUserWidget>(Widget) )
		{
			if ( UserWidget->WidgetTree )
			{
				WidgetsToIgnore.Add(UserWidget->WidgetTree);
				// FExportObjectInnerContext does not automatically ignore UObjects if their outer is ignored
				GetObjectsWithOuter(UserWidget->WidgetTree, WidgetsToIgnore);
			}
		}
	}

	const FExportObjectInnerContext Context(WidgetsToIgnore);

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	const TArrayView<const TSharedPtr<IClipboardExtension>> ClipboardExtensions = EditorModule.GetClipboardExtensibilityManager()->GetExtensions();

	// Get the widget blueprint containing the exported widgets
	UWidgetBlueprint* WidgetBlueprint = nullptr;
	if (WidgetsToExport.Num() > 0)
	{
		WidgetBlueprint = FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(WidgetsToExport[0]);
	}

	// Export each of the selected nodes
	for ( UWidget* Widget : WidgetsToExport )
	{

		UExporter::ExportToOutputDevice(&Context, Widget, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, LastOuter);

		// Check to see if this widget was content of another widget holding it in a named slot.
		if ( Widget->GetParent() == nullptr )
		{
			for ( UWidget* ExportableWidget : WidgetsToExport )
			{
				if ( INamedSlotInterface* NamedSlotContainer = Cast<INamedSlotInterface>(ExportableWidget) )
				{
					if ( NamedSlotContainer->ContainsContent(Widget) )
					{
						continue;
					}
				}
			}
		}

		if ( Widget->GetParent() == nullptr || !WidgetsToExport.Contains(Widget->GetParent()) )
		{
			auto SlotMetaData = NewObject<UWidgetSlotPair>();
			SlotMetaData->SetWidget(Widget);

			UExporter::ExportToOutputDevice(&Context, SlotMetaData, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, nullptr);
		}

		if (WidgetBlueprint)
		{
			for (const TSharedPtr<IClipboardExtension>& ClipboardExtension : ClipboardExtensions)
			{
				if (ClipboardExtension->CanAppendToClipboard(Widget))
				{
					IClipboardExtension::FExportArgs ExportArgs;
					ExportArgs.Context = &Context;
					ExportArgs.Exporter = nullptr;
					ExportArgs.FileType = TEXT("copy");
					ExportArgs.Indent = 0;
					ExportArgs.PortFlags = PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited;
					ExportArgs.bSelectedOnly = false;
					ExportArgs.ExportRootScope = nullptr;
					ExportArgs.Out = &Archive;
					ClipboardExtension->AppendToClipboard(Widget, ExportArgs);
				}
			}
		}
	}

	ExportedText = Archive;
}

TArray<UWidget*> FWidgetBlueprintEditorUtils::PasteWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference ParentWidgetRef, FName SlotName, FVector2D PasteLocation)
{
	FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	bool bTransactionSuccessful = true;
	TArray<UWidget*> PastedWidgets = PasteWidgetsInternal(BlueprintEditor, BP, TextToImport, ParentWidgetRef, SlotName, PasteLocation, false, bTransactionSuccessful);
	if (!bTransactionSuccessful)
	{
		BlueprintEditor->LogSimpleMessage(LOCTEXT("PasteWidgetsCancel", "Paste operation on widget cancelled."));
		Transaction.Cancel();
	}
	return PastedWidgets;
}

bool FWidgetBlueprintEditorUtils::DisplayPasteWarningAndEarlyExit()
{
	const FText DeleteConfirmationPrompt = LOCTEXT("DeleteConfirmationPrompt", "Pasting in a single-slot widget will erase its content. Do you wish to proceed?");
	const FText DeleteConfirmationTitle = LOCTEXT("DeleteConfirmationTitle", "Delete widget");

	// Warn the user that this may result in data loss
	FSuppressableWarningDialog::FSetupInfo Info(DeleteConfirmationPrompt, DeleteConfirmationTitle, TEXT("Paste_Warning"));
	Info.ConfirmText = LOCTEXT("DeleteConfirmation_Yes", "Yes");
	Info.CancelText = LOCTEXT("DeleteConfirmation_No", "No");

	FSuppressableWarningDialog DeleteChildWidgetWarningDialog(Info);
	return DeleteChildWidgetWarningDialog.ShowModal() == FSuppressableWarningDialog::Cancel;
}

TArray<UWidget*> FWidgetBlueprintEditorUtils::PasteWidgetsInternal(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, const FString& TextToImport, FWidgetReference ParentWidgetRef, FName SlotName, FVector2D PasteLocation, bool bForceSibling, bool& bTransactionSuccessful)
{
	// Do an intial text processing to make sure we have any widgets to paste
	UPackage* TempPackage = nullptr;
	FWidgetObjectTextFactory Factory = ProcessImportedText(BP, TextToImport, TempPackage);
	TGCObjectScopeGuard<UPackage> TempPackageGCGuard(TempPackage);
	const bool bHasPastedWidget = Factory.NewWidgetMap.Num() > 0;

	// Ignore an empty set of widget paste data.
	if (!bHasPastedWidget)
	{
		bTransactionSuccessful = false;
		return TArray<UWidget*>();
	}

	TArray<UWidget*> RootPasteWidgets;
	TMap<FName, UWidgetSlotPair*> PastedExtraSlotData;
	TSet<UWidget*> PastedWidgets;

	auto ImportWidgets = [&]()
	{
		FWidgetBlueprintEditorUtils::ImportWidgetsFromText(BP, TextToImport, /*out*/ PastedWidgets, /*out*/ PastedExtraSlotData);

		for (UWidget* NewWidget : PastedWidgets)
		{
			BP->OnVariableAdded(NewWidget->GetFName());
			// Widgets with a null parent mean that they were the root most widget of their selection set when
			// they were copied and thus we need to paste only the root most widgets.  All their children will be added
			// automatically.
			if (NewWidget->GetParent() == nullptr)
			{
				// Check to see if this widget is content of another widget holding it in a named slot.
				bool bIsNamedSlot = false;
				for (UWidget* ContainerWidget : PastedWidgets)
				{
					if (INamedSlotInterface* NamedSlotContainer = Cast<INamedSlotInterface>(ContainerWidget))
					{
						if (NamedSlotContainer->ContainsContent(NewWidget))
						{
							bIsNamedSlot = true;
							break;
						}
					}
				}

				// It's a Root widget only if it's not not in a named slot.
				if (!bIsNamedSlot)
				{
					RootPasteWidgets.Add(NewWidget);
				}
			}
		}
	};

	// If we're pasting into a content widget of the same type, treat it as a sibling duplication
	UWidget* FirstPastedWidget = Factory.NewWidgetMap.CreateIterator()->Value;
	if (FirstPastedWidget->IsA(UContentWidget::StaticClass()) &&
		ParentWidgetRef.IsValid() &&
		FirstPastedWidget->GetClass() == ParentWidgetRef.GetTemplate()->GetClass())
	{
		UPanelWidget* TargetParentWidget = ParentWidgetRef.GetTemplate()->GetParent();
		if (TargetParentWidget && TargetParentWidget->CanAddMoreChildren())
		{
			bForceSibling = true;
		}
	}

	if ( SlotName == NAME_None )
	{
		UPanelWidget* ParentWidget = nullptr;
		int32 IndexToInsert = INDEX_NONE;

		if ( ParentWidgetRef.IsValid() )
		{
			ParentWidget = Cast<UPanelWidget>(ParentWidgetRef.GetTemplate());

			// If the widget isn't a panel or we just really want it to be a sibling (ie. when duplicating), we'll try it's parent to see if the pasted widget can be a sibling (and get its index to insert at)
			if ( bForceSibling || !ParentWidget )
			{
				if (UWidget* WidgetTemplate = ParentWidgetRef.GetTemplate())
				{
					ParentWidget = WidgetTemplate->GetParent();
					if (ParentWidget && ParentWidget->CanHaveMultipleChildren())
					{
						IndexToInsert = ParentWidget->GetChildIndex(WidgetTemplate) + 1;
					}
				}
			}
		}

		if ( !ParentWidget )
		{
			// If we already have a root widget, then we can't replace the root.
			if ( BP->WidgetTree->RootWidget )
			{
				bTransactionSuccessful = false;
				return TArray<UWidget*>();
			}
		}

		if ( ParentWidget )
		{
			// If parent widget can only have one child and that slot is already occupied, we will remove its contents so the pasted widgets can be inserted in their place
			UWidget* ChildWidgetToDelete = nullptr;
			if (!ParentWidget->CanHaveMultipleChildren() && ParentWidget->GetChildrenCount() > 0)
			{
				// We do not Remove child if there is nothing to paste.
				if ( bHasPastedWidget )
				{
					if (FWidgetBlueprintEditorUtils::DisplayPasteWarningAndEarlyExit())
					{
						bTransactionSuccessful = false;
						return TArray<UWidget*>();
					}

					// Delete the singular child
					ChildWidgetToDelete = ParentWidget->GetAllChildren()[0];
					ChildWidgetToDelete->SetFlags(RF_Transactional);
					ChildWidgetToDelete->Modify();

					ParentWidget->SetFlags(RF_Transactional);
					ParentWidget->Modify();
					ParentWidget->RemoveChild(ChildWidgetToDelete);
				}
			}

			if (ChildWidgetToDelete)
			{
				DeleteWidgets(BP, { ChildWidgetToDelete }, EDeleteWidgetWarningType::DeleteSilently);
			}
		}

		ImportWidgets();

		// If there isn't a root widget and we're copying multiple root widgets, then we need to add a container root
		// to hold the pasted data since multiple root widgets isn't permitted.
		if (!ParentWidget && RootPasteWidgets.Num() > 1)
		{
			ParentWidget = BP->WidgetTree->ConstructWidget<UCanvasPanel>(UCanvasPanel::StaticClass());
			BP->WidgetTree->Modify();
			BP->WidgetTree->RootWidget = ParentWidget;
			BP->OnVariableAdded(ParentWidget->GetFName());
		}

		if (ParentWidget)
		{

			// A bit of a hack, but we can look at the widget's slot properties to determine if it is a canvas slot. If so, we'll try and maintain the relative positions
			bool bShouldReproduceOffsets = true;
			static const FName LayoutDataLabel = FName(TEXT("LayoutData"));
			for (TPair<FName, UWidgetSlotPair*>  SlotData : PastedExtraSlotData)
			{
				UWidgetSlotPair* SlotDataPair = SlotData.Value;
				TMap<FName, FString> SlotProperties;
				SlotDataPair->GetSlotProperties(SlotProperties);
				if (!SlotProperties.Contains(LayoutDataLabel))
				{
					bShouldReproduceOffsets = false;
					break;
				}
			}

			FVector2D FirstWidgetPosition;
			ParentWidget->Modify();
			for ( UWidget* NewWidget : RootPasteWidgets )
			{
				UPanelSlot* Slot;
				if ( IndexToInsert == INDEX_NONE )
				{
					Slot = ParentWidget->AddChild(NewWidget);
				}
				else
				{
					Slot = ParentWidget->InsertChildAt(IndexToInsert, NewWidget);
				}
				
				if ( Slot )
				{
					if ( UWidgetSlotPair* OldSlotData = PastedExtraSlotData.FindRef(NewWidget->GetFName()) )
					{
						TMap<FName, FString> OldSlotProperties;
						OldSlotData->GetSlotProperties(OldSlotProperties);
						FWidgetBlueprintEditorUtils::ImportPropertiesFromText(Slot, OldSlotProperties);

						// Cache the initial position of the first widget so we can calculate offsets for additional widgets
						if (NewWidget == RootPasteWidgets[0])
						{
							if (UCanvasPanelSlot* FirstCanvasSlot = Cast<UCanvasPanelSlot>(Slot))
							{
								FirstWidgetPosition = FirstCanvasSlot->GetPosition();
							}
						}
					}

					BlueprintEditor->RefreshPreview();
						
					FWidgetReference WidgetRef = BlueprintEditor->GetReferenceFromTemplate(NewWidget);

					UPanelSlot* PreviewSlot = WidgetRef.GetPreview()->Slot;
					UPanelSlot* TemplateSlot = WidgetRef.GetTemplate()->Slot;

					if ( UCanvasPanelSlot* CanvasSlot = Cast<UCanvasPanelSlot>(PreviewSlot) )
					{
						FVector2D PasteOffset = FVector2D(0, 0);
						if (bShouldReproduceOffsets)
						{
							PasteOffset = CanvasSlot->GetPosition()- FirstWidgetPosition;
						}

						if (UCanvasPanel* Canvas = Cast<UCanvasPanel>(CanvasSlot->Parent))
						{
							Canvas->TakeWidget(); // Generate the underlying widget so redoing the layout below works.
						}

						CanvasSlot->SaveBaseLayout();
						CanvasSlot->SetDesiredPosition(PasteLocation + PasteOffset);
						CanvasSlot->RebaseLayout();
					}

					TMap<FName, FString> SlotProperties;
					FWidgetBlueprintEditorUtils::ExportPropertiesToText(PreviewSlot, SlotProperties);
					FWidgetBlueprintEditorUtils::ImportPropertiesFromText(TemplateSlot, SlotProperties);
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
		}
		else
		{
			check(RootPasteWidgets.Num() == 1)
			// If we've arrived here, we must be creating the root widget from paste data, and there can only be
			// one item in the paste data by now.
			BP->WidgetTree->Modify();

			if (RootPasteWidgets.Num() > 0)
			{
				BP->WidgetTree->RootWidget = RootPasteWidgets[0];
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
		}
	}
	else
	{
		ImportWidgets();

		if ( RootPasteWidgets.Num() > 1 )
		{
			FNotificationInfo Info(LOCTEXT("NamedSlotsOnlyHoldOneWidget", "Can't paste content, a slot can only hold one widget at the root."));
			FSlateNotificationManager::Get().AddNotification(Info);

			bTransactionSuccessful = false;
			return TArray<UWidget*>();
		}

		BP->WidgetTree->Modify();

		// If there's a ParentWidgetRef, then we're pasting into a named slot of a widget in the tree.
		if (UWidget* NamedSlotHostWidget = ParentWidgetRef.GetTemplate())
		{
			NamedSlotHostWidget->SetFlags(RF_Transactional);
			NamedSlotHostWidget->Modify();

			INamedSlotInterface* NamedSlotInterface = Cast<INamedSlotInterface>(NamedSlotHostWidget);
			NamedSlotInterface->SetContentForSlot(SlotName, RootPasteWidgets[0]);
		}
		else
		{
			// If there's no ParentWidgetRef then we're pasting into the exposed named slots of the widget tree.
			// these are the slots that our parent class is exposing for use externally, but we can also override
			// them as a subclass.

			BP->WidgetTree->Modify();
			BP->WidgetTree->SetContentForSlot(SlotName, RootPasteWidgets[0]);
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
	}

	return RootPasteWidgets;
}

FWidgetObjectTextFactory FWidgetBlueprintEditorUtils::ProcessImportedText(UWidgetBlueprint* BP, const FString& TextToImport, /*out*/ UPackage*& TempPackage)
{
	// We create our own transient package here so that we can deserialize the data in isolation and ensure unreferenced
	// objects not part of the deserialization set are unresolved.
	TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/UMG/Editor/Transient"), RF_Transient);

	// Force the transient package to have the same namespace as the final widget blueprint package.
	// This ensures any text properties serialized from the buffer will be keyed correctly for the target package.
#if USE_STABLE_LOCALIZATION_KEYS
	{
		const FString PackageNamespace = TextNamespaceUtil::EnsurePackageNamespace(BP);
		if (!PackageNamespace.IsEmpty())
		{
			TextNamespaceUtil::ForcePackageNamespace(TempPackage, PackageNamespace);
		}
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	// Turn the text buffer into objects
	FWidgetObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
	return Factory;
}

void FWidgetBlueprintEditorUtils::ImportWidgetsFromText(UWidgetBlueprint* BP, const FString& TextToImport, /*out*/ TSet<UWidget*>& ImportedWidgetSet, /*out*/ TMap<FName, UWidgetSlotPair*>& PastedExtraSlotData)
{
	UPackage* TempPackage = nullptr;
	FWidgetObjectTextFactory Factory = ProcessImportedText(BP, TextToImport, TempPackage);
	TGCObjectScopeGuard<UPackage> TempPackageGCGuard(TempPackage);

	IUMGEditorModule& EditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	const TArrayView<const TSharedPtr<IClipboardExtension>> ClipboardExtensions = EditorModule.GetClipboardExtensibilityManager()->GetExtensions();
	for (const TSharedPtr<IClipboardExtension>& ClipboardExtension : ClipboardExtensions)
	{
		ClipboardExtension->ProcessImportedText(BP, TextToImport, TempPackage);
	}

	PastedExtraSlotData = Factory.MissingSlotData;

	for ( auto& Entry : Factory.NewWidgetMap )
	{
		UWidget* Widget = Entry.Value;

		ImportedWidgetSet.Add(Widget);

		Widget->SetFlags(RF_Transactional);

		// We don't export parent slot pointers, so each panel will need to point it's children back to itself
		UPanelWidget* PanelWidget = Cast<UPanelWidget>(Widget);
		if (PanelWidget)
		{
			TArray<UPanelSlot*> PanelSlots = PanelWidget->GetSlots();
			for (int32 i = 0; i < PanelWidget->GetChildrenCount(); i++)
			{
				UWidget* PanelChild = PanelWidget->GetChildAt(i);
				if (ensure(PanelChild))
				{
					PanelChild->Slot = PanelSlots[i];
				}
			}
		}

		// If there is an existing widget with the same name, rename the newly placed widget.
		FString WidgetOldName = Widget->GetName();
		FString NewName = FindNextValidName(BP->WidgetTree, WidgetOldName);
		if (NewName != WidgetOldName)
		{
			UWidgetSlotPair* SlotData = PastedExtraSlotData.FindRef(Widget->GetFName());
			if ( SlotData )
			{
				PastedExtraSlotData.Remove(Widget->GetFName());
			}
			Widget->Rename(*NewName, BP->WidgetTree);

			if (Widget->GetDisplayLabel().Equals(WidgetOldName))
			{
				Widget->SetDisplayLabel(Widget->GetName());
			}

			if ( SlotData )
			{
				SlotData->SetWidgetName(Widget->GetFName());
				PastedExtraSlotData.Add(Widget->GetFName(), SlotData);
			}
		}
		else
		{
			Widget->Rename(*WidgetOldName, BP->WidgetTree);
		}

		for (const TSharedPtr<IClipboardExtension>& Extension : ClipboardExtensions)
		{
			if (Extension->CanImportFromClipboard(Widget))
			{
				Extension->ImportDataToWidget(Widget, FName(WidgetOldName));
			}
		}
	}
}

void FWidgetBlueprintEditorUtils::ExportPropertiesToText(UObject* Object, TMap<FName, FString>& ExportedProperties)
{
	if ( Object )
	{
		static const TSet<FName> SpecialCaseProperties = { TEXT("bIsVariable") };
		for ( TFieldIterator<FProperty> PropertyIt(Object->GetClass()); PropertyIt && PropertyIt.GetStruct() != UVisual::StaticClass(); ++PropertyIt )
		{
			FProperty* Property = *PropertyIt;

			// Skip EditDefaultOnly, transient, and instanced properties, only include properties that the user can directly edit and some special cases
			if (!Property->HasAnyPropertyFlags(CPF_TextExportTransient | CPF_Transient | CPF_DuplicateTransient | CPF_DisableEditOnInstance) && (
					Property->HasAllPropertyFlags(CPF_Edit) ||
					Property->IsA<FMulticastDelegateProperty>() ||
					SpecialCaseProperties.Contains(Property->GetFName())))
			{
				FString ValueText;
				if (Property->ExportText_InContainer(0, ValueText, Object, Object, Object, PPF_Copy))
				{
					ExportedProperties.Add(Property->GetFName(), ValueText);
				}
			}
		}
	}
}

void FWidgetBlueprintEditorUtils::ImportPropertiesFromText(UObject* Object, const TMap<FName, FString>& ExportedProperties)
{
	if ( Object )
	{
		for ( const auto& Entry : ExportedProperties )
		{
			if ( FProperty* Property = FindFProperty<FProperty>(Object->GetClass(), Entry.Key) )
			{
				FEditPropertyChain PropertyChain;
				PropertyChain.AddHead(Property);
				Object->PreEditChange(PropertyChain);

				Property->ImportText_InContainer(*Entry.Value, Object, Object, PPF_Copy);

				FPropertyChangedEvent ChangedEvent(Property);
				Object->PostEditChangeProperty(ChangedEvent);
			}
		}
	}
}

bool FWidgetBlueprintEditorUtils::DoesClipboardTextContainWidget(UWidgetBlueprint* BP)
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	UPackage* TempPackage = nullptr;
	FWidgetObjectTextFactory Factory = ProcessImportedText(BP, TextToImport, TempPackage);
	return Factory.NewWidgetMap.Num() > 0;
}

bool FWidgetBlueprintEditorUtils::IsBindWidgetProperty(const FProperty* InProperty)
{
	bool bIsOptional;
	return IsBindWidgetProperty(InProperty, bIsOptional);
}

bool FWidgetBlueprintEditorUtils::IsBindWidgetProperty(const FProperty* InProperty, bool& bIsOptional)
{
	if ( InProperty )
	{
		bool bIsBindWidget = InProperty->HasMetaData("BindWidget") || InProperty->HasMetaData("BindWidgetOptional");
		bIsOptional = InProperty->HasMetaData("BindWidgetOptional") || ( InProperty->HasMetaData("OptionalWidget") || InProperty->GetBoolMetaData("OptionalWidget") );

		return bIsBindWidget;
	}

	return false;
}

bool FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(const FProperty* InProperty)
{
	bool bIsOptional;
	return IsBindWidgetAnimProperty(InProperty, bIsOptional);
}

bool FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(const FProperty* InProperty, bool& bIsOptional)
{
	if (InProperty)
	{
		bool bIsBindWidgetAnim = InProperty->HasMetaData("BindWidgetAnim") || InProperty->HasMetaData("BindWidgetAnimOptional");
		bIsOptional = InProperty->HasMetaData("BindWidgetAnimOptional");

		return bIsBindWidgetAnim;
	}

	return false;
}

namespace UE::UMG::Private
{
/** Helper class to perform path based filtering for unloaded BP's */
class FUnloadedBlueprintData : public IUnloadedBlueprintData
{
public:
	FUnloadedBlueprintData(const FAssetData& InAssetData)
		:ClassPath()
		, ClassFlags(CLASS_None)
		, bIsNormalBlueprintType(false)
	{
		ClassName = MakeShared<FString>(InAssetData.AssetName.ToString());

		FString GeneratedClassPath;
		const UClass* AssetClass = InAssetData.GetClass();
		if (AssetClass && AssetClass->IsChildOf(UBlueprintGeneratedClass::StaticClass()))
		{
			ClassPath = InAssetData.ToSoftObjectPath().GetAssetPathString();
		}
		else if (InAssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, GeneratedClassPath))
		{
			ClassPath = FTopLevelAssetPath(*FPackageName::ExportTextPathToObjectPath(GeneratedClassPath));
		}

		FEditorClassUtils::GetImplementedInterfaceClassPathsFromAsset(InAssetData, ImplementedInterfaces);
	}

	virtual ~FUnloadedBlueprintData()
	{
	}

	// Begin IUnloadedBlueprintData interface
	virtual bool HasAnyClassFlags(uint32 InFlagsToCheck) const
	{
		return (ClassFlags & InFlagsToCheck) != 0;
	}

	virtual bool HasAllClassFlags(uint32 InFlagsToCheck) const
	{
		return ((ClassFlags & InFlagsToCheck) == InFlagsToCheck);
	}

	virtual void SetClassFlags(uint32 InFlags)
	{
		ClassFlags = InFlags;
	}

	virtual bool ImplementsInterface(const UClass* InInterface) const
	{
		FString InterfacePath = InInterface->GetPathName();
		for (const FString& ImplementedInterface : ImplementedInterfaces)
		{
			if (ImplementedInterface == InterfacePath)
			{
				return true;
			}
		}

		return false;
	}

	virtual bool IsChildOf(const UClass* InClass) const
	{
		return false;
	}

	virtual bool IsA(const UClass* InClass) const
	{
		// Unloaded blueprint classes should always be a BPGC, so this just checks against the expected type.
		return UBlueprintGeneratedClass::StaticClass()->UObject::IsA(InClass);
	}

	virtual const UClass* GetClassWithin() const
	{
		return nullptr;
	}

	virtual const UClass* GetNativeParent() const
	{
		return nullptr;
	}

	virtual void SetNormalBlueprintType(bool bInNormalBPType)
	{
		bIsNormalBlueprintType = bInNormalBPType;
	}

	virtual bool IsNormalBlueprintType() const
	{
		return bIsNormalBlueprintType;
	}

	virtual TSharedPtr<FString> GetClassName() const
	{
		return ClassName;
	}

	virtual FTopLevelAssetPath GetClassPathName() const
	{
		return ClassPath;
	}
	// End IUnloadedBlueprintData interface

private:
	TSharedPtr<FString> ClassName;
	FTopLevelAssetPath ClassPath;
	uint32 ClassFlags;
	TArray<FString> ImplementedInterfaces;
	bool bIsNormalBlueprintType;
};

bool IsUsableWidgetClass(const FString& WidgetPathName, const FAssetData& WidgetAssetData, FName Category, const UClass* WidgetClass, TSharedRef<FWidgetBlueprintEditor> InCurrentActiveBlueprintEditor)
{
	const UWidgetEditingProjectSettings* UMGEditorProjectSettings = FWidgetBlueprintEditorUtils::GetRelevantSettings(InCurrentActiveBlueprintEditor);

	// Excludes engine content if user sets it to false
	if (!UMGEditorProjectSettings->bShowWidgetsFromEngineContent)
	{
		if (WidgetPathName.StartsWith(TEXT("/Engine")))
		{
			return false;
		}
	}

	// Excludes developer content if user sets it to false
	if (!UMGEditorProjectSettings->bShowWidgetsFromDeveloperContent)
	{
		if (WidgetPathName.StartsWith(TEXT("/Game/Developers")))
		{
			return false;
		}
	}

	UWidgetBlueprint* WidgetBP = InCurrentActiveBlueprintEditor->GetWidgetBlueprintObj();
	const bool bAllowEditorWidget = WidgetBP ? WidgetBP->AllowEditorWidget() : false;
	if (!bAllowEditorWidget)
	{
		if (WidgetClass && IsEditorOnlyObject(WidgetClass))
		{
			return false;
		}
		else if (WidgetAssetData.IsValid())
		{
			// should not load since the default for GetClass is EResolveClass::No
			const UClass* AssetClass = WidgetAssetData.GetClass();
			if (AssetClass && IsEditorOnlyObject(AssetClass))
			{
				return false;
			}
			
		}
	}

	if (UMGEditorProjectSettings->bUseEditorConfigPaletteFiltering)
	{
		FClassViewerModule* ClassViewerModule = FModuleManager::GetModulePtr<FClassViewerModule>("ClassViewer");
		const TSharedPtr<IClassViewerFilter> GlobalClassFilter = ClassViewerModule ? ClassViewerModule->GetGlobalClassViewerFilter() : TSharedPtr<IClassViewerFilter>();
		if (UMGEditorProjectSettings->GetAllowedPaletteCategories().PassesFilter(Category) && GlobalClassFilter.IsValid())
		{
			if (WidgetClass)
			{
				return GlobalClassFilter->IsClassAllowed(FClassViewerInitializationOptions(), WidgetClass, ClassViewerModule->CreateFilterFuncs());
			}
			else if (WidgetAssetData.IsValid())
			{
				TSharedRef<FUnloadedBlueprintData> UnloadedBlueprint = MakeShared<FUnloadedBlueprintData>(WidgetAssetData);
				return GlobalClassFilter->IsUnloadedClassAllowed(FClassViewerInitializationOptions(), UnloadedBlueprint, ClassViewerModule->CreateFilterFuncs());
			}
		}

		auto IsPathUnderMountPoints = [](FStringView Path)
			{
				static const FString EnginePath = TEXT("Engine");
				static const FString GamePath = TEXT("Game");

				const TSet<FString>& MountPoints = IPluginManager::Get().GetBuiltInPluginNames();
				if (MountPoints.Num() > 0)
				{
					const FStringView MountPoint = FPathViews::GetMountPointNameFromPath(Path);
					return MountPoints.ContainsByHash(GetTypeHash(MountPoint), MountPoint)
						|| MountPoint.Equals(EnginePath, ESearchCase::IgnoreCase)
						|| MountPoint.Equals(GamePath, ESearchCase::IgnoreCase);
				}
				return false;
			};

		const bool bPassesAllowedPalletteFilter = UMGEditorProjectSettings->GetAllowedPaletteWidgets().PassesFilter(WidgetPathName);
		if (FPackageName::IsScriptPackage(WidgetPathName))
		{
			return bPassesAllowedPalletteFilter;
		}

		const bool bPathUnderMountPoints = IsPathUnderMountPoints(WidgetPathName);
		if (bPathUnderMountPoints && !bPassesAllowedPalletteFilter)
		{
			return false;
		}

		return true;
	}
	else
	{
		// Excludes this widget if it is on the hide list
		for (const FSoftClassPath& WidgetClassToHide : UMGEditorProjectSettings->WidgetClassesToHide)
		{
			if (WidgetPathName.Find(WidgetClassToHide.ToString()) == 0)
			{
				return false;
			}
		}
	}
	return true;
}
}

bool FWidgetBlueprintEditorUtils::IsUsableWidgetClass(const UClass* WidgetClass)
{
	return false;
}

TValueOrError<FWidgetBlueprintEditorUtils::FUsableWidgetClassResult, void> FWidgetBlueprintEditorUtils::IsUsableWidgetClass(const FAssetData& WidgetAsset)
{
	return MakeError();
}

bool FWidgetBlueprintEditorUtils::IsUsableWidgetClass(const UClass* WidgetClass, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor)
{
	if (WidgetClass->IsChildOf(UWidget::StaticClass()))
	{
		// We aren't interested in classes that are experimental or cannot be instantiated
		bool bIsExperimental, bIsEarlyAccess;
		FString MostDerivedDevelopmentClassName;
		FObjectEditorUtils::GetClassDevelopmentStatus(const_cast<UClass*>(WidgetClass), bIsExperimental, bIsEarlyAccess, MostDerivedDevelopmentClassName);
		const bool bIsInvalid = WidgetClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists);
		if (bIsExperimental || bIsEarlyAccess || bIsInvalid)
		{
			return false;
		}

		// Don't include skeleton classes or the same class as the widget being edited
		const bool bIsSkeletonClass = WidgetClass->HasAnyFlags(RF_Transient) && WidgetClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint);

		// Check that the asset that generated this class is valid (necessary b/c of a larger issue wherein force delete does not wipe the generated class object)
		if (bIsSkeletonClass)
		{
			return false;
		}

		return UE::UMG::Private::IsUsableWidgetClass(WidgetClass->GetPathName(), FAssetData(), *WidgetClass->GetDefaultObject<UWidget>()->GetPaletteCategory().ToString(), WidgetClass, BlueprintEditor);
	}

	return false;
}

TValueOrError<FWidgetBlueprintEditorUtils::FUsableWidgetClassResult, void> FWidgetBlueprintEditorUtils::IsUsableWidgetClass(const FAssetData& WidgetAsset, TSharedRef<FWidgetBlueprintEditor> InCurrentActiveBlueprintEditor)
{
	if (const UClass* WidgetAssetClass = WidgetAsset.GetClass(EResolveClass::No))
	{
		if (IsUsableWidgetClass(WidgetAssetClass, InCurrentActiveBlueprintEditor))
		{
			FWidgetBlueprintEditorUtils::FUsableWidgetClassResult Result;
			Result.NativeParentClass = WidgetAssetClass;
			Result.AssetClassFlags = WidgetAssetClass->GetClassFlags();
			return MakeValue(Result);
		}
	}

	// Blueprints get the class type actions for their parent native class - this avoids us having to load the blueprint
	UClass* NativeParentClass = nullptr;
	FString NativeParentClassName;
	WidgetAsset.GetTagValue(FBlueprintTags::NativeParentClassPath, NativeParentClassName);
	if (NativeParentClassName.IsEmpty())
	{
		return MakeError();
	}
	else
	{
		const FString NativeParentClassPath = FPackageName::ExportTextPathToObjectPath(NativeParentClassName);
		if (NativeParentClassPath.StartsWith(TEXT("/")))
		{
			// Metadata may be pointing to classes that no longer exist, so check for redirectors first
			const FString RedirectedClassPath = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(NativeParentClassPath)).ToString();
			NativeParentClass = UClass::TryFindTypeSlow<UClass>(RedirectedClassPath);
		}
		if (NativeParentClass == nullptr)
		{
			return MakeError();
		}
		if (!NativeParentClass->IsChildOf(UWidget::StaticClass()))
		{
			return MakeError();
		}
	}

	EClassFlags BPFlags = static_cast<EClassFlags>(WidgetAsset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags));
	const bool bIsInvalid = (BPFlags & (CLASS_Deprecated | CLASS_Abstract | CLASS_NewerVersionExists)) != 0;
	if (bIsInvalid)
	{
		return MakeError();
	}

	const FName CategoryName = *GetPaletteCategory(WidgetAsset, TSubclassOf<UWidget>(NativeParentClass)).ToString();
	if (UE::UMG::Private::IsUsableWidgetClass(WidgetAsset.GetObjectPathString(), WidgetAsset, CategoryName, nullptr, InCurrentActiveBlueprintEditor))
	{
		FWidgetBlueprintEditorUtils::FUsableWidgetClassResult Result;
		Result.NativeParentClass = NativeParentClass;
		Result.AssetClassFlags = BPFlags;
		return MakeValue(Result);
	}
	return MakeError();
}

FString RemoveSuffixFromName(const FString OldName)
{
	int NameLen = OldName.Len();
	int SuffixIndex = 0;
	if (OldName.FindLastChar('_', SuffixIndex))
	{
		NameLen = SuffixIndex;
		for (int32 i = SuffixIndex + 1; i < OldName.Len(); ++i)
		{
			const TCHAR& C = OldName[i];
			const bool bGoodChar = ((C >= '0') && (C <= '9'));
			if (!bGoodChar)
			{
				return OldName;
			}
		}
	}
	return FString::ConstructFromPtrSize(*OldName, NameLen);
}

FString FWidgetBlueprintEditorUtils::FindNextValidName(UWidgetTree* WidgetTree, const FString& Name)
{
	// If the name of the widget is not already used, we use it.	
	if (FindObject<UObject>(WidgetTree, *Name))
	{
		// If the name is already used, we will suffix it with '_X'
		FString NameWithoutSuffix = RemoveSuffixFromName(Name);
		FString NewName = NameWithoutSuffix;

		int32 Postfix = 0;
		while (FindObject<UObject>(WidgetTree, *NewName))
		{
			++Postfix;
			NewName = FString::Printf(TEXT("%s_%d"), *NameWithoutSuffix, Postfix);
		}

		return NewName;
	}
	return Name;
}

UWidgetTree* FWidgetBlueprintEditorUtils::FindLatestWidgetTree(UWidgetBlueprint* Blueprint, UUserWidget* UserWidget)
{
	UWidgetTree* LatestWidgetTree = Blueprint->WidgetTree;

	// If there is no RootWidget, we look for a WidgetTree in the parents classes until we find one.
	if (LatestWidgetTree->RootWidget == nullptr)
	{
		UWidgetBlueprintGeneratedClass* BGClass = UserWidget->GetWidgetTreeOwningClass();
		// If we find a class that owns the widget tree, just make sure it's not our current class, that would imply we've removed all the widgets
		// from this current tree, and if we use this classes compiled tree it's going to be the outdated old version.
		if (BGClass && BGClass != Blueprint->GeneratedClass)
		{
			LatestWidgetTree = BGClass->GetWidgetTreeArchetype();
		}
	}
	return LatestWidgetTree;
}

int32 FWidgetBlueprintEditorUtils::UpdateHittestGrid(FHittestGrid& HitTestGrid, TSharedRef<SWindow> Window, float Scale, FVector2D DrawSize, float DeltaTime)
{
	FSlateApplication::Get().InvalidateAllWidgets(false);

	const FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize * (1.f / Scale), FSlateLayoutTransform(Scale));
	const FSlateRect WindowClipRect = WindowGeometry.GetLayoutBoundingRect();
	FPaintArgs PaintArgs(nullptr, HitTestGrid, FVector2D::ZeroVector, FApp::GetCurrentTime(), DeltaTime);

	FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
	FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());

	Window->SlatePrepass(WindowGeometry.Scale);
	PaintArgs.GetHittestGrid().SetHittestArea(WindowClipRect.GetTopLeft(), WindowClipRect.GetSize());
	PaintArgs.GetHittestGrid().Clear();

	// Get the free buffer & add our virtual window
	bool bUseGammaSpace = false;
	TSharedPtr<ISlate3DRenderer, ESPMode::ThreadSafe> Renderer = FModuleManager::Get().LoadModuleChecked<ISlateRHIRendererModule>("SlateRHIRenderer")
		.CreateSlate3DRenderer(bUseGammaSpace);

	int32 MaxLayerId = 0;
	{
		ISlate3DRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *Renderer };
		FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(Window);

		MaxLayerId = Window->Paint(
			PaintArgs,
			WindowGeometry, WindowClipRect,
			WindowElementList,
			0,
			FWidgetStyle(),
			Window->IsEnabled());
	}

	FSlateApplication::Get().InvalidateAllWidgets(false);

	return MaxLayerId;
}

TTuple<FVector2D, FVector2D> FWidgetBlueprintEditorUtils::GetWidgetPreviewAreaAndSize(UUserWidget* UserWidget, FVector2D DesiredSize, FVector2D PreviewSize, EDesignPreviewSizeMode SizeMode, TOptional<FVector2D> ThumbnailCustomSize)
{
	FVector2D Size(PreviewSize.X, PreviewSize.Y);
	FVector2D Area(PreviewSize.X, PreviewSize.Y);

	if (UserWidget)
	{

		switch (SizeMode)
		{
		case EDesignPreviewSizeMode::Custom:
			Area = ThumbnailCustomSize.IsSet()? ThumbnailCustomSize.GetValue() : UserWidget->DesignTimeSize;
			// If the custom size is 0 in some dimension, use the desired size instead.
			if (Area.X == 0)
			{
				Area.X = DesiredSize.X;
			}
			if (Area.Y == 0)
			{
				Area.Y = DesiredSize.Y;
			}
			Size = Area;
			break;
		case EDesignPreviewSizeMode::CustomOnScreen:
			Size = ThumbnailCustomSize.IsSet() ? ThumbnailCustomSize.GetValue() : UserWidget->DesignTimeSize;

			// If the custom size is 0 in some dimension, use the desired size instead.
			if (Size.X == 0)
			{
				Size.X = DesiredSize.X;
			}
			if (Size.Y == 0)
			{
				Size.Y = DesiredSize.Y;
			}
			return TTuple<FVector2D, FVector2D>(Area, Size);
		case EDesignPreviewSizeMode::Desired:
			Area = DesiredSize;
			// Fall through to DesiredOnScreen
		case EDesignPreviewSizeMode::DesiredOnScreen:
			Size = DesiredSize;
			return TTuple<FVector2D, FVector2D>(Area, Size);
		case EDesignPreviewSizeMode::FillScreen:
			break;
		}
	}
	return TTuple<FVector2D, FVector2D>(Area, Size);
}

float FWidgetBlueprintEditorUtils::GetWidgetPreviewDPIScale(UUserWidget* UserWidget, FVector2D PreviewSize)
{
	// If the user is using a custom size then we disable the DPI scaling logic.
	if (UserWidget)
	{
		if (UserWidget->DesignSizeMode == EDesignPreviewSizeMode::Custom ||
			UserWidget->DesignSizeMode == EDesignPreviewSizeMode::Desired)
		{
			return 1.0f;
		}
	}

	return GetDefault<UUserInterfaceSettings>()->GetDPIScaleBasedOnSize(FIntPoint(FMath::TruncToInt32(PreviewSize.X), FMath::TruncToInt32(PreviewSize.Y)));
}


FVector2D FWidgetBlueprintEditorUtils::GetWidgetPreviewUnScaledCustomSize(FVector2D DesiredSize, UUserWidget* UserWidget, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode)
{

	checkf(DesiredSize.X > 0.f && DesiredSize.Y > 0.f, TEXT("The size should have been previously checked to be > 0."));

	FVector2D FinalSize(0.f,0.f);
	int32 PreviewWidth;
	const TCHAR* ConfigSectionName = TEXT("UMGEditor.Designer");
	GConfig->GetInt(ConfigSectionName, TEXT("PreviewWidth"), PreviewWidth, GEditorPerProjectIni);
	int32 PreviewHeight;
	GConfig->GetInt(ConfigSectionName, TEXT("PreviewHeight"), PreviewHeight, GEditorPerProjectIni);

	FVector2D PreviewSize(PreviewWidth, PreviewHeight);

	TTuple<FVector2D, FVector2D> AreaAndSize = GetWidgetPreviewAreaAndSize(UserWidget, DesiredSize, PreviewSize, ConvertThumbnailSizeModeToDesignerSizeMode(ThumbnailSizeMode, UserWidget), ThumbnailCustomSize.IsSet() ? ThumbnailCustomSize.GetValue() : TOptional<FVector2D>());

	float DPIScale;
	if (ThumbnailCustomSize.IsSet())
	{
		DPIScale = 1.0f;
	}
	else 
	{
		DPIScale = GetWidgetPreviewDPIScale(UserWidget, PreviewSize);

	}

	if (ensure(DPIScale > 0.f))
	{
		FinalSize = AreaAndSize.Get<1>() / DPIScale;
	}

	return FinalSize;

}

EDesignPreviewSizeMode FWidgetBlueprintEditorUtils::ConvertThumbnailSizeModeToDesignerSizeMode(EThumbnailPreviewSizeMode ThumbnailSizeMode, UUserWidget* WidgetInstance)
{
	switch (ThumbnailSizeMode)
	{
	case EThumbnailPreviewSizeMode::MatchDesignerMode:
		return WidgetInstance->DesignSizeMode;
	case EThumbnailPreviewSizeMode::FillScreen:
		return EDesignPreviewSizeMode::FillScreen;
	case EThumbnailPreviewSizeMode::Custom:
		return EDesignPreviewSizeMode::Custom;
	case EThumbnailPreviewSizeMode::Desired:
		return EDesignPreviewSizeMode::Desired;
	default:
		return EDesignPreviewSizeMode::Desired;
	}
}

TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTarget(UUserWidget* WidgetInstance, UTextureRenderTarget2D* RenderTarget2D)
{
	return DrawSWidgetInRenderTargetInternal(WidgetInstance, nullptr, RenderTarget2D, FVector2D(256.f,256.f), false, TOptional<FVector2D>(), EThumbnailPreviewSizeMode::MatchDesignerMode);
}

UWidgetEditingProjectSettings* FWidgetBlueprintEditorUtils::GetRelevantMutableSettings(TWeakPtr<FWidgetBlueprintEditor> CurrentEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> PinnedEditor = CurrentEditor.Pin())
	{
		if (UWidgetBlueprint* WidgetBP = PinnedEditor->GetWidgetBlueprintObj())
		{
			return WidgetBP->GetRelevantSettings();
		}
	}
	// Fall back to the UMG Editor settings as default
	return GetMutableDefault<UUMGEditorProjectSettings>();
}

const UWidgetEditingProjectSettings* FWidgetBlueprintEditorUtils::GetRelevantSettings(TWeakPtr<FWidgetBlueprintEditor> CurrentEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> PinnedEditor = CurrentEditor.Pin())
	{
		if (UWidgetBlueprint* WidgetBP = PinnedEditor->GetWidgetBlueprintObj())
		{
			return WidgetBP->GetRelevantSettings();
		}
	}
	// Fall back to the UMG Editor settings as default
	return GetDefault<UUMGEditorProjectSettings>();
}

TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetForThumbnail(UUserWidget* WidgetInstance, FRenderTarget* RenderTarget2D, FVector2D ThumbnailSize, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode)
{
	return DrawSWidgetInRenderTargetInternal(WidgetInstance, RenderTarget2D, nullptr,ThumbnailSize, true, ThumbnailCustomSize, ThumbnailSizeMode);
}

TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetForThumbnail(UUserWidget* WidgetInstance, UTextureRenderTarget2D* RenderTarget2D, FVector2D ThumbnailSize, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode)
{
	return DrawSWidgetInRenderTargetInternal(WidgetInstance, nullptr, RenderTarget2D,ThumbnailSize, true, ThumbnailCustomSize, ThumbnailSizeMode);
}

TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>  FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetInternal(UUserWidget* WidgetInstance, FRenderTarget* RenderTarget2D, UTextureRenderTarget2D* TextureRenderTarget,FVector2D ThumbnailSize, bool bIsForThumbnail, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode)
{
	if (TextureRenderTarget == nullptr && RenderTarget2D == nullptr)
	{
		return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>(); 
	}
	//Create Window
	FVector2D Offset(0.f, 0.f);
	FVector2D ScaledSize(0.f, 0.f);
	TSharedPtr<SWidget> WindowContent = WidgetInstance->TakeWidget();

	if (!WindowContent)
	{
		return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>();
	}

	TSharedRef<SVirtualWindow> Window = SNew(SVirtualWindow);
	TUniquePtr<FHittestGrid> HitTestGrid = MakeUnique<FHittestGrid>();
	Window->SetContent(WindowContent.ToSharedRef());
	Window->Resize(ThumbnailSize);

	// Store the desired size to maintain the aspect ratio later
	FGeometry WindowGeometry = FGeometry::MakeRoot(ThumbnailSize, FSlateLayoutTransform(1.0f));
	Window->SlatePrepass(1.0f);
	FVector2D DesiredSizeWindow = Window->GetDesiredSize();
	
	if (DesiredSizeWindow.X < SMALL_NUMBER || DesiredSizeWindow.Y < SMALL_NUMBER)
	{
		return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>();
	}

	FVector2D UnscaledSize = FWidgetBlueprintEditorUtils::GetWidgetPreviewUnScaledCustomSize(DesiredSizeWindow, WidgetInstance, ThumbnailCustomSize, ThumbnailSizeMode);
	if (UnscaledSize.X < SMALL_NUMBER || UnscaledSize.Y < SMALL_NUMBER)
	{
		return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>();
	}

	float Scale = 1.f;
	// Change some configuration if it is for thumbnail creation
	if (bIsForThumbnail)
	{
		TTuple<float, FVector2D> ScaleAndOffset = FWidgetBlueprintEditorUtils::GetThumbnailImageScaleAndOffset(UnscaledSize, ThumbnailSize);
		Scale = ScaleAndOffset.Get<0>();
		Offset = ScaleAndOffset.Get<1>();
	}

	ScaledSize = UnscaledSize * Scale;
	if (ScaledSize.X < 1.f || ScaledSize.Y < 1.f)
	{
		return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>();
	}

	// Create Renderer Target and WidgetRenderer
	bool bApplyGammaCorrection = RenderTarget2D? true : false;
	FWidgetRenderer* WidgetRenderer = new FWidgetRenderer(bApplyGammaCorrection);

	if (!bIsForThumbnail)
	{
		WidgetRenderer->SetIsPrepassNeeded(false);
	}

	if (TextureRenderTarget)
	{
		TextureRenderTarget->Filter = TF_Bilinear;
		TextureRenderTarget->ClearColor = FLinearColor::Transparent;
		TextureRenderTarget->SRGB = true;
		TextureRenderTarget->RenderTargetFormat = RTF_RGBA8;

		uint32 ScaledSizeX = static_cast<uint32>(ScaledSize.X);
		uint32 ScaledSizeY = static_cast<uint32>(ScaledSize.Y);

		const bool bForceLinearGamma = false;
		const EPixelFormat RequestedFormat = FSlateApplication::Get().GetRenderer()->GetSlateRecommendedColorFormat();
		TextureRenderTarget->InitCustomFormat(ScaledSizeX, ScaledSizeY, RequestedFormat, bForceLinearGamma);
		WidgetRenderer->DrawWindow(TextureRenderTarget, *HitTestGrid, Window, Scale, ScaledSize, 0.1f);
	}
	else
	{
		ensure(RenderTarget2D != nullptr);
		WidgetRenderer->SetShouldClearTarget(false);
		WidgetRenderer->ViewOffset = Offset;
		WidgetRenderer->DrawWindow(RenderTarget2D, *HitTestGrid, Window, Scale, ScaledSize, 0.1f);
	}

	if (WidgetRenderer)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}

	return TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties>(FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties{ ScaledSize,Offset });
}

TTuple<float, FVector2D> FWidgetBlueprintEditorUtils::GetThumbnailImageScaleAndOffset(FVector2D WidgetSize, FVector2D ThumbnailSize)
{
	// Scale the widget blueprint image to fit in the thumbnail

	checkf(WidgetSize.X > 0.f && WidgetSize.Y > 0.f, TEXT("The size should have been previously checked to be > 0."));

	float Scale;
	double XOffset = 0;
	double YOffset = 0;
	if (WidgetSize.X > WidgetSize.Y)
	{
		Scale = static_cast<float>(ThumbnailSize.X / WidgetSize.X);
		WidgetSize *= Scale;
		YOffset = (ThumbnailSize.Y - WidgetSize.Y) / 2.f;
	}
	else
	{
		Scale = static_cast<float>(ThumbnailSize.Y / WidgetSize.Y);
		WidgetSize *= Scale;
		XOffset = (ThumbnailSize.X - WidgetSize.X) / 2.f;
	}

	return TTuple<float, FVector2D>(Scale, FVector2D(XOffset, YOffset));
}

void FWidgetBlueprintEditorUtils::SetTextureAsAssetThumbnail(UWidgetBlueprint* WidgetBlueprint, UTexture2D* ThumbnailTexture)
{
	const TCHAR* ThumbnailName = TEXT("Thumbnail");
	UTexture2D* ExistingThumbnail = FindObject<UTexture2D>(WidgetBlueprint, ThumbnailName, EFindObjectFlags::None);
	if (ExistingThumbnail)
	{
		ExistingThumbnail->Rename(nullptr, GetTransientPackage());
	}
	if (!ThumbnailTexture)
	{
		WidgetBlueprint->ThumbnailImage = nullptr;
		return;
	}
	FVector2D TextureSize(ThumbnailTexture->GetSizeX(), ThumbnailTexture->GetSizeY());
	if (TextureSize.X < SMALL_NUMBER || TextureSize.Y < SMALL_NUMBER)
	{
		WidgetBlueprint->ThumbnailImage = nullptr;
	}
	else
	{
		ThumbnailTexture->Rename(ThumbnailName, WidgetBlueprint, REN_NonTransactional | REN_DontCreateRedirectors);
		WidgetBlueprint->ThumbnailImage = ThumbnailTexture;
	}
}

FText FWidgetBlueprintEditorUtils::GetPaletteCategory(const TSubclassOf<UWidget> WidgetClass)
{
	if (WidgetClass.Get())
	{
		return WidgetClass.GetDefaultObject()->GetPaletteCategory();
	}
	return GetMutableDefault<UWidget>()->GetPaletteCategory();
}

FText FWidgetBlueprintEditorUtils::GetPaletteCategory(const FAssetData& WidgetAsset, const TSubclassOf<UWidget> NativeClass)
{
	//The asset can be a UBlueprint, UBlueprintGeneratedClass, a UWidgetBlueprint or a UWidgetBlueprintGeneratedClass

	if (UClass* WidgetAssetClass = WidgetAsset.GetClass(EResolveClass::No))
	{
		if (WidgetAssetClass->IsChildOf(UWidget::StaticClass()))
		{
			return GetPaletteCategory(TSubclassOf<UWidget>(WidgetAssetClass));
		}
	}

	//If the blueprint is unloaded we need to extract it from the asset metadata.
	FText FoundPaletteCategoryText = WidgetAsset.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UWidgetBlueprint, PaletteCategory));
	if (!FoundPaletteCategoryText.IsEmpty())
	{
		return FoundPaletteCategoryText;
	}
	else if (NativeClass.Get() != nullptr && NativeClass->IsChildOf(UWidget::StaticClass()) && !NativeClass->IsChildOf(UUserWidget::StaticClass()))
	{
		return NativeClass.GetDefaultObject()->GetPaletteCategory();
	}

	static const FTopLevelAssetPath BlueprintGeneratedClassAssetPath = UWidgetBlueprintGeneratedClass::StaticClass()->GetClassPathName();
	static const FTopLevelAssetPath WidgetBlueprintAssetPath = UWidgetBlueprint::StaticClass()->GetClassPathName();
	if (WidgetAsset.AssetClassPath == BlueprintGeneratedClassAssetPath || WidgetAsset.AssetClassPath == WidgetBlueprintAssetPath)
	{
		return GetMutableDefault<UUserWidget>()->GetPaletteCategory();
	}
	else
	{
		return GetMutableDefault<UWidget>()->GetPaletteCategory();
	}
}

UWidgetBlueprint* FWidgetBlueprintEditorUtils::GetWidgetBlueprintFromWidget(const UWidget* Widget)
{
	if (Widget)
	{
		if (UObject* WidgetTree = Widget->GetOuter())
		{
			UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(WidgetTree->GetOuter());
			if (WidgetBlueprint)
			{
				return WidgetBlueprint;
			}
			else if (WidgetTree->GetOuter())
			{
				WidgetBlueprint = Cast<UWidgetBlueprint>(WidgetTree->GetOuter()->GetClass()->ClassGeneratedBy);
				if (WidgetBlueprint)
				{
					return WidgetBlueprint;
				}
			}
		}
	}
	return nullptr;
}

TSet<UWidget*> FWidgetBlueprintEditorUtils::ResolveWidgetTemplates(const TSet<FWidgetReference>& Widgets)
{
	TSet<UWidget*> Templates;
	Algo::TransformIf(Widgets, Templates,
		[&](const FWidgetReference& Widget)
		{
			return Widget.GetTemplate() != nullptr;
		},
		[&](const FWidgetReference& Widget)
		{
			return Widget.GetTemplate();
		});

	return Templates;
}

#undef LOCTEXT_NAMESPACE
