// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintExtension.h"
#include "Engine/Blueprint.h"
#include "WidgetBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprintExtension)


UWidgetBlueprintExtension* UWidgetBlueprintExtension::RequestExtension(UWidgetBlueprint* InBlueprint, TSubclassOf<UWidgetBlueprintExtension> InExtensionType)
{
	checkf(!InBlueprint->bBeingCompiled, TEXT("Do not use RequestExtension when a blueprint is being compiled."));

	// Look for an existing extension
	if (UWidgetBlueprintExtension* ExistingExtension = GetExtension(InBlueprint, InExtensionType))
	{
		return ExistingExtension;
	}

	// Not found, create one
	UWidgetBlueprintExtension* NewExtension = NewObject<UWidgetBlueprintExtension>(InBlueprint, InExtensionType.Get());
	InBlueprint->AddExtension(NewExtension);
	return NewExtension;
}


UWidgetBlueprintExtension* UWidgetBlueprintExtension::GetExtension(const UWidgetBlueprint* InBlueprint, TSubclassOf<UWidgetBlueprintExtension> InExtensionType)
{
	// Look for an existing extension
	for (const TObjectPtr<UBlueprintExtension>& Extension : InBlueprint->GetExtensions())
	{
		if (Extension && Extension->GetClass() == InExtensionType)
		{
			return CastChecked<UWidgetBlueprintExtension>(Extension);
		}
	}

	return nullptr;
}


TArray<UWidgetBlueprintExtension*> UWidgetBlueprintExtension::GetExtensions(const UWidgetBlueprint* InBlueprint)
{
	TArray<UWidgetBlueprintExtension*> Extensions;

	for (const TObjectPtr<UBlueprintExtension>& Extension : InBlueprint->GetExtensions())
	{
		if (Extension && Extension->GetClass()->IsChildOf(UWidgetBlueprintExtension::StaticClass()))
		{
			Extensions.Add(CastChecked<UWidgetBlueprintExtension>(Extension));
		}
	}

	return Extensions;
}

UWidgetBlueprint* UWidgetBlueprintExtension::GetWidgetBlueprint() const
{
	return GetOuterUWidgetBlueprint();
}
