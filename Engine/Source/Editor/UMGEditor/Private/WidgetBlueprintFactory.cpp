// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintFactory.h"
#include "UObject/Interface.h"
#include "Misc/MessageDialog.h"
#include "Blueprint/UserWidget.h"
#include "WidgetBlueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "WidgetBlueprintOperationUtils.h"

#include "UMGEditorProjectSettings.h"
#include "ClassViewerModule.h"
#include "Kismet2/SClassPickerDialog.h"
#include "ClassViewerFilter.h"
#include "Components/CanvasPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprintFactory)

#define LOCTEXT_NAMESPACE "UWidgetBlueprintFactory"

/*------------------------------------------------------------------------------
	UWidgetBlueprintFactory implementation.
------------------------------------------------------------------------------*/

class FWidgetClassFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet <const UClass*> AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}
	
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}

};

UWidgetBlueprintFactory::UWidgetBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UWidgetBlueprint::StaticClass();
	ParentClass = nullptr;
}

bool UWidgetBlueprintFactory::ConfigureProperties()
{
	if (GetDefault<UUMGEditorProjectSettings>()->bUseUserWidgetParentClassViewerSelector || GetDefault<UUMGEditorProjectSettings>()->bUseUserWidgetParentDefaultClassViewerSelector)
	{
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.DisplayMode = EClassViewerDisplayMode::Type::TreeView;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.bShowNoneOption = false;
		Options.bExpandAllNodes = true;
		Options.bIsBlueprintBaseOnly = true;
		Options.bShowDefaultClasses = GetDefault<UUMGEditorProjectSettings>()->bUseUserWidgetParentDefaultClassViewerSelector;
		Options.bShowClassesViewer = GetDefault<UUMGEditorProjectSettings>()->bUseUserWidgetParentClassViewerSelector;

		TSharedPtr<FWidgetClassFilter> Filter = MakeShareable(new FWidgetClassFilter);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		const TArray<TSoftClassPtr<UUserWidget>>& FavoriteWidgetParentClasses = GetDefault<UUMGEditorProjectSettings>()->FavoriteWidgetParentClasses;
		for (int32 Index = 0; Index < FavoriteWidgetParentClasses.Num(); ++Index)
		{
			UClass* FavoriteWidgetParentClass = FavoriteWidgetParentClasses[Index].LoadSynchronous();
			if (FavoriteWidgetParentClass && FavoriteWidgetParentClass->IsChildOf(UUserWidget::StaticClass()))
			{
				if (!Options.ExtraPickerCommonClasses.Contains(FavoriteWidgetParentClass))
				{
					Options.ExtraPickerCommonClasses.Add(FavoriteWidgetParentClass);
				}
			}
		}

		if (Options.ExtraPickerCommonClasses.Num() == 0)
		{
			Options.ExtraPickerCommonClasses.Add(UUserWidget::StaticClass());
		}

		Filter->DisallowedClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden | CLASS_HideDropDown;
		Filter->AllowedChildrenOfClasses.Add(UUserWidget::StaticClass());

		const FText TitleText = LOCTEXT("CreateWidgetBlueprint", "Pick Parent Class for New Widget Blueprint");

		UClass* ChosenParentClass = nullptr;
		bool isSuccessful = SClassPickerDialog::PickClass(TitleText, Options, ChosenParentClass, UUserWidget::StaticClass());
		ParentClass = ChosenParentClass ? ChosenParentClass : UUserWidget::StaticClass();

		if (!isSuccessful)
		{
			return false;
		}
	}

	if (GetDefault<UUMGEditorProjectSettings>()->bUseWidgetTemplateSelector)
	{
		// Load the classviewer module to display a class picker
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.bShowNoneOption = true;

		TArray<TSoftClassPtr<UPanelWidget>> CommonRootWidgetClasses = GetDefault <UUMGEditorProjectSettings>()->CommonRootWidgetClasses;
		for (int32 Index = 0; Index < CommonRootWidgetClasses.Num(); ++Index)
		{
			UClass* PanelWidgetClass = CommonRootWidgetClasses[Index].LoadSynchronous();
			if (PanelWidgetClass && PanelWidgetClass->IsChildOf(UPanelWidget::StaticClass()))
			{
				if (!Options.ExtraPickerCommonClasses.Contains(PanelWidgetClass))
				{
					Options.ExtraPickerCommonClasses.Add(PanelWidgetClass);
				}
			}
		}

		if (Options.ExtraPickerCommonClasses.Num() == 0)
		{
			Options.ExtraPickerCommonClasses.Add(UCanvasPanel::StaticClass());
		}

		TSharedPtr<FWidgetClassFilter> Filter = MakeShareable(new FWidgetClassFilter);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
		Filter->AllowedChildrenOfClasses.Add(UPanelWidget::StaticClass());

		const FText TitleText = LOCTEXT("CreateRootWidgetBlueprint", "Pick Root Widget for New Widget Blueprint");
		return SClassPickerDialog::PickClass(TitleText, Options, static_cast<UClass*&>(RootWidgetClass), UPanelWidget::StaticClass());

	}
	return true;
}

bool UWidgetBlueprintFactory::ShouldShowInNewMenu() const
{
	return true;
}

UObject* UWidgetBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a Widget Blueprint, then create and init one
	check(Class->IsChildOf(UWidgetBlueprint::StaticClass()));

	UClass* CurrentParentClass = ParentClass;
	if (CurrentParentClass == nullptr)
	{
		CurrentParentClass = UUserWidget::StaticClass();
	}

	// If they selected an interface, force the parent class to be UInterface
	if (BlueprintType == BPTYPE_Interface)
	{
		CurrentParentClass = UInterface::StaticClass();
	}

	if ( (CurrentParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(CurrentParentClass) || !CurrentParentClass->IsChildOf(UUserWidget::StaticClass()) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), CurrentParentClass ? FText::FromString( CurrentParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateWidgetBlueprint", "Cannot create a Widget Blueprint based on the class '{ClassName}'."), Args ) );
		return nullptr;
	}

	if (!GetDefault<UUMGEditorProjectSettings>()->bUseWidgetTemplateSelector)
	{
		RootWidgetClass = GetDefault<UUMGEditorProjectSettings>()->DefaultRootWidget;
	}

	return FWidgetBlueprintOperationUtils::CreateWidgetBlueprint(InParent, Name, BlueprintType, CurrentParentClass, RootWidgetClass, CallingContext, /*bRegisterAndCompile=*/false);
}

UObject* UWidgetBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

#undef LOCTEXT_NAMESPACE
