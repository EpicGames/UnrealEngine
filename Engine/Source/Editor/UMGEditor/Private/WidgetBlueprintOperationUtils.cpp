// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintOperationUtils.h"

#include "Animation/WidgetAnimation.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Blueprint/WidgetNavigation.h"
#include "Blueprint/WidgetTree.h"
#include "Components/ContentWidget.h"
#include "Components/NamedSlotInterface.h"
#include "Components/PanelSlot.h"
#include "Components/PanelWidget.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CallFunctionOnMember.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_GetArrayItem.h"
#include "K2Node_Knot.h"
#include "K2Node_Select.h"
#include "K2Node_StructMemberGet.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "ScopedTransaction.h"
#include "Templates/WidgetTemplateBlueprintClass.h"
#include "Templates/WidgetTemplateClass.h"
#include "Templates/WidgetTemplateImageClass.h"
#include "UIComponentUtils.h"
#include "UMGEditorModule.h"
#include "UMGEditorProjectSettings.h"
#include "UObject/ScriptInterface.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprintEditorUtils.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Event.h"
#include "K2Node_Variable.h"
#include "ScopedTransaction.h"
#include "Templates/WidgetTemplateBlueprintClass.h"
#include "Templates/WidgetTemplateImageClass.h"

#define LOCTEXT_NAMESPACE "UMG"

namespace Private
{
	FName SanitizeWidgetName(const FString& NewName, const FName CurrentName)
	{
		FString GeneratedName = SlugStringForValidName(NewName);

		// If the new name is empty (for example, because it was composed entirely of invalid characters).
		// then we'll use the current name
		if (GeneratedName.IsEmpty())
		{
			return CurrentName;
		}

		const FName GeneratedFName(*GeneratedName);
		check(GeneratedFName.IsValidXName(INVALID_OBJECTNAME_CHARACTERS));

		return GeneratedFName;
	}

	FWidgetBlueprintEditor* GetWidgetBlueprintEditorIfOpen(UWidgetBlueprint* WidgetBlueprint)
	{
		constexpr bool bFocusIfOpen = false;
		return static_cast<FWidgetBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(WidgetBlueprint, bFocusIfOpen));
	}

	UWidget* GetPreviewWidgetFromTemplate(FWidgetBlueprintEditor* BlueprintEditor, UWidget* TemplateWidget)
	{
		if (TemplateWidget && BlueprintEditor)
		{
			if (UUserWidget* PreviewRoot = BlueprintEditor->GetPreview())
			{
				UWidget* PreviewWidget = PreviewRoot->GetWidgetFromName(TemplateWidget->GetFName());
				return PreviewWidget;
			}
		}
		return nullptr;
	}

	void WalkImpl(
		const UWidget* Widget, const int32 Depth, const FName SlotName,
		const int32 MaxDepth, const TFunctionRef<void(const UWidget*, int32, FName)>& Visitor,
		TSet<const UWidget*>& Visited)
	{
		if (!Widget || Visited.Contains(Widget))
			return;

		Visited.Add(Widget);

		Visitor(Widget, Depth, SlotName);

		if (MaxDepth >= 0 && Depth >= MaxDepth)
			return;

		if (const UPanelWidget* Panel = Cast<const UPanelWidget>(Widget))
		{
			for (int32 i = 0; i < Panel->GetChildrenCount(); ++i)
			{
				WalkImpl(Panel->GetChildAt(i), Depth + 1, NAME_None, MaxDepth, Visitor, Visited);
			}
		}

		if (const INamedSlotInterface* NamedSlot = Cast<const INamedSlotInterface>(Widget))
		{
			TArray<FName> SlotNames;
			NamedSlot->GetSlotNames(SlotNames);

			for (const FName& ChildSlotName : SlotNames)
			{
				if (const UWidget* Content = NamedSlot->GetContentForSlot(ChildSlotName))
				{
					WalkImpl(Content, Depth + 1, ChildSlotName, MaxDepth, Visitor, Visited);
				}
			}
		}
	}

	int32 ComputeDepthImpl(const UWidget* Widget, TSet<const UWidget*>& Visited)
	{
		if (!Widget || Visited.Contains(Widget))
			return 0;

		Visited.Add(Widget);

		int32 MaxChild = -1;
		if (const UPanelWidget* Panel = Cast<const UPanelWidget>(Widget))
		{
			for (int32 i = 0; i < Panel->GetChildrenCount(); ++i)
			{
				MaxChild = FMath::Max(MaxChild, ComputeDepthImpl(Panel->GetChildAt(i), Visited));
			}
		}

		if (const INamedSlotInterface* NamedSlot = Cast<const INamedSlotInterface>(Widget))
		{
			TArray<FName> SlotNames;
			NamedSlot->GetSlotNames(SlotNames);

			for (const FName& ChildSlotName : SlotNames)
			{
				if (const UWidget* Content = NamedSlot->GetContentForSlot(ChildSlotName))
				{
					MaxChild = FMath::Max(MaxChild, ComputeDepthImpl(Content, Visited));
				}
			}
		}

		return MaxChild + 1;
	}
}

bool FWidgetBlueprintOperationUtils::DoesPropertyExistInClass(UClass* Class, FProperty* Property, FText& OutError)
{
	if (!Class || !Property)
	{
		OutError = NSLOCTEXT("UMG", "PropertyExistCheck_InvalidArgs", "Cannot check property existence: missing class or property.");
		return false;
	}
	FProperty* FoundProperty = Class->FindPropertyByName(Property->GetFName());
	if (!FoundProperty)
	{
		OutError = FText::Format(
			NSLOCTEXT("UMG", "PropertyExistCheck_NameMissing", "Property '{0}' does not exist on class '{1}'."),
			FText::FromName(Property->GetFName()),
			FText::FromString(Class->GetName()));
		return false;
	}
	if (!FoundProperty->SameType(Property))
	{
		OutError = FText::Format(
			NSLOCTEXT("UMG", "PropertyExistCheck_TypeMismatch", "Property '{0}' on class '{1}' has type '{2}' which is not compatible with the original type '{3}'."),
			FText::FromName(Property->GetFName()),
			FText::FromString(Class->GetName()),
			FText::FromString(FoundProperty->GetClass()->GetName()),
			FText::FromString(Property->GetClass()->GetName()));
		return false;
	}
	return true;
}

bool FWidgetBlueprintOperationUtils::DoesFunctionExistInClass(UClass* Class, UFunction* Function, FText& OutError)
{
	if (!Class || !Function)
	{
		OutError = NSLOCTEXT("UMG", "FunctionExistCheck_InvalidArgs", "Cannot check function existence: missing class or function.");
		return false;
	}
	UFunction* FoundFunction = Class->FindFunctionByName(Function->GetFName());
	if (!FoundFunction)
	{
		OutError = FText::Format(
			NSLOCTEXT("UMG", "FunctionExistCheck_NameMissing", "Function '{0}' does not exist on class '{1}'."),
			FText::FromName(Function->GetFName()),
			FText::FromString(Class->GetName()));
		return false;
	}
	if (!FoundFunction->IsSignatureCompatibleWith(Function))
	{
		OutError = FText::Format(
			NSLOCTEXT("UMG", "FunctionExistCheck_SignatureMismatch", "Function '{0}' on class '{1}' has a signature incompatible with the original on class '{2}'."),
			FText::FromName(Function->GetFName()),
			FText::FromString(Class->GetName()),
			FText::FromString(Function->GetOuterUClass() ? Function->GetOuterUClass()->GetName() : FString(TEXT("?"))));
		return false;
	}
	return true;
}

UWidgetBlueprint* FWidgetBlueprintOperationUtils::CreateWidgetBlueprint(UObject* InParent, FName Name, EBlueprintType BlueprintType, TSubclassOf<UUserWidget> ParentClass, TSubclassOf<UWidget> RootWidgetClass, FName CallingContext, bool bRegisterAndCompile)
{
	if (!InParent || Name.IsNone() || !ParentClass)
	{
		return nullptr;
	}

	UWidgetBlueprint* BP = Cast<UWidgetBlueprint>(
		FKismetEditorUtilities::CreateBlueprint(
			ParentClass,
			InParent,
			Name,
			BlueprintType,
			UWidgetBlueprint::StaticClass(),
			UWidgetBlueprintGeneratedClass::StaticClass(),
			CallingContext));

	if (!BP)
	{
		return nullptr;
	}

	// Optionally set a root widget.
	if (BP->WidgetTree->RootWidget == nullptr)
	{
		if (TSubclassOf<UPanelWidget> RootWidgetPanel = static_cast<UClass*>(RootWidgetClass))
		{
			UWidget* Root = BP->WidgetTree->ConstructWidget<UWidget>(RootWidgetPanel);
			BP->WidgetTree->RootWidget = Root;
			BP->OnVariableAdded(Root->GetFName());
		}
	}

	BP->bCanCallInitializedWithoutPlayerContext = GetDefault<UUMGEditorProjectSettings>()->bCanCallInitializedWithoutPlayerContext;

	{
		IUMGEditorModule::FWidgetBlueprintCreatedArgs Args;
		Args.ParentClass = ParentClass;
		Args.Blueprint = BP;

		IUMGEditorModule& UMGEditor = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		UMGEditor.OnWidgetBlueprintCreated().Broadcast(Args);
	}

	if (bRegisterAndCompile)
	{
		BP->MarkPackageDirty();
		FKismetEditorUtilities::CompileBlueprint(BP);
		FAssetRegistryModule::AssetCreated(BP);
	}

	return BP;
}

bool FWidgetBlueprintOperationUtils::CanAddToParent(UWidgetBlueprint* WidgetBlueprint, UWidget* ParentWidget, FText& OutErrorMessage)
{
	FWidgetBlueprintEditor* BlueprintEditor = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);
	UWidget* ParentPreviewWidget = Private::GetPreviewWidgetFromTemplate(BlueprintEditor, ParentWidget);

	if (ParentPreviewWidget && ParentPreviewWidget->IsLockedInDesigner())
	{
		OutErrorMessage = LOCTEXT("LockedWidget", "Widget is locked.");
		return false;
	}

	if (ParentWidget)
	{
		UPanelWidget* NewParent = Cast<UPanelWidget>(ParentWidget);
		if (!NewParent)
		{
			OutErrorMessage = LOCTEXT("CantHaveChildren", "Widget can't have children.");
			return false;
		}

		if (!NewParent->CanAddMoreChildren())
		{
			OutErrorMessage = LOCTEXT("NoAdditionalChildren", "Widget can't accept additional children.");
			return false;
		}
	}
	else if (WidgetBlueprint->WidgetTree->RootWidget != nullptr)
	{
		OutErrorMessage = LOCTEXT("CantHaveChildren", "Widget can't have children.");
		return false;
	}

	return true;
}

UWidget* FWidgetBlueprintOperationUtils::CreateWidgetFromAsset(UWidgetBlueprint* WidgetBlueprint, const FAssetData& AssetData, UWidgetTree* RootWidgetTree, FText& OutErrorMessage)
{
	if (!WidgetBlueprint)
	{
		OutErrorMessage = LOCTEXT("InvalidWidgetBlueprint", "Widget blueprint is invalid. Can't construct a widget without a widget blueprint");
		return nullptr;
	}
	 
	RootWidgetTree = RootWidgetTree ? RootWidgetTree : WidgetBlueprint->WidgetTree.Get();

	if (!ensure(RootWidgetTree))
	{
		OutErrorMessage = LOCTEXT("InvalidWidgetTree", "Widget tree is invalid. Can't construct a widget without a widget tree");
		return nullptr;
	}

	UWidget* Widget = nullptr;

	bool CodeClass = AssetData.AssetClassPath == FTopLevelAssetPath(TEXT("/Script/CoreUObject"), TEXT("Class"));
	FString ClassName = CodeClass ? AssetData.GetObjectPathString() : AssetData.AssetClassPath.ToString();
	UClass* AssetClass = FindObject<UClass>(nullptr, *ClassName);

	if (!AssetClass)
	{
		ensure(false);
		OutErrorMessage = LOCTEXT("InvalidWidgetClass", "Can't export a valid class from the asset. Widget won't be constructed");
		return nullptr;
	}

	if (FWidgetTemplateBlueprintClass::Supports(AssetClass))
	{
		// Allows a UMG Widget Blueprint to be dragged from the Content Browser to another Widget Blueprint...as long as we're not trying to place a
		// blueprint inside itself.
		FString BlueprintPath = WidgetBlueprint->GetPathName();
		if (BlueprintPath != AssetData.GetSoftObjectPath().ToString())
		{
			Widget = FWidgetTemplateBlueprintClass(AssetData).Create(RootWidgetTree);
		}
	}
	else if (CodeClass && AssetClass && AssetClass->IsChildOf(UWidget::StaticClass()))
	{
		Widget = FWidgetTemplateClass(AssetClass).Create(RootWidgetTree);
	}
	else if (FWidgetTemplateImageClass::Supports(AssetClass))
	{
		Widget = FWidgetTemplateImageClass(AssetData).Create(RootWidgetTree);
	}
	else
	{
		OutErrorMessage = LOCTEXT("UnsupportedAssetClass", "Can't construct a widget using the passed class because it is unsupported.");
	}

	// Check to make sure that this widget can be added to the current blueprint
	if (Cast<UUserWidget>(Widget) && !WidgetBlueprint->IsWidgetFreeFromCircularReferences(Cast<UUserWidget>(Widget)))
	{
		OutErrorMessage = LOCTEXT("CircularDependencyWidget", "Circular dependency detected. Can't construct a widget in the widget blueprint");
		return nullptr;
	}

	return Widget;
}

bool FWidgetBlueprintOperationUtils::AddWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* NewWidget, UWidget* ParentWidget, int32 ChildIndex, FText& OutErrorMessage)
{
	if (!WidgetBlueprint)
	{
		OutErrorMessage = LOCTEXT("NoWidgetBlueprint", "Can't add widget without a valid WidgetBlueprint to add to.");
		return false;
	}

	if (!NewWidget)
	{
		OutErrorMessage = LOCTEXT("NoWidget", "The widget instance to add is not valid.");
		return false;
	}

	if (!CanAddToParent(WidgetBlueprint, ParentWidget, OutErrorMessage))
	{
		RemoveTransientWidgetFromTree(WidgetBlueprint, NewWidget);
		return false;
	}

	// Are we adding to the root?
	if (!ParentWidget && WidgetBlueprint->WidgetTree->RootWidget == nullptr)
	{
		// TODO UMG Allow showing a preview of this.
		FScopedTransaction Transaction(LOCTEXT("AddWidgetFromTemplate", "Add Widget"));

		WidgetBlueprint->WidgetTree->SetFlags(RF_Transactional);
		WidgetBlueprint->WidgetTree->Modify();
		WidgetBlueprint->WidgetTree->RootWidget = NewWidget;
		WidgetBlueprint->OnVariableAdded(NewWidget->GetFName());
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);

		return true;
	}
	// Are we adding to a panel?
	else if (UPanelWidget* Parent = Cast<UPanelWidget>(ParentWidget))
	{
		FScopedTransaction Transaction(LOCTEXT("AddWidgetFromTemplate", "Add Widget"));

		WidgetBlueprint->WidgetTree->SetFlags(RF_Transactional);
		WidgetBlueprint->WidgetTree->Modify();

		Parent->SetFlags(RF_Transactional);
		Parent->Modify();
		UPanelSlot* NewSlot = nullptr;
		if (ChildIndex > -1)
		{
			NewSlot = Parent->InsertChildAt(ChildIndex, NewWidget);
		}
		else
		{
			NewSlot = Parent->AddChild(NewWidget);
		}
		check(NewSlot);

		WidgetBlueprint->OnVariableAdded(NewWidget->GetFName());

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
		return true;
	}

	return false;
}

bool FWidgetBlueprintOperationUtils::IsParentChildCycleFree(UWidgetBlueprint* WidgetBlueprint, UWidget* ChildWidget, UWidget* ParentWidget)
{
	if (!WidgetBlueprint || !ChildWidget || !ParentWidget)
	{
		return true;
	}

	bool bFoundParentInChildSet = false;
	WidgetBlueprint->WidgetTree->ForWidgetAndChildren(ChildWidget, [&](UWidget* Widget) {
		if (ParentWidget == Widget)
		{
			bFoundParentInChildSet = true;
		}
		});

	return !bFoundParentInChildSet;
}

bool FWidgetBlueprintOperationUtils::MoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, UPanelWidget* NewParent, int32 ChildIndex, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !Widget || !NewParent)
	{
		OutErrorMessage = LOCTEXT("CantMoveWidgetWithoutValidParameters", "A valid widget instance, parent instance, and widget blueprint is required to move a widget.");
		return false;
	}

	if (!CanAddToParent(WidgetBlueprint, NewParent, OutErrorMessage))
	{
		return false;
	}

	if (!IsParentChildCycleFree(WidgetBlueprint, Widget, NewParent))
	{
		OutErrorMessage = LOCTEXT("CantMakeWidgetChildOfItsChildren", "Can't make widget a child of its children.");
		return false;
	}

	NewParent->SetFlags(RF_Transactional);
	NewParent->Modify();

	Widget->SetFlags(RF_Transactional);
	Widget->Modify();
	if (ChildIndex > -1)
	{
		// If we're inserting at an index, and the widget we're moving is already
		// in the hierarchy before the point we're moving it to, we need to reduce the index
		// count by one, because the whole set is about to be shifted when it's removed.
		const bool bInsertInSameParent = Widget->GetParent() == NewParent;
		const bool bNeedToDropIndex = NewParent->GetChildIndex(Widget) < ChildIndex;

		if (bInsertInSameParent && bNeedToDropIndex)
		{
			ChildIndex = ChildIndex - 1;
		}
	}

	// We don't know if this widget is being removed from a named slot and RemoveFromParent is not enough to take care of this
	UWidget* NamedSlotHostWidget = FWidgetBlueprintEditorUtils::FindNamedSlotHostWidgetForContent(Widget, WidgetBlueprint->WidgetTree);
	if (NamedSlotHostWidget != nullptr)
	{
		if (TScriptInterface<INamedSlotInterface> NamedSlotHost = TScriptInterface<INamedSlotInterface>(NamedSlotHostWidget))
		{
			NamedSlotHostWidget->SetFlags(RF_Transactional);
			NamedSlotHostWidget->Modify();
			FWidgetBlueprintEditorUtils::RemoveNamedSlotHostContent(Widget, NamedSlotHost);
		}
	}

	// If this widget inherits from another one, we can't access the inherited named slots by traversing the widget tree from its root.
	// So we have to look at the NamedSlotBindings to find a named slot for the moved content.
	else if (WidgetBlueprint->ParentClass && WidgetBlueprint->ParentClass != UUserWidget::StaticClass())
	{
		TArray<FName> SlotNames;
		WidgetBlueprint->WidgetTree->GetSlotNames(SlotNames);
		for (FName SlotName : SlotNames)
		{
			if (UWidget* SlotContent = WidgetBlueprint->WidgetTree->GetContentForSlot(SlotName))
			{
				if (SlotContent == Widget)
				{
					WidgetBlueprint->WidgetTree->SetContentForSlot(SlotName, nullptr);
				}
			}
		}
	}

	const FName OriginalWidgetName = Widget->GetFName();
	UPanelWidget* OriginalParent = Widget->GetParent();
	UWidgetBlueprint* OriginalBP = nullptr;

	// The widget's parent is changing
	if (OriginalParent != NewParent)
	{
		NewParent->SetFlags(RF_Transactional);
		NewParent->Modify();

		WidgetBlueprint->WidgetTree->SetFlags(RF_Transactional);
		WidgetBlueprint->WidgetTree->Modify();

		UWidgetTree* OriginalWidgetTree = Cast<UWidgetTree>(Widget->GetOuter());

		if (OriginalWidgetTree != nullptr && UWidgetTree::TryMoveWidgetToNewTree(Widget, WidgetBlueprint->WidgetTree))
		{
			OriginalWidgetTree->SetFlags(RF_Transactional);
			OriginalWidgetTree->Modify();

			OriginalBP = OriginalWidgetTree->GetTypedOuter<UWidgetBlueprint>();
		}
	}

	TMap<FName, FString> ExportedSlotProperties;
	FWidgetBlueprintEditorUtils::ExportPropertiesToText(Widget->Slot, ExportedSlotProperties);

	Widget->RemoveFromParent();

	if (OriginalBP != nullptr && OriginalBP != WidgetBlueprint)
	{
		OriginalBP->OnVariableRemoved(OriginalWidgetName);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(OriginalBP);
	}

	UPanelSlot* NewSlot = nullptr;
	if (ChildIndex > -1)
	{
		NewSlot = NewParent->InsertChildAt(ChildIndex, Widget);
		ChildIndex = ChildIndex + 1;
	}
	else
	{
		NewSlot = NewParent->AddChild(Widget);
	}
	check(NewSlot);

	if (OriginalBP != nullptr && OriginalBP != WidgetBlueprint)
	{
		WidgetBlueprint->OnVariableAdded(Widget->GetFName());
	}

	// Import the old slot properties
	FWidgetBlueprintEditorUtils::ImportPropertiesFromText(NewSlot, ExportedSlotProperties);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);

	return true;
}

bool FWidgetBlueprintOperationUtils::RemoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !Widget)
	{
		OutErrorMessage = LOCTEXT("CantRemoveWidgetWithoutValidParameters", "A valid widget instance and widget blueprint is required to remove a widget.");
		return false;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (!ensure(WidgetTree))
	{
		OutErrorMessage = LOCTEXT("CantRemoveWidgetWithInvalidWidgetTree", "Widget tree is invalid. Can't remove a widget without a widget tree");
		return false;
	}

	if (!WidgetTree->FindWidget(Widget->GetFName()) && Widget->GetOuter() != WidgetTree)
	{
		OutErrorMessage = LOCTEXT("CantRemoveWidgetNotInBlueprint", "Widget not found in blueprint");
		return false;
	}

	FWidgetBlueprintEditorUtils::DeleteWidgets(WidgetBlueprint, { Widget }, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::DeleteSilently);
	return true;
}

bool FWidgetBlueprintOperationUtils::RenameWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FString& NewDisplayName)
{
	if (!WidgetBlueprint || !Widget)
	{
		return false;
	}

	const FName OldObjectName = Widget->GetFName();

	UClass* ParentClass = WidgetBlueprint->ParentClass;
	check(ParentClass);

	bool bRenamed = false;

	TSharedPtr<INameValidatorInterface> NameValidator = MakeShareable(new FKismetNameValidator(WidgetBlueprint, OldObjectName));

	const FName NewFName = Private::SanitizeWidgetName(NewDisplayName, Widget->GetFName());

	FObjectPropertyBase* ExistingProperty = CastField<FObjectPropertyBase>(ParentClass->FindPropertyByName(NewFName));
	const bool bBindWidget = ExistingProperty && FWidgetBlueprintEditorUtils::IsBindWidgetProperty(ExistingProperty) && Widget->IsA(ExistingProperty->PropertyClass);

	// NewName should be already validated. But one must make sure that NewTemplateName is also unique.
	const bool bUniqueNameForTemplate = (EValidatorResult::Ok == NameValidator->IsValid(NewFName) || bBindWidget);
	if (bUniqueNameForTemplate)
	{
		// Stringify the FNames
		const FString NewNameStr = NewFName.ToString();
		const FString OldNameStr = OldObjectName.ToString();

		const FScopedTransaction Transaction(LOCTEXT("RenameWidget", "Rename Widget"));

		// Rename Template
		WidgetBlueprint->Modify();
		Widget->Modify();

		WidgetBlueprint->OnVariableRenamed(OldObjectName, NewFName);

		FWidgetBlueprintEditor* BlueprintEditor = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);

		if (BlueprintEditor)
		{
			// Rename Preview before renaming the template widget so the preview widget can be found
			UWidget* WidgetPreview = BlueprintEditor->GetReferenceFromTemplate(Widget).GetPreview();
			if (WidgetPreview)
			{
				WidgetPreview->SetDisplayLabel(NewDisplayName);
				WidgetPreview->Rename(*NewNameStr);
			}

			if (!WidgetPreview || WidgetPreview != Widget)
			{
				// Find and update all variable references in the graph
				Widget->SetDisplayLabel(NewDisplayName);
				Widget->Rename(*NewNameStr);
			}
		}
		else
		{
			Widget->SetDisplayLabel(NewDisplayName);
			Widget->Rename(*NewNameStr);
		}
#if UE_HAS_WIDGET_GENERATED_BY_CLASS
		// When a widget gets renamed we need to check any existing blueprint getters that may be placed
		// in the graphs to fix up their state
		if (Widget->bIsVariable)
		{
			TArray<UEdGraph*> AllGraphs;
			WidgetBlueprint->GetAllGraphs(AllGraphs);

			for (const UEdGraph* CurrentGraph : AllGraphs)
			{
				TArray<UK2Node_Variable*> GraphNodes;
				CurrentGraph->GetNodesOfClass(GraphNodes);

				for (UK2Node_Variable* CurrentNode : GraphNodes)
				{
					UClass* SelfClass = WidgetBlueprint->GeneratedClass;
					UClass* VariableParent = CurrentNode->VariableReference.GetMemberParentClass(SelfClass);

					if (SelfClass == VariableParent)
					{
						// Reconstruct this node in order to give it orphan pins and invalidate any 
						// connections that will no longer be valid
						if (NewFName == CurrentNode->GetVarName())
						{
							UEdGraphPin* ValuePin = CurrentNode->GetValuePin();
							ValuePin->Modify();
							CurrentNode->Modify();

							// Make the old pin an orphan and add a new pin of the proper type
							UEdGraphPin* NewPin = CurrentNode->CreatePin(
								ValuePin->Direction,
								ValuePin->PinType.PinCategory,
								ValuePin->PinType.PinSubCategory,
								Widget->WidgetGeneratedByClass.Get(),	// This generated object is what needs to patched up
								NewFName
							);

							ValuePin->bOrphanedPin = true;
						}
					}
				}
			}
		}
#endif

		// Replace the Desired focus Widget name if it match the renamed widget
		FWidgetBlueprintEditorUtils::ReplaceDesiredFocus(WidgetBlueprint, OldObjectName, NewFName);

		// Find and update all binding references in the widget blueprint
		for (FDelegateEditorBinding& Binding : WidgetBlueprint->Bindings)
		{
			if (Binding.ObjectName == OldNameStr)
			{
				Binding.ObjectName = NewNameStr;
			}
		}

		// Update widget blueprint names
		for (UWidgetAnimation* WidgetAnimation : WidgetBlueprint->Animations)
		{
			for (FWidgetAnimationBinding& AnimBinding : WidgetAnimation->AnimationBindings)
			{
				if (AnimBinding.WidgetName == OldObjectName)
				{
					AnimBinding.WidgetName = NewFName;

					WidgetAnimation->MovieScene->Modify();

					if (AnimBinding.SlotWidgetName == NAME_None)
					{
						FMovieScenePossessable* Possessable = WidgetAnimation->MovieScene->FindPossessable(AnimBinding.AnimationGuid);
						if (Possessable)
						{
							Possessable->SetName(NewFName.ToString());
						}
					}
					else
					{
						break;
					}
				}
			}
		}

		// Update any explicit widget bindings.
		WidgetBlueprint->WidgetTree->ForEachWidget([OldObjectName, NewFName](UWidget* Widget) {
			if (Widget->Navigation)
			{
				Widget->Navigation->SetFlags(RF_Transactional);
				Widget->Navigation->Modify();
				Widget->Navigation->TryToRenameBinding(OldObjectName, NewFName);
			}
			});

		// If we use Components, make sure to remane the target.
		FUIComponentUtils::OnWidgetRenamed(BlueprintEditor, WidgetBlueprint, OldObjectName, NewFName);

		// Validate child blueprints and adjust variable names to avoid a potential name collision
		FBlueprintEditorUtils::ValidateBlueprintChildVariables(WidgetBlueprint, NewFName);

		// Refresh references and flush editors
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);

		// Update Variable References and
		// Update Event References to member variables
		FBlueprintEditorUtils::ReplaceVariableReferences(WidgetBlueprint, OldObjectName, NewFName);

		// Update graph references of omponent variable names ("ComponentName_WidgetName").
		// Must run after MarkBlueprintAsStructurallyModified to avoid BroadcastChanged() reverting changes.
		FUIComponentUtils::ReplaceComponentVariableReferences(WidgetBlueprint, OldObjectName, NewFName);

		bRenamed = true;
	}

	return bRenamed;
}

void FWidgetBlueprintOperationUtils::ToggleWidgetAsVariable(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, bool bIsVariable, bool bMarkBlueprintModified)
{
	if (!WidgetBlueprint || !Widget)
	{
		return;
	}

	if (Widget->bIsVariable == bIsVariable)
	{
		return;
	}

	WidgetBlueprint->Modify();
	Widget->bIsVariable = bIsVariable;

	if (bMarkBlueprintModified)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	}
}

bool FWidgetBlueprintOperationUtils::BindToEventProperty(UWidgetBlueprint* WidgetBlueprint, FName EventName, FName PropertyName, UClass* PropertyClass, bool bShouldJumpToNode, FText& OutErrorMessage)
{
	if (!WidgetBlueprint)
	{
		OutErrorMessage = LOCTEXT("BindToEventProperty_NullBlueprint", "BindToEventProperty: WidgetBlueprint is required.");
		return false;
	}

	if (EventName.IsNone() || PropertyName.IsNone())
	{
		OutErrorMessage = LOCTEXT("BindToEventProperty_MissingNames", "BindToEventProperty: EventName and PropertyName are required.");
		return false;
	}

	if (!PropertyClass)
	{
		OutErrorMessage = LOCTEXT("BindToEventProperty_NullPropertyClass", "BindToEventProperty: a valid PropertyClass for the widget is required.");
		return false;
	}

	FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(WidgetBlueprint->SkeletonGeneratedClass, PropertyName);
	if (!VariableProperty)
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("BindToEventProperty_VariableNotFound", "BindToEventProperty: Failed to find widget variable named '{0}' in widget blueprint '{1}'."),
			FText::FromName(PropertyName), FText::FromString(WidgetBlueprint->GetName()));
		return false;
	}

	if (const UK2Node_ComponentBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(WidgetBlueprint, EventName, VariableProperty->GetFName()))
	{
		if (bShouldJumpToNode)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
			return false;
		}

		OutErrorMessage = FText::Format(
			LOCTEXT("BindToEventProperty_AlreadyBound", "BindToEventProperty: The event '{0}' is already bound on widget '{1}' in widget blueprint '{2}'."),
			FText::FromName(EventName), FText::FromName(PropertyName), FText::FromString(WidgetBlueprint->GetName()));
		return false;
	}

	FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(PropertyClass, EventName);
	if (!DelegateProperty)
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("BindToEventProperty_DelegateNotFound", "BindToEventProperty: An event named '{0}' must exist on widget '{1}' in widget blueprint '{2}'."),
			FText::FromName(EventName), FText::FromName(PropertyName), FText::FromString(WidgetBlueprint->GetName()));
		return false;
	}

	FKismetEditorUtilities::CreateNewBoundEventForClass(PropertyClass, EventName, WidgetBlueprint, VariableProperty, bShouldJumpToNode);
	return true;
}

void FWidgetBlueprintOperationUtils::FixupWidgetBlueprintReferences(UWidgetBlueprint* WidgetBlueprint, UClass* OldClass, UClass* NewWidgetClass, FName VariableName)
{
	if (OldClass != NewWidgetClass)
	{
		TArray<UEdGraph*> AllGraphs;
		WidgetBlueprint->GetAllGraphs(AllGraphs);

		// Phase 1: Forward BFS from the variable's Get nodes.  Follow the widget
		// reference through reroutes and casts, retargeting function calls and
		// property accesses whose MemberParent / TargetType is the old class.
		TArray<UEdGraphPin*> Worklist;

		for (UEdGraph* Graph : AllGraphs)
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				if (UK2Node_VariableGet* VarGet = Cast<UK2Node_VariableGet>(Node))
				{
					if (VarGet->VariableReference.IsSelfContext() && VarGet->GetVarName() == VariableName)
					{
						if (UEdGraphPin* ValuePin = VarGet->GetValuePin())
						{
							Worklist.Append(ValuePin->LinkedTo);
						}
					}
				}
			}
		}

		FText MissingReason;
		TSet<UEdGraphNode*> VisitedNodes;

		for (int32 WorkIdx = 0; WorkIdx < Worklist.Num(); ++WorkIdx)
		{
			UEdGraphNode* Node = Worklist[WorkIdx]->GetOwningNode();

			bool bAlreadyVisited = false;
			VisitedNodes.Add(Node, &bAlreadyVisited);
			if (bAlreadyVisited)
			{
				continue;
			}

			if (UK2Node_Knot* Knot = Cast<UK2Node_Knot>(Node))
			{
				// Reroute — follow through.
				if (Knot->GetOutputPin())
				{
					Worklist.Append(Knot->GetOutputPin()->LinkedTo);
				}
			}
			else if (UK2Node_DynamicCast* CastNode = Cast<UK2Node_DynamicCast>(Node))
			{
				if (CastNode->TargetType == OldClass)
				{
					CastNode->TargetType = NewWidgetClass;
				}
				// Follow through the cast result — the value is the same widget.
				if (UEdGraphPin* CastResult = CastNode->GetCastResultPin())
				{
					Worklist.Append(CastResult->LinkedTo);
				}
			}
			else if (UK2Node_GetArrayItem* GetItem = Cast<UK2Node_GetArrayItem>(Node))
			{
				// Pure pass-through: array element retains the widget's type.
				if (UEdGraphPin* ResultPin = GetItem->GetResultPin())
				{
					Worklist.Append(ResultPin->LinkedTo);
				}
			}
			else if (UK2Node_Select* SelectNode = Cast<UK2Node_Select>(Node))
			{
				// The Select's return value is the same widget as the chosen option.
				if (UEdGraphPin* ReturnPin = SelectNode->GetReturnValuePin())
				{
					Worklist.Append(ReturnPin->LinkedTo);
				}
			}
			else if (Cast<UK2Node_StructMemberGet>(Node) != nullptr)
			{
				for (UEdGraphPin* Pin : Node->Pins)
				{
					if (Pin->Direction == EGPD_Output && Pin->PinType.PinSubCategoryObject.Get() == OldClass)
					{
						Worklist.Append(Pin->LinkedTo);
					}
				}
			}
			else if (UK2Node_CallFunction* CallNode = Cast<UK2Node_CallFunction>(Node))
			{
				if (CallNode->FunctionReference.GetMemberParentClass() == OldClass)
				{
					if (DoesFunctionExistInClass(NewWidgetClass, CallNode->GetTargetFunction(), MissingReason))
					{
						CallNode->FunctionReference.SetExternalMember(CallNode->FunctionReference.GetMemberName(), NewWidgetClass);
					}
					else
					{
						UE_LOG(LogUMGEditor, Warning, TEXT("Invalid Widget Blueprint Reference: Function '%s' on node '%s' has no compatible counterpart on class '%s': %s"),
							*CallNode->FunctionReference.GetMemberName().ToString(),
							*CallNode->GetNodeTitle(ENodeTitleType::ListView).ToString(),
							*NewWidgetClass->GetName(),
							*MissingReason.ToString());
					}
					MissingReason = FText::GetEmpty();
				}
			}
			else if (UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(Node))
			{
				// Covers UK2Node_AddDelegate, CallDelegate, RemoveDelegate, ClearDelegate, AssignDelegate.
				if (DelegateNode->DelegateReference.GetMemberParentClass() == OldClass)
				{
					// Multicast delegate properties on a widget appear as FMulticastDelegateProperty,
					// so the property compatibility check is the right shape here.
					if (DoesPropertyExistInClass(NewWidgetClass, DelegateNode->GetProperty(), MissingReason))
					{
						DelegateNode->DelegateReference.SetExternalMember(DelegateNode->DelegateReference.GetMemberName(), NewWidgetClass);
					}
					else
					{
						UE_LOG(LogUMGEditor, Warning, TEXT("Invalid Widget Blueprint Reference: Delegate '%s' on node '%s' has no compatible counterpart on class '%s': %s"),
							*DelegateNode->DelegateReference.GetMemberName().ToString(),
							*DelegateNode->GetNodeTitle(ENodeTitleType::ListView).ToString(),
							*NewWidgetClass->GetName(),
							*MissingReason.ToString());
					}
					MissingReason = FText::GetEmpty();
				}
			}
			else if (UK2Node_Variable* VarNode = Cast<UK2Node_Variable>(Node))
			{
				if (!VarNode->VariableReference.IsSelfContext() &&
					!VarNode->VariableReference.IsLocalScope() &&
					VarNode->VariableReference.GetMemberParentClass() == OldClass)
				{
					if (DoesPropertyExistInClass(NewWidgetClass, VarNode->GetPropertyForVariable(), MissingReason))
					{
						VarNode->VariableReference.SetExternalMember(VarNode->VariableReference.GetMemberName(), NewWidgetClass);
					}
					else
					{
						UE_LOG(LogUMGEditor, Warning, TEXT("Invalid Widget Blueprint Reference: Property '%s' on node '%s' has no compatible counterpart on class '%s': %s"),
							*VarNode->VariableReference.GetMemberName().ToString(),
							*VarNode->GetNodeTitle(ENodeTitleType::ListView).ToString(),
							*NewWidgetClass->GetName(),
							*MissingReason.ToString());
					}
					MissingReason = FText::GetEmpty();
				}
			}
		}

		// Phase 2: Handle nodes that reference the variable by name rather than
		// through pin connections (no graph traversal needed).
		for (UEdGraph* Graph : AllGraphs)
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				if (UK2Node_ComponentBoundEvent* EventNode = Cast<UK2Node_ComponentBoundEvent>(Node))
				{
					if (EventNode->ComponentPropertyName == VariableName && OldClass->IsChildOf(EventNode->DelegateOwnerClass))
					{
						if (DoesPropertyExistInClass(NewWidgetClass, EventNode->GetTargetDelegateProperty(), MissingReason))
						{
							EventNode->DelegateOwnerClass = NewWidgetClass;
						}
						else
						{
							UE_LOG(LogUMGEditor, Warning, TEXT("Invalid Widget Blueprint Reference: Bound-event delegate '%s' on node '%s' has no compatible counterpart on class '%s': %s"),
								*EventNode->DelegatePropertyName.ToString(),
								*EventNode->GetNodeTitle(ENodeTitleType::ListView).ToString(),
								*NewWidgetClass->GetName(),
								*MissingReason.ToString());

							// Clear out the name so that the event node doesn't attempt to attach to the new replacement widget
							EventNode->ComponentPropertyName = FName();
						}
						MissingReason = FText::GetEmpty();
					}
				}
				else if (UK2Node_CallFunctionOnMember* MemberCallNode = Cast<UK2Node_CallFunctionOnMember>(Node))
				{
					if (MemberCallNode->MemberVariableToCallOn.GetMemberName() == VariableName &&
						MemberCallNode->FunctionReference.GetMemberParentClass() == OldClass)
					{
						if (DoesFunctionExistInClass(NewWidgetClass, MemberCallNode->GetTargetFunction(), MissingReason))
						{
							MemberCallNode->FunctionReference.SetExternalMember(MemberCallNode->FunctionReference.GetMemberName(), NewWidgetClass);
						}
						else
						{
							UE_LOG(LogUMGEditor, Warning, TEXT("Invalid Widget Blueprint Reference: Function '%s' on node '%s' has no compatible counterpart on class '%s': %s"),
								*MemberCallNode->FunctionReference.GetMemberName().ToString(),
								*MemberCallNode->GetNodeTitle(ENodeTitleType::ListView).ToString(),
								*NewWidgetClass->GetName(),
								*MissingReason.ToString());
						}
						MissingReason = FText::GetEmpty();
					}
				}
			}
		}
	}

	FBlueprintEditorUtils::RefreshAllNodes(WidgetBlueprint);
}

bool FWidgetBlueprintOperationUtils::ReplaceWidgetsWithTemplateClass(UWidgetBlueprint* BP, TSet<UWidget*> Widgets, TSharedPtr<FWidgetTemplateClass> TemplateClass, FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod NewWidgetNamingMethod)
{
	TMap<FName, FName> ReplacedWidgetMap;

	bool bReplacedAll = true;
	for (UWidget* Item : Widgets)
	{
		BP->WidgetTree->SetFlags(RF_Transactional);
		BP->WidgetTree->Modify();

		UWidget* WidgetToReplace = Item;

		// If replacing a panel widget, then it must not have children or the replacement must also be a panel widget
		if (UPanelWidget* ExistingPanel = Cast<UPanelWidget>(WidgetToReplace))
		{
			if (ExistingPanel->GetChildrenCount() > 0 && !TemplateClass->GetWidgetClass()->IsChildOf(UPanelWidget::StaticClass()))
			{
				bReplacedAll = false;
				continue;
			}
		}

		UWidget* NewReplacementWidget = TemplateClass->Create(BP->WidgetTree);

		TMap<FName, FString> ExportedProperties;
		FWidgetBlueprintEditorUtils::ExportPropertiesToText(WidgetToReplace, ExportedProperties);
		FWidgetBlueprintEditorUtils::ImportPropertiesFromText(NewReplacementWidget, ExportedProperties);

		WidgetToReplace->SetFlags(RF_Transactional);
		WidgetToReplace->Modify();

		const FName OriginalWidgetName = WidgetToReplace->GetFName();

		// Look if the Widget to replace is inside a NamedSlot.
		if (TScriptInterface<INamedSlotInterface> NamedSlotHost = FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(WidgetToReplace, BP->WidgetTree))
		{
			const bool bDidReplace = FWidgetBlueprintEditorUtils::ReplaceNamedSlotHostContent(WidgetToReplace, NamedSlotHost, NewReplacementWidget);
			if (!bDidReplace)
			{
				bReplacedAll = false;
				continue;
			}
		}
		else if (UPanelWidget* CurrentParent = WidgetToReplace->GetParent())
		{
			CurrentParent->SetFlags(RF_Transactional);
			CurrentParent->Modify();
			const bool bDidReplace = CurrentParent->ReplaceChild(WidgetToReplace, NewReplacementWidget);
			if (!bDidReplace)
			{
				bReplacedAll = false;
				continue;
			}
		}
		else if (WidgetToReplace == BP->WidgetTree->RootWidget)
		{
			BP->WidgetTree->RootWidget = NewReplacementWidget;
		}
		else
		{
			bReplacedAll = false;
			continue;
		}

		NewReplacementWidget->SetFlags(RF_Transactional);
		NewReplacementWidget->Modify();

		if (UPanelWidget* ExistingPanel = Cast<UPanelWidget>(WidgetToReplace))
		{
			if (UPanelWidget* NewReplacementPanelWidget = Cast<UPanelWidget>(NewReplacementWidget))
			{
				while (ExistingPanel->GetChildrenCount() > 0)
				{
					UWidget* Widget = ExistingPanel->GetChildAt(0);
					Widget->SetFlags(RF_Transactional);
					Widget->Modify();

					// Preserve the child's slot properties as text so any properties shared with
					// the new slot type carry over. Properties unique to the old slot type are
					// silently dropped during import - that's the "as much as possible" semantic.
					TMap<FName, FString> ExportedSlotProperties;
					if (Widget->Slot)
					{
						FWidgetBlueprintEditorUtils::ExportPropertiesToText(Widget->Slot, ExportedSlotProperties);
					}

					UPanelSlot* NewSlot = NewReplacementPanelWidget->AddChild(Widget);
					if (NewSlot && ExportedSlotProperties.Num() > 0)
					{
						FWidgetBlueprintEditorUtils::ImportPropertiesFromText(NewSlot, ExportedSlotProperties);
					}
				}
			}
		}

		// We need to check before replacing because the Widget might be deleted, reseting the DesiredFocus
		const bool bReplacingDesiredFocus = FWidgetBlueprintEditorUtils::IsDesiredFocusWidget(BP, WidgetToReplace);

		FString ReplaceName = WidgetToReplace->GetName();
		const bool bCanKeepName = (NewWidgetNamingMethod == FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod::MaintainNameAndReferencesForUnmatchingClass) ||
			(!WidgetToReplace->IsGeneratedName() && NewWidgetNamingMethod == FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod::MaintainNameAndReferences
				&& ((WidgetToReplace->IsA<UPanelWidget>() && NewReplacementWidget->IsA<UPanelWidget>())
					|| WidgetToReplace->IsA(NewReplacementWidget->GetClass())
					|| NewReplacementWidget->IsA(WidgetToReplace->GetClass())));

		// Rename the removed widget to the transient package so that it doesn't conflict with the new widget if we try to keep the same name.
		FName TrashName = MakeUniqueObjectName(GetTransientPackage(), WidgetToReplace->GetClass(), *FString::Printf(TEXT("TRASH_%s"), *WidgetToReplace->GetName()));
		WidgetToReplace->Rename(*TrashName.ToString(), GetTransientPackage());

		// Rename the new Widget to maintain the current name if it's not a generic name
		if (NewWidgetNamingMethod == FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod::MaintainNameAndReferences || NewWidgetNamingMethod == FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod::MaintainNameAndReferencesForUnmatchingClass)
		{
			if (bCanKeepName)
			{
				ReplaceName = FWidgetBlueprintEditorUtils::FindNextValidName(BP->WidgetTree, ReplaceName);
				NewReplacementWidget->Rename(*ReplaceName, BP->WidgetTree);
			}

			// Preserve references to the widget if we haven't kept the same name
			if (OriginalWidgetName != NewReplacementWidget->GetFName())
			{
				BP->OnVariableRenamed(OriginalWidgetName, NewReplacementWidget->GetFName());
			}

			// Even if the name hasn't changed, we need to replace references since the type might have changed
			ReplacedWidgetMap.Add(OriginalWidgetName, NewReplacementWidget->GetFName());
		}
		else if (NewReplacementWidget->GetFName() != OriginalWidgetName)
		{
			BP->OnVariableRemoved(OriginalWidgetName);
			BP->OnVariableAdded(NewReplacementWidget->GetFName());
		}

		// Delete the widget that has been replaced
		FWidgetBlueprintEditorUtils::DeleteWidgets(BP, { Item }, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::DeleteSilently);

		if (bReplacingDesiredFocus)
		{
			FWidgetBlueprintEditorUtils::SetDesiredFocus(BP, NewReplacementWidget->GetFName());
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);

	for (const TPair<FName, FName>& RenamedWidgets : ReplacedWidgetMap)
	{
		FBlueprintEditorUtils::ReplaceVariableReferences(BP, RenamedWidgets.Key, RenamedWidgets.Value);
	}

	return bReplacedAll;
}

bool FWidgetBlueprintOperationUtils::VerifyWidgetRename(UWidgetBlueprint* WidgetBlueprint, UWidget* TemplateWidget, const FText& NewName, FText& OutErrorMessage)
{
	if (NewName.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("EmptyWidgetName", "Empty Widget Name");
		return false;
	}

	const FString& NewNameString = NewName.ToString();

	if (NewNameString.Len() >= NAME_SIZE)
	{
		OutErrorMessage = LOCTEXT("WidgetNameTooLong", "Widget Name is Too Long");
		return false;
	}

	if (!TemplateWidget)
	{
		// In certain situations, the template might be lost due to mid recompile with focus lost on the rename box at
		// during a strange moment.
		return false;
	}

	// Slug the new name down to a valid object name
	const FName NewNameSlug = Private::SanitizeWidgetName(NewNameString, TemplateWidget->GetFName());

	UWidget* ExistingTemplate = WidgetBlueprint->WidgetTree->FindWidget(NewNameSlug);

	bool bIsSameWidget = false;
	if (ExistingTemplate != nullptr)
	{
		if (TemplateWidget != ExistingTemplate)
		{
			OutErrorMessage = LOCTEXT("ExistingWidgetName", "Existing Widget Name");
			return false;
		}
		else
		{
			bIsSameWidget = true;
		}
	}
	else
	{
		FWidgetBlueprintEditor* BlueprintEditor = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);
		UWidget* PreviewWidget = Private::GetPreviewWidgetFromTemplate(BlueprintEditor, TemplateWidget);

		// Not an existing widget in the tree BUT it still mustn't create a UObject name clash
		if (PreviewWidget)
		{
			// Dummy rename with flag REN_Test returns if rename is possible
			if (!PreviewWidget->Rename(*NewNameSlug.ToString(), nullptr, REN_Test))
			{
				OutErrorMessage = LOCTEXT("ExistingObjectName", "Existing Object Name");
				return false;
			}
		}
		UWidget* RenamedTemplateWidget = TemplateWidget;
		// Dummy rename with flag REN_Test returns if rename is possible
		if (!RenamedTemplateWidget->Rename(*NewNameSlug.ToString(), nullptr, REN_Test))
		{
			OutErrorMessage = LOCTEXT("ExistingObjectName", "Existing Object Name");
			return false;
		}
	}

	FObjectPropertyBase* Property = CastField<FObjectPropertyBase>(WidgetBlueprint->ParentClass->FindPropertyByName(NewNameSlug));
	if (Property && FWidgetBlueprintEditorUtils::IsBindWidgetProperty(Property))
	{
		if (!TemplateWidget->IsA(Property->PropertyClass))
		{
			OutErrorMessage = FText::Format(LOCTEXT("WidgetBindingOfWrongType", "Widget Binding is not type {0}"), Property->PropertyClass->GetDisplayNameText());
			return false;
		}
		return true;
	}

	FKismetNameValidator Validator(WidgetBlueprint);

	// For variable comparison, use the slug
	EValidatorResult ValidatorResult = Validator.IsValid(NewNameSlug);

	if (ValidatorResult != EValidatorResult::Ok)
	{
		if (bIsSameWidget && (ValidatorResult == EValidatorResult::AlreadyInUse || ValidatorResult == EValidatorResult::ExistingName))
		{
			// Continue successfully
		}
		else
		{
			OutErrorMessage = INameValidatorInterface::GetErrorText(NewNameString, ValidatorResult);
			return false;
		}
	}

	return true;
}

bool FWidgetBlueprintOperationUtils::ReplaceWidgetWithTemplate(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, TSubclassOf<UWidget> TemplateClass, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !WidgetToReplace || !TemplateClass)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithTemplate_InvalidParams", "WidgetBlueprint, WidgetToReplace, and TemplateClass must all be valid.");
		return false;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (!WidgetTree || WidgetTree->FindWidget(WidgetToReplace->GetFName()) != WidgetToReplace)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithTemplate_NotInTree", "Widget to replace is not in the blueprint's widget tree.");
		return false;
	}

	// Check circular references using the template class CDO
	UUserWidget* TemplateCDO = Cast<UUserWidget>(TemplateClass->GetDefaultObject());
	if (TemplateCDO && !WidgetBlueprint->IsWidgetFreeFromCircularReferences(TemplateCDO))
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithTemplate_CircularRef", "Template blueprint would create a circular reference.");
		return false;
	}

	UClass* OldClass = WidgetToReplace->GetClass();
	const FName OriginalName = WidgetToReplace->GetFName();

	// BindWidget contract: if a property up the parent class chain is bound to this widget by
	// name (UPROPERTY meta=(BindWidget) / BindWidgetOptional), the new class must satisfy that
	// property's declared widget class — otherwise the binding will be invalid as soon as the
	// blueprint compiles. FindPropertyByName walks ancestors, so checking from ParentClass
	// covers BindWidget declared on any C++ ancestor or intermediate widget blueprint.
	if (UClass* ParentClass = WidgetBlueprint->ParentClass)
	{
		if (FObjectPropertyBase* BindProperty = CastField<FObjectPropertyBase>(ParentClass->FindPropertyByName(OriginalName)))
		{
			if (FWidgetBlueprintEditorUtils::IsBindWidgetProperty(BindProperty)
				&& BindProperty->PropertyClass
				&& !TemplateClass->IsChildOf(BindProperty->PropertyClass))
			{
				OutErrorMessage = FText::Format(
					LOCTEXT("ReplaceWidgetWithTemplate_BindWidgetMismatch",
						"Cannot replace widget '{0}': it is bound to a BindWidget property declared as '{1}', and the new class '{2}' is not a subclass of that type."),
					FText::FromName(OriginalName),
					FText::FromString(BindProperty->PropertyClass->GetName()),
					FText::FromString(TemplateClass->GetName()));
				return false;
			}
		}
	}

	const bool bCanWidgetHaveBlueprintReferences = WidgetToReplace->bIsVariable;
	TSharedPtr<FWidgetTemplateClass> Template = MakeShared<FWidgetTemplateClass>(TemplateClass);

	if (!ReplaceWidgetsWithTemplateClass(WidgetBlueprint, { WidgetToReplace }, Template, FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod::MaintainNameAndReferencesForUnmatchingClass))
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithTemplate_InnerFailed", "Replacement could not be performed (the widget could not be re-parented in the tree).");
		return false;
	}

	if (bCanWidgetHaveBlueprintReferences)
	{
		FixupWidgetBlueprintReferences(WidgetBlueprint, OldClass, TemplateClass, OriginalName);
	}
	return true;
}

void FWidgetBlueprintOperationUtils::RemoveTransientWidgetFromTree(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget)
{
	WidgetBlueprint->WidgetTree->SetFlags(RF_Transactional);
	WidgetBlueprint->WidgetTree->Modify();
	WidgetBlueprint->WidgetTree->RemoveWidget(Widget);
	if (Widget->GetOutermost() != GetTransientPackage())
	{
		Widget->SetFlags(RF_NoFlags);
		Widget->Rename(nullptr, GetTransientPackage());
	}
}

UWidget* FWidgetBlueprintOperationUtils::AddUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClass, FName WidgetName, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !ComponentClass || WidgetName.IsNone())
	{
		OutErrorMessage = LOCTEXT("UIComponentRequiresValidParams", "A valid widget blueprint, component class, and widget name are required.");
		return nullptr;
	}

	FWidgetBlueprintEditor* EditorRaw = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);
	return FUIComponentUtils::AddComponent(EditorRaw, WidgetBlueprint, ComponentClass, WidgetName, OutErrorMessage);
}

bool FWidgetBlueprintOperationUtils::RemoveUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClass, FName WidgetName, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !ComponentClass || WidgetName.IsNone())
	{
		OutErrorMessage = LOCTEXT("UIComponentRequiresValidParams", "A valid widget blueprint, component class, and widget name are required.");
		return false;
	}

	FWidgetBlueprintEditor* EditorRaw = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);
	return FUIComponentUtils::RemoveComponent(EditorRaw, WidgetBlueprint, ComponentClass, WidgetName, OutErrorMessage);
}

bool FWidgetBlueprintOperationUtils::MoveUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClassToMove, UClass* RelativeToComponentClass, FName WidgetName, bool bMoveAfter, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !ComponentClassToMove || !RelativeToComponentClass || WidgetName.IsNone())
	{
		OutErrorMessage = LOCTEXT("MoveUIComponentRequiresBothClasses", "A valid widget blueprint, both component classes, and a widget name are required.");
		return false;
	}

	FWidgetBlueprintEditor* EditorRaw = Private::GetWidgetBlueprintEditorIfOpen(WidgetBlueprint);
	return FUIComponentUtils::MoveComponent(EditorRaw, WidgetBlueprint, ComponentClassToMove, RelativeToComponentClass, WidgetName, bMoveAfter, OutErrorMessage);
}

TArray<UWidget*> FWidgetBlueprintOperationUtils::WrapWidgets(UWidgetBlueprint* BP, TArray<UWidget*> Widgets, UClass* WidgetClass)
{	
	TArray<UWidget*> NewWrappers;
	if (!BP || !WidgetClass || Widgets.IsEmpty())
	{
		return NewWrappers;
	}

	const FScopedTransaction Transaction(LOCTEXT("WrapWidgets", "Wrap Widgets"));

	TSharedPtr<FWidgetTemplateClass> Template = MakeShareable(new FWidgetTemplateClass(WidgetClass));

	FWidgetBlueprintEditor* BlueprintEditor = Private::GetWidgetBlueprintEditorIfOpen(BP);
	const EWidgetDesignFlags DesignerFlags = BlueprintEditor ? BlueprintEditor->GetCurrentDesignerFlags() : EWidgetDesignFlags::None;

	TSet<UWidget*> WidgetSet(Widgets);
	// When selecting multiple widgets, we only want to create a new wrapping widget around the root-most set of widgets.
	// Remove any that are children of other selected widgets (their parents will be wrapped instead).
	TSet<UWidget*> WidgetsToRemove;
	for (UWidget* Widget : WidgetSet)
	{
		int32 OutIndex;
		UPanelWidget* CurrentParent = BP->WidgetTree->FindWidgetParent(Widget, OutIndex);
		if (WidgetSet.Contains(CurrentParent))
		{
			WidgetsToRemove.Add(Widget);
		}
	}
	for (UWidget* Widget : WidgetsToRemove)
	{
		WidgetSet.Remove(Widget);
	}

	// Old Parent -> New Parent Map
	TMap<UPanelWidget*, UPanelWidget*> OldParentToNewParent;

	for (UWidget* Widget : WidgetSet)
	{
		int32 OutIndex;
		UPanelWidget* CurrentParent = BP->WidgetTree->FindWidgetParent(Widget, OutIndex);
		TScriptInterface<INamedSlotInterface> NamedSlotHost = FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(Widget, BP->WidgetTree);

		// If the widget doesn't currently have a slot or parent, and isn't the root, ignore it.
		if (NamedSlotHost == nullptr && CurrentParent == nullptr && Widget != BP->WidgetTree->RootWidget)
		{
			continue;
		}

		Widget->Modify();
		BP->WidgetTree->SetFlags(RF_Transactional);
		BP->WidgetTree->Modify();

		if (NamedSlotHost)
		{
			// If this is a named slot, we need to properly remove and reassign the slot content
			if (UObject* NamedSlotObject = NamedSlotHost.GetObject())
			{
				NamedSlotObject->SetFlags(RF_Transactional);
				NamedSlotObject->Modify();

				UPanelWidget* NewSlotContents = CastChecked<UPanelWidget>(Template->Create(BP->WidgetTree));
				NewSlotContents->SetDesignerFlags(DesignerFlags);

				BP->OnVariableAdded(NewSlotContents->GetFName());

				FWidgetBlueprintEditorUtils::ReplaceNamedSlotHostContent(Widget, NamedSlotHost, NewSlotContents);

				NewSlotContents->AddChild(Widget);
				NewWrappers.AddUnique(NewSlotContents);
			}
		}
		else if (CurrentParent)
		{
			UPanelWidget*& NewWrapperWidget = OldParentToNewParent.FindOrAdd(CurrentParent);
			if (NewWrapperWidget == nullptr || !NewWrapperWidget->CanAddMoreChildren())
			{
				NewWrapperWidget = CastChecked<UPanelWidget>(Template->Create(BP->WidgetTree));
				NewWrapperWidget->SetDesignerFlags(DesignerFlags);

				BP->OnVariableAdded(NewWrapperWidget->GetFName());

				CurrentParent->SetFlags(RF_Transactional);
				CurrentParent->Modify();
				CurrentParent->ReplaceChildAt(OutIndex, NewWrapperWidget);
				NewWrappers.AddUnique(NewWrapperWidget);
			}

			if (NewWrapperWidget != nullptr && NewWrapperWidget->CanAddMoreChildren())
			{
				NewWrapperWidget->Modify();
				NewWrapperWidget->AddChild(Widget);
			}
		}
		else
		{
			UPanelWidget* NewRootContents = CastChecked<UPanelWidget>(Template->Create(BP->WidgetTree));
			NewRootContents->SetDesignerFlags(DesignerFlags);

			BP->OnVariableAdded(NewRootContents->GetFName());

			BP->WidgetTree->RootWidget = NewRootContents;
			NewRootContents->AddChild(Widget);
			NewWrappers.AddUnique(NewRootContents);
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
	return NewWrappers;
}

bool FWidgetBlueprintOperationUtils::ReplaceWidgetWithChild(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !WidgetToReplace)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_InvalidParams", "WidgetBlueprint and WidgetToReplace must both be valid.");
		return false;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (!WidgetTree || WidgetTree->FindWidget(WidgetToReplace->GetFName()) != WidgetToReplace)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_NotInTree", "Widget to replace is not in the blueprint's widget tree.");
		return false;
	}

	UPanelWidget* ExistingPanelTemplate = Cast<UPanelWidget>(WidgetToReplace);
	if (!ExistingPanelTemplate)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_NotPanel", "Widget to replace must be a UPanelWidget.");
		return false;
	}

	if (ExistingPanelTemplate->GetChildrenCount() != 1)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_NotExactlyOneChild", "Panel widget must have exactly one child.");
		return false;
	}

	UWidget* FirstChildTemplate = ExistingPanelTemplate->GetChildAt(0);
	if (!FirstChildTemplate)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_NullChild", "Panel widget's first child is null.");
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ReplaceWidgets", "Replace Widgets"));

	ExistingPanelTemplate->SetFlags(RF_Transactional);
	ExistingPanelTemplate->Modify();

	FirstChildTemplate->SetFlags(RF_Transactional);
	FirstChildTemplate->Modify();

	if (TScriptInterface<INamedSlotInterface> NamedSlotHost = FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(ExistingPanelTemplate, WidgetBlueprint->WidgetTree))
	{
		FWidgetBlueprintEditorUtils::ReplaceNamedSlotHostContent(ExistingPanelTemplate, NamedSlotHost, FirstChildTemplate);
	}
	else if (UPanelWidget* PanelParentTemplate = ExistingPanelTemplate->GetParent())
	{
		PanelParentTemplate->Modify();

		FirstChildTemplate->RemoveFromParent();
		PanelParentTemplate->ReplaceChild(ExistingPanelTemplate, FirstChildTemplate);
	}
	else if (ExistingPanelTemplate == WidgetBlueprint->WidgetTree->RootWidget)
	{
		FirstChildTemplate->RemoveFromParent();

		WidgetBlueprint->WidgetTree->Modify();
		WidgetBlueprint->WidgetTree->RootWidget = FirstChildTemplate;
	}
	else
	{
		Transaction.Cancel();
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithChild_NoParent", "Widget to replace has no parent and is not the root widget.");
		return false;
	}

	FWidgetBlueprintEditorUtils::DeleteWidgets(WidgetBlueprint, {WidgetToReplace}, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::DeleteSilently);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return true;
}

bool FWidgetBlueprintOperationUtils::ReplaceWidgetWithNamedSlot(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FName NamedSlot, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType DeleteWarningType, FText& OutErrorMessage)
{
	if (!WidgetBlueprint || !WidgetToReplace)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithNamedSlot_InvalidParams", "WidgetBlueprint and WidgetToReplace must both be valid.");
		return false;
	}

	if (NamedSlot.IsNone())
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithNamedSlot_NoneSlot", "NamedSlot must not be None.");
		return false;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (!WidgetTree || WidgetTree->FindWidget(WidgetToReplace->GetFName()) != WidgetToReplace)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithNamedSlot_NotInTree", "Widget to replace is not in the blueprint's widget tree.");
		return false;
	}

	INamedSlotInterface* ExistingNamedSlotContainerTemplate = Cast<INamedSlotInterface>(WidgetToReplace);
	if (!ExistingNamedSlotContainerTemplate)
	{
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithNamedSlot_NotNamedSlotHost", "Widget to replace does not implement INamedSlotInterface.");
		return false;
	}

	UWidget* NamedSlotContentTemplate = ExistingNamedSlotContainerTemplate->GetContentForSlot(NamedSlot);
	if (!NamedSlotContentTemplate)
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("ReplaceWidgetWithNamedSlot_EmptySlot", "Named slot '{0}' has no content."),
			FText::FromName(NamedSlot));
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ReplaceWidgets", "Replace Widgets"));

	WidgetToReplace->SetFlags(RF_Transactional);
	WidgetToReplace->Modify();

	NamedSlotContentTemplate->SetFlags(RF_Transactional);
	NamedSlotContentTemplate->Modify();

	if (TScriptInterface<INamedSlotInterface> NamedSlotHost = FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(WidgetToReplace, WidgetBlueprint->WidgetTree))
	{
		FWidgetBlueprintEditorUtils::ReplaceNamedSlotHostContent(WidgetToReplace, NamedSlotHost, NamedSlotContentTemplate);
	}
	else if (UPanelWidget* PanelParentTemplate = WidgetToReplace->GetParent())
	{
		PanelParentTemplate->Modify();

		if (TScriptInterface<INamedSlotInterface> ContentNamedSlotHost = FWidgetBlueprintEditorUtils::FindNamedSlotHostForContent(NamedSlotContentTemplate, WidgetBlueprint->WidgetTree))
		{
			FWidgetBlueprintEditorUtils::RemoveNamedSlotHostContent(NamedSlotContentTemplate, ContentNamedSlotHost);
		}

		PanelParentTemplate->ReplaceChild(WidgetToReplace, NamedSlotContentTemplate);
	}
	else if (WidgetToReplace == WidgetBlueprint->WidgetTree->RootWidget)
	{
		if (UPanelWidget* Parent = NamedSlotContentTemplate->GetParent())
		{
			Parent->Modify();
			NamedSlotContentTemplate->RemoveFromParent();
		}

		WidgetBlueprint->WidgetTree->Modify();
		WidgetBlueprint->WidgetTree->RootWidget = NamedSlotContentTemplate;
	}
	else
	{
		Transaction.Cancel();
		OutErrorMessage = LOCTEXT("ReplaceWidgetWithNamedSlot_NoParent", "Widget to replace has no parent and is not the root widget.");
		return false;
	}

	FWidgetBlueprintEditorUtils::DeleteWidgets(WidgetBlueprint, {WidgetToReplace}, DeleteWarningType);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return true;
}

void FWidgetBlueprintOperationUtils::WalkWidgetTree(
	const UWidgetBlueprint* WidgetBlueprint,
	const UWidget* StartWidget,
	int32 MaxDepth,
	TFunctionRef<void(const UWidget* Widget, int32 Depth, FName SlotName)> Visitor)
{
	if (!WidgetBlueprint || !WidgetBlueprint->WidgetTree) return;

	const UWidget* Root = StartWidget ? StartWidget : WidgetBlueprint->WidgetTree->RootWidget;
	if (!Root) return;

	TSet<const UWidget*> Visited;
	Private::WalkImpl(Root, 0, NAME_None, MaxDepth, Visitor, Visited);
}

int32 FWidgetBlueprintOperationUtils::ComputeWidgetTreeDepth(
	const UWidgetBlueprint* WidgetBlueprint,
	const UWidget* StartWidget)
{
	if (!WidgetBlueprint || !WidgetBlueprint->WidgetTree) return -1;

	const UWidget* Root = StartWidget ? StartWidget : WidgetBlueprint->WidgetTree->RootWidget;
	if (!Root) return 0;

	TSet<const UWidget*> Visited;
	return Private::ComputeDepthImpl(Root, Visited);
}

#undef LOCTEXT_NAMESPACE