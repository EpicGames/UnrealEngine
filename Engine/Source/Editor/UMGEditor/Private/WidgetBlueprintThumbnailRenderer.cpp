// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintThumbnailRenderer.h"

#include "Blueprint/UserWidget.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalRenderResources.h"
#include "Input/HittestGrid.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Slate/WidgetRenderer.h"
#include "TextureResource.h"
#include "WidgetBlueprint.h"
#include "Widgets/SVirtualWindow.h"
#include "WidgetBlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprintThumbnailRenderer)

#define LOCTEXT_NAMESPACE "UWidgetBlueprintThumbnailRenderer"

class FWidgetBlueprintThumbnailPool
{
public:
	struct FInstance
	{
		TWeakObjectPtr<UClass> WidgetClass;
		TWeakObjectPtr<UUserWidget> Widget;
	};

public:
	static constexpr int32 MaxNumInstance = 50;

	FWidgetBlueprintThumbnailPool()
	{
		InstancedThumbnails.Reserve(MaxNumInstance);
	}

	~FWidgetBlueprintThumbnailPool()
	{
		Clear();
	}

	FInstance* FindThumbnail(const UClass* InClass) const
	{
		check(InClass);
		const FName ClassName = InClass->GetFName();

		return InstancedThumbnails.FindRef(ClassName);
	}

	FInstance& EnsureThumbnail(const UClass* InClass)
	{
		check(InClass);
		const FName ClassName = InClass->GetFName();

		FInstance* ExistingThumbnail = InstancedThumbnails.FindRef(ClassName);
		if (!ExistingThumbnail)
		{
			if (InstancedThumbnails.Num() >= MaxNumInstance)
			{
				Clear();
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}

			ExistingThumbnail = new FInstance;
			InstancedThumbnails.Add(ClassName, ExistingThumbnail);
		}

		return *ExistingThumbnail;
	}

	void RemoveThumbnail(const UClass* InClass)
	{
		check(InClass);
		InstancedThumbnails.Remove(InClass->GetFName());
	}

	void Clear()
	{
		for(auto& Instance : InstancedThumbnails)
		{
			delete Instance.Value;
		}
		InstancedThumbnails.Reset();
	}

private:
	TMap<FName, FInstance*> InstancedThumbnails;
};


void UWidgetBlueprintThumbnailRenderer::FWidgetBlueprintThumbnailPoolDeleter::operator()(FWidgetBlueprintThumbnailPool* Pointer)
{
	delete Pointer;
}


UWidgetBlueprintThumbnailRenderer::UWidgetBlueprintThumbnailRenderer()
	: ThumbnailPool(new FWidgetBlueprintThumbnailPool)
{
	FKismetEditorUtilities::OnBlueprintUnloaded.AddUObject(this, &UWidgetBlueprintThumbnailRenderer::OnBlueprintUnloaded);
}


UWidgetBlueprintThumbnailRenderer::~UWidgetBlueprintThumbnailRenderer()
{
	FKismetEditorUtilities::OnBlueprintUnloaded.RemoveAll(this);
}


bool UWidgetBlueprintThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UWidgetBlueprint* Blueprint = Cast<UWidgetBlueprint>(Object);
	return (Blueprint && Blueprint->GeneratedClass && Blueprint->GeneratedClass->IsChildOf(UWidget::StaticClass()));
}


void UWidgetBlueprintThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
#if !UE_SERVER

	if (Width < 1 || Height < 1)
	{
		return;
	}

	if (!FApp::CanEverRender())
	{
		return;
	}

	if (!RenderTarget)
	{
		return;
	}

	UWidgetBlueprint* WidgetBlueprintToRender = Cast<UWidgetBlueprint>(Object);
	const bool bIsBlueprintValid = IsValid(WidgetBlueprintToRender)
		&& IsValid(WidgetBlueprintToRender->GeneratedClass)
		&& WidgetBlueprintToRender->bHasBeenRegenerated
		&& !WidgetBlueprintToRender->bBeingCompiled
		&& !WidgetBlueprintToRender->HasAnyFlags(RF_Transient)
		&& !WidgetBlueprintToRender->GeneratedClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract)
		&& WidgetBlueprintToRender->GeneratedClass->IsChildOf(UWidget::StaticClass());
	if (!bIsBlueprintValid)
	{
		return;
	}

	// Create a plain gray background for the thumbnail
	const int32 SizeOfUV = 1;
	FLinearColor GrayBackgroundColor(FVector4(.03f, .03f, .03f, 1.0f));
	FCanvasTileItem TileItem(FVector2D(X, Y),  GWhiteTexture, FVector2D(Width, Height), FVector2D(0, 0), FVector2D(SizeOfUV, SizeOfUV), GrayBackgroundColor);
	TileItem.BlendMode = SE_BLEND_AlphaBlend;
	TileItem.Draw(Canvas);

	// check if an image is used instead of auto generating the thumbnail
	if (WidgetBlueprintToRender->ThumbnailImage)
	{
		FVector2D TextureSize(WidgetBlueprintToRender->ThumbnailImage->GetSizeX(), WidgetBlueprintToRender->ThumbnailImage->GetSizeY());
		if (TextureSize.X > SMALL_NUMBER && TextureSize.Y > SMALL_NUMBER)
		{
			TTuple<float, FVector2D> ScaleAndOffset = FWidgetBlueprintEditorUtils::GetThumbnailImageScaleAndOffset(TextureSize, FVector2D(Width, Height));
			float Scale = ScaleAndOffset.Get<0>();
			FVector2D Offset = ScaleAndOffset.Get<1>();
			FVector2D ThumbnailImageOffset = Offset;
			FVector2D ThumbnailImageScaledSize = Scale * TextureSize;

			FCanvasTileItem CanvasTile(ThumbnailImageOffset, WidgetBlueprintToRender->ThumbnailImage->GetResource(), ThumbnailImageScaledSize, FLinearColor::White);
			CanvasTile.BlendMode = SE_BLEND_Translucent;
			CanvasTile.Draw(Canvas);
		}
	}
	else
	{
		Canvas->Flush_GameThread();
		UUserWidget* WidgetInstance = nullptr;

		{
			UClass* ClassToGenerate = WidgetBlueprintToRender->GeneratedClass;
			FWidgetBlueprintThumbnailPool::FInstance& ThumbnailInstance = ThumbnailPool->EnsureThumbnail(ClassToGenerate);

			WidgetInstance = ThumbnailInstance.Widget.Get();
			if (!WidgetInstance)
			{
				UWorld* World = GEditor->GetEditorWorldContext().World();
				ThumbnailInstance.Widget = NewObject<UUserWidget>(World, ClassToGenerate, NAME_None, RF_Transient);
				ThumbnailInstance.Widget->SetDesignerFlags(EWidgetDesignFlags::Designing | EWidgetDesignFlags::ExecutePreConstruct);
				ThumbnailInstance.Widget->Initialize();

				WidgetInstance = ThumbnailInstance.Widget.Get();
			}
		}

		FVector2D ThumbnailSize(Width, Height);
		TOptional<FWidgetBlueprintEditorUtils::FWidgetThumbnailProperties> ScaleAndOffset;
		if (WidgetBlueprintToRender->ThumbnailSizeMode == EThumbnailPreviewSizeMode::Custom)
		{
			ScaleAndOffset = FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetForThumbnail(WidgetInstance, Canvas->GetRenderTarget(), ThumbnailSize, WidgetBlueprintToRender->ThumbnailCustomSize, WidgetBlueprintToRender->ThumbnailSizeMode);
		}
		else
		{
			ScaleAndOffset = FWidgetBlueprintEditorUtils::DrawSWidgetInRenderTargetForThumbnail(WidgetInstance, Canvas->GetRenderTarget(), ThumbnailSize, TOptional<FVector2D>(), WidgetBlueprintToRender->ThumbnailSizeMode);

		}
		if (!ScaleAndOffset.IsSet())
		{
			return;
		}
	}
#endif
}

EThumbnailRenderFrequency UWidgetBlueprintThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return EThumbnailRenderFrequency::OnAssetSave;
}

void UWidgetBlueprintThumbnailRenderer::OnBlueprintUnloaded(UBlueprint* Blueprint)
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		ThumbnailPool->RemoveThumbnail(Blueprint->GeneratedClass);
	}
}

#undef LOCTEXT_NAMESPACE
