// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetCompilerRule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetCompilerRule)

UWidgetCompilerRule::UWidgetCompilerRule()
{
}

void UWidgetCompilerRule::ExecuteRule(UWidgetBlueprint* WidgetBlueprint, FCompilerResultsLog& MessageLog)
{
}
