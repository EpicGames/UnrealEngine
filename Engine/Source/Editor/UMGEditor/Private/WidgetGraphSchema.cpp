// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetGraphSchema.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_RemoveDelegate.h"
#include "K2Node_ClearDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_WidgetAnimationEvent.h"
#include "Animation/WidgetAnimation.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_Self.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetGraphSchema)

UWidgetGraphSchema::UWidgetGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

EWidgetAnimationEvent GetAnimationEventFromDelegateName(FName DelegateName)
{
	if (DelegateName == TEXT("OnAnimationStarted"))
	{
		return EWidgetAnimationEvent::Started;
	}
	else if (DelegateName == TEXT("OnAnimationFinished"))
	{
		return EWidgetAnimationEvent::Finished;
	}

	ensure(false);
	return EWidgetAnimationEvent::Started;
}

void UWidgetGraphSchema::BackwardCompatibilityNodeConversion(UEdGraph* Graph, bool bOnlySafeChanges) const
{
	if (Graph)
	{
		if (UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(Graph->GetOuter()))
		{
			const int32 WidgetBPVersion = WidgetBlueprint->GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);

			if (WidgetBPVersion < FFortniteMainBranchObjectVersion::WidgetStopDuplicatingAnimations)
			{
				ConvertAnimationEventNodes(Graph);

				ConvertAddAnimationDelegate(Graph);
				ConvertRemoveAnimationDelegate(Graph);
				ConvertClearAnimationDelegate(Graph);
			}
			else if (WidgetBPVersion < FFortniteMainBranchObjectVersion::WidgetAnimationDefaultToSelfFail)
			{
				FixDefaultToSelfForAnimation(Graph);
			}
		}
	}

	Super::BackwardCompatibilityNodeConversion(Graph, bOnlySafeChanges);
}

void UWidgetGraphSchema::ConvertAnimationEventNodes(UEdGraph* Graph) const
{
	TArray<UK2Node_ComponentBoundEvent*> ComponentBoundEventNodes;
	Graph->GetNodesOfClass<UK2Node_ComponentBoundEvent>(ComponentBoundEventNodes);

	for (UK2Node_ComponentBoundEvent* Node : ComponentBoundEventNodes)
	{
		if (Node->DelegateOwnerClass == UWidgetAnimation::StaticClass())
		{
			UBlueprintNodeSpawner* GetItemNodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_WidgetAnimationEvent::StaticClass(), /*Outer =*/nullptr);

			FVector2D NodePos(Node->NodePosX, Node->NodePosY);
			IBlueprintNodeBinder::FBindingSet Bindings;
			UK2Node_WidgetAnimationEvent* GetItemNode = Cast<UK2Node_WidgetAnimationEvent>(GetItemNodeSpawner->Invoke(Graph, Bindings, NodePos));
			GetItemNode->SourceWidgetBlueprint = CastChecked<UWidgetBlueprint>(Graph->GetOuter());
			GetItemNode->Action = GetAnimationEventFromDelegateName(Node->DelegatePropertyName);
			GetItemNode->AnimationPropertyName = Node->ComponentPropertyName;

			TMap<FName, FName> OldToNewPinMap;
			ReplaceOldNodeWithNew(Node, GetItemNode, OldToNewPinMap);

			GetItemNode->MarkDirty();
		}
	}
}

void UWidgetGraphSchema::ConvertAddAnimationDelegate(UEdGraph* Graph) const
{
	UBlueprintNodeSpawner* CallFunctionSpawner = UBlueprintNodeSpawner::Create(UK2Node_CallFunction::StaticClass(), /*Outer =*/nullptr);

	TArray<UK2Node_AddDelegate*> AddDelegateNodes;
	Graph->GetNodesOfClass<UK2Node_AddDelegate>(AddDelegateNodes);

	for (UK2Node_AddDelegate* Node : AddDelegateNodes)
	{
		if (Node->DelegateReference.GetMemberParentClass() == UWidgetAnimation::StaticClass())
		{
			FVector2D NodePos(Node->NodePosX, Node->NodePosY);
			IBlueprintNodeBinder::FBindingSet Bindings;
			UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(CallFunctionSpawner->Invoke(Graph, Bindings, NodePos));

			TSubclassOf<UObject> FunctionClass = Node->FindPinChecked(UEdGraphSchema_K2::PN_Self)->LinkedTo.Num() > 1 ? UWidgetAnimation::StaticClass() : UUserWidget::StaticClass();

			switch (GetAnimationEventFromDelegateName(Node->DelegateReference.GetMemberName()))
			{
			case EWidgetAnimationEvent::Started:
				CallFunction->FunctionReference.SetExternalMember(TEXT("BindToAnimationStarted"), FunctionClass);
				break;
			case EWidgetAnimationEvent::Finished:
				CallFunction->FunctionReference.SetExternalMember(TEXT("BindToAnimationFinished"), FunctionClass);
				break;
			}

			ReplaceAnimationFunctionAndAllocateDefaultPins(Graph, Node, CallFunction);
		}
	}
}

void UWidgetGraphSchema::ConvertRemoveAnimationDelegate(UEdGraph* Graph) const
{
	UBlueprintNodeSpawner* CallFunctionSpawner = UBlueprintNodeSpawner::Create(UK2Node_CallFunction::StaticClass(), /*Outer =*/nullptr);

	TArray<UK2Node_RemoveDelegate*> RemoveDelegateNodes;
	Graph->GetNodesOfClass<UK2Node_RemoveDelegate>(RemoveDelegateNodes);

	for (UK2Node_RemoveDelegate* Node : RemoveDelegateNodes)
	{
		if (Node->DelegateReference.GetMemberParentClass() == UWidgetAnimation::StaticClass())
		{
			FVector2D NodePos(Node->NodePosX, Node->NodePosY);
			IBlueprintNodeBinder::FBindingSet Bindings;
			UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(CallFunctionSpawner->Invoke(Graph, Bindings, NodePos));

			TSubclassOf<UObject> FunctionClass = Node->FindPinChecked(UEdGraphSchema_K2::PN_Self)->LinkedTo.Num() > 1 ? UWidgetAnimation::StaticClass() : UUserWidget::StaticClass();

			switch (GetAnimationEventFromDelegateName(Node->DelegateReference.GetMemberName()))
			{
			case EWidgetAnimationEvent::Started:
				CallFunction->FunctionReference.SetExternalMember(TEXT("UnbindFromAnimationStarted"), FunctionClass);
				break;
			case EWidgetAnimationEvent::Finished:
				CallFunction->FunctionReference.SetExternalMember(TEXT("UnbindFromAnimationFinished"), FunctionClass);
				break;
			}

			ReplaceAnimationFunctionAndAllocateDefaultPins(Graph, Node, CallFunction);
		}
	}
}

void UWidgetGraphSchema::ConvertClearAnimationDelegate(UEdGraph* Graph) const
{
	UBlueprintNodeSpawner* CallFunctionSpawner = UBlueprintNodeSpawner::Create(UK2Node_CallFunction::StaticClass(), /*Outer =*/nullptr);

	TArray<UK2Node_ClearDelegate*> ClearDelegateNodes;
	Graph->GetNodesOfClass<UK2Node_ClearDelegate>(ClearDelegateNodes);

	for (UK2Node_ClearDelegate* Node : ClearDelegateNodes)
	{
		if (Node->DelegateReference.GetMemberParentClass() == UWidgetAnimation::StaticClass())
		{
			FVector2D NodePos(Node->NodePosX, Node->NodePosY);
			IBlueprintNodeBinder::FBindingSet Bindings;
			UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(CallFunctionSpawner->Invoke(Graph, Bindings, NodePos));

			TSubclassOf<UObject> FunctionClass = Node->FindPinChecked(UEdGraphSchema_K2::PN_Self)->LinkedTo.Num() > 1 ? UWidgetAnimation::StaticClass() : UUserWidget::StaticClass();

			switch (GetAnimationEventFromDelegateName(Node->DelegateReference.GetMemberName()))
			{
			case EWidgetAnimationEvent::Started:
				CallFunction->FunctionReference.SetExternalMember(TEXT("UnbindAllFromAnimationStarted"), FunctionClass);
				break;
			case EWidgetAnimationEvent::Finished:
				CallFunction->FunctionReference.SetExternalMember(TEXT("UnbindAllFromAnimationFinished"), FunctionClass);
				break;
			}

			ReplaceAnimationFunctionAndAllocateDefaultPins(Graph, Node, CallFunction);
		}
	}
}

void UWidgetGraphSchema::ReplaceAnimationFunctionAndAllocateDefaultPins(UEdGraph* Graph, UK2Node* OldNode, UK2Node_CallFunction* NewFunctionNode) const
{
	NewFunctionNode->AllocateDefaultPins();

	TMap<FName, FName> OldToNewPinMap;
	if (NewFunctionNode->FindPin(TEXT("Animation")))
	{
		OldToNewPinMap.Add(UEdGraphSchema_K2::PN_Self, TEXT("Animation"));
	}
	OldToNewPinMap.Add(TEXT("Delegate"), TEXT("Delegate"));
	ReplaceOldNodeWithNew(OldNode, NewFunctionNode, OldToNewPinMap);

	UEdGraphPin* WidgetPin = NewFunctionNode->FindPin(TEXT("Widget"), EGPD_Input);
	if (WidgetPin && WidgetPin->LinkedTo.Num() == 0)
	{
		FVector2D PinPos(NewFunctionNode->NodePosX - 200, NewFunctionNode->NodePosY + 128);
		IBlueprintNodeBinder::FBindingSet Bindings;
		UK2Node_Self* SelfNode = Cast<UK2Node_Self>(UBlueprintNodeSpawner::Create<UK2Node_Self>()->Invoke(Graph, Bindings, PinPos));

		if (!TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), WidgetPin))
		{
			SelfNode->DestroyNode();
		}
	}
}

void UWidgetGraphSchema::FixDefaultToSelfForAnimation(UEdGraph* Graph) const
{
	TArray<UK2Node_CallFunction*> CallFunctionNodes;
	Graph->GetNodesOfClass<UK2Node_CallFunction>(CallFunctionNodes);

	TArray<UFunction*> AnimationFunctionsToFix;
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("BindToAnimationStarted")));
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("UnbindFromAnimationStarted")));
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("UnbindAllFromAnimationStarted")));
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("BindToAnimationFinished")));
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("UnbindFromAnimationFinished")));
	AnimationFunctionsToFix.Add(UWidgetAnimation::StaticClass()->FindFunctionByName(TEXT("UnbindAllFromAnimationFinished")));

	for (UK2Node_CallFunction* FunctionNode : CallFunctionNodes)
	{
		UFunction* Function = FunctionNode->GetTargetFunction();
		if (AnimationFunctionsToFix.Contains(Function))
		{
			UEdGraphPin* WidgetPin = FunctionNode->FindPin(TEXT("Widget"), EGPD_Input);
			if (WidgetPin && WidgetPin->LinkedTo.Num() == 0)
			{
				FVector2D PinPos(FunctionNode->NodePosX - 200, FunctionNode->NodePosY + 128);
				IBlueprintNodeBinder::FBindingSet Bindings;
				UK2Node_Self* SelfNode = Cast<UK2Node_Self>(UBlueprintNodeSpawner::Create<UK2Node_Self>()->Invoke(Graph, Bindings, PinPos));

				if (!TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), WidgetPin))
				{
					SelfNode->DestroyNode();
				}
			}
		}
	}
}
