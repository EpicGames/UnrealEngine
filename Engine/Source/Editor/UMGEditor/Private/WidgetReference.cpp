// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetReference.h"
#include "WidgetBlueprintEditor.h"

#define LOCTEXT_NAMESPACE "UMG"

FWidgetHandle::FWidgetHandle(UWidget* InWidget)
	: Widget(InWidget)
{

}

// FWidgetReference

FWidgetReference::FWidgetReference()
	: WidgetEditor()
	, TemplateHandle()
{

}

FWidgetReference::FWidgetReference(TSharedPtr<FWidgetBlueprintEditor> InWidgetEditor, TSharedPtr< FWidgetHandle > InTemplateHandle)
	: WidgetEditor(InWidgetEditor)
	, TemplateHandle(InTemplateHandle)
{
	
}

bool FWidgetReference::IsValid() const
{
	if ( TemplateHandle.IsValid() )
	{
		return TemplateHandle->Widget.Get() != nullptr && GetPreview();
	}

	return false;
}

UWidget* FWidgetReference::GetTemplate() const
{
	if ( TemplateHandle.IsValid() )
	{
		return TemplateHandle->Widget.Get();
	}

	return nullptr;
}

UWidget* FWidgetReference::GetPreview() const
{
	if ( WidgetEditor.IsValid() && TemplateHandle.IsValid() )
	{
		if ( UUserWidget* PreviewRoot = WidgetEditor.Pin()->GetPreview() )
		{
			if ( UWidget* TemplateWidget = TemplateHandle->Widget.Get() )
			{
				UWidget* PreviewWidget = PreviewRoot->GetWidgetFromName(TemplateWidget->GetFName());
				return PreviewWidget;
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
