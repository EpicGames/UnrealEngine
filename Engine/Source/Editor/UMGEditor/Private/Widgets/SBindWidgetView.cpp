// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SBindWidgetView.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprintEditorUtils.h"

#include "IDocumentation.h"
#include "Editor.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#include "Animation/WidgetAnimation.h"


#define LOCTEXT_NAMESPACE "UMG.BindWidget"

void FBindWidgetCommands::RegisterCommands() 
{
	UI_COMMAND( GotoNativeVarDefinition, "Goto Code Definition", "Goto the native code definition of this variable", EUserInterfaceActionType::Button, FInputChord() );
}

namespace UE
{
namespace UMG
{
static FName Column_PropertyName = "PropertyName";
static FName Column_WidgetType = "WidgetType";
static FName Column_Optional = "Optional";


/**
 * 
 */
struct FBindWidgetListEntry
{
	enum class EEntryType
	{
		Category,
		Binding,
	};
	enum class EBindingType
	{
		Widget,		//UWidget
		Animation,	//UWidgetAnimation
	};

	EEntryType EntryType = EEntryType::Category;
	EBindingType BindingType = EBindingType::Widget;

	FObjectProperty* Property = nullptr;
	bool bIsOptional = false;
	bool bIsBound = false;
	bool bIsCorrectClass = false;

	FText CategoryName;
};

typedef TSharedPtr<FBindWidgetListEntry> FBindWidgetListEntryPtr;

/**
 *
 */
class SBindWidgetListRow : public SMultiColumnTableRow<FBindWidgetListEntryPtr>
{
public:
	SLATE_BEGIN_ARGS(SBindWidgetListRow) {}
		SLATE_ARGUMENT(FBindWidgetListEntryPtr, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		EntryPtr = Args._Entry;

		SMultiColumnTableRow<FBindWidgetListEntryPtr>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f)
			.ShowSelection(EntryPtr->EntryType == FBindWidgetListEntry::EEntryType::Binding)
			,
			OwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (EntryPtr->EntryType == FBindWidgetListEntry::EEntryType::Binding)
		{
			if (ColumnName == Column_PropertyName)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(EntryPtr->Property->GetFName()))
					.ToolTipText(this, &SBindWidgetListRow::HandleGetTooltip);
					
			}
			if (ColumnName == Column_WidgetType)
			{
				const FSlateBrush* Icon = FSlateIconFinder::FindIconBrushForClass(EntryPtr->Property->PropertyClass);
				return SNew(SHorizontalBox)
					.ToolTipText(this, &SBindWidgetListRow::HandleGetTooltip)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.f, 2.f)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(Icon)
					]
					+ SHorizontalBox::Slot()
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(STextBlock)
						.Text(EntryPtr->Property->PropertyClass->GetDisplayNameText())
					];
			}
			if (ColumnName == Column_Optional)
			{
				const FSlateBrush* Image = nullptr;
				FText Tooltip = FText::GetEmpty();
				if (EntryPtr->bIsBound && EntryPtr->bIsCorrectClass)
				{
					Image = FAppStyle::GetBrush("Icons.SuccessWithColor");
					Tooltip = LOCTEXT("BoundCorrectlyTooltip", "The widget is bound");
				}
				else if (EntryPtr->bIsBound) // is not of the correct class
				{
					Image = FAppStyle::GetBrush("Icons.ErrorWithColor");
					Tooltip = LOCTEXT("BoundWidgetWrongClassTooltip", "The bound widget is not of the correct type.");
				}
				else if (EntryPtr->bIsOptional) // is not of the correct class
				{
					Image = FAppStyle::GetBrush("Icons.FilledCircle");
					Tooltip = LOCTEXT("BoundOptionalTooltip", "The widget is not bound but it is optional.");
				}
				else
				{
					Image = FAppStyle::GetBrush("Icons.ErrorWithColor");
					Tooltip = LOCTEXT("BoundNoWidgetTooltip", "The widget is not bound.");
				}

				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(Image)
					]
					.ToolTipText(Tooltip);
			}
		}

		return SNullWidget::NullWidget;
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (EntryPtr->EntryType == FBindWidgetListEntry::EEntryType::Binding
			&& EntryPtr->Property
			&& FSourceCodeNavigation::CanNavigateToProperty(EntryPtr->Property))
		{
			FSourceCodeNavigation::NavigateToProperty(EntryPtr->Property);
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

private:
	FText HandleGetTooltip() const
	{
		return EntryPtr->Property->GetToolTipText();
	}

private:
	FBindWidgetListEntryPtr EntryPtr;
};


/**
 *
 */
class SBindWidgetView : public STreeView<FBindWidgetListEntryPtr>
{
private:
	using Super = STreeView<FBindWidgetListEntryPtr>;

public:
	SLATE_BEGIN_ARGS(SBindWidgetView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, TArray<FBindWidgetListEntryPtr> InSourceData)
	{
		AllSourceData = MoveTemp(InSourceData);
		{
			TSharedRef<FBindWidgetListEntry> Ref = MakeShared<FBindWidgetListEntry>();
			Ref->CategoryName = LOCTEXT("Widget", "Widget");
			Ref->EntryType = FBindWidgetListEntry::EEntryType::Category;
			Ref->BindingType = FBindWidgetListEntry::EBindingType::Widget;

			CategorySourceData.Add(Ref);
		}
		{
			TSharedRef<FBindWidgetListEntry> Ref = MakeShared<FBindWidgetListEntry>();
			Ref->CategoryName = LOCTEXT("Animation", "Animation");
			Ref->EntryType = FBindWidgetListEntry::EEntryType::Category;
			Ref->BindingType = FBindWidgetListEntry::EBindingType::Animation;

			CategorySourceData.Add(Ref);
		}

		Super::Construct(Super::FArguments()
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SBindWidgetView::HandleGenerateRow)
			.OnGetChildren(this, &SBindWidgetView::HandleGetChildren)
			.OnContextMenuOpening(this, &SBindWidgetView::OnContextMenuOpening)
			.OnIsSelectableOrNavigable(this, &SBindWidgetView::OnIsSelectableOrNavigable)
			.TreeItemsSource(&CategorySourceData)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(Column_WidgetType)
				.FillWidth(0.5f)
				.DefaultLabel(LOCTEXT("TypeHeaderName", "Type"))
				+ SHeaderRow::Column(Column_PropertyName)
				.FillWidth(0.5f)
				.DefaultLabel(LOCTEXT("PropertyNameHeaderName", "Property"))
				+ SHeaderRow::Column(Column_Optional)
				.FixedWidth(16.f)
				.DefaultLabel(FText())
			));

		CommandList = MakeShareable(new FUICommandList);
		CommandList->MapAction(FBindWidgetCommands::Get().GotoNativeVarDefinition,
			FExecuteAction::CreateSP(this, &SBindWidgetView::GotoNativeCodeVarDefinition),
			FCanExecuteAction(),
			FIsActionChecked());

		// Default to open
		for (const auto& Cat : CategorySourceData)
		{
			this->SetItemExpansion(Cat, true);
		}
	}

	void SetSourceData(TArray<FBindWidgetListEntryPtr> InSourceData)
	{
		AllSourceData = MoveTemp(InSourceData);
		RebuildList();
	}

private:
	TSharedRef<ITableRow> HandleGenerateRow(FBindWidgetListEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		if (Entry->EntryType == FBindWidgetListEntry::EEntryType::Category)
		{
			return SNew(STableRow<FBindWidgetListEntryPtr>, OwnerTable)
				.Style(FAppStyle::Get(), "UMGEditor.PaletteHeader")
				.Padding(5.0f)
				.ShowSelection(false)
				[
					SNew(STextBlock)
					.TransformPolicy(ETextTransformPolicy::ToUpper)
					.Text(Entry->CategoryName)
					.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
				];
		}
		else
		{
			return SNew(SBindWidgetListRow, OwnerTable)
				.Entry(Entry);
		}
	}

	void HandleGetChildren(FBindWidgetListEntryPtr Parent, TArray<FBindWidgetListEntryPtr>& FilteredChildren)
	{
		if (Parent->EntryType == FBindWidgetListEntry::EEntryType::Category)
		{
			for(const FBindWidgetListEntryPtr& Data : AllSourceData)
			{
				if (Data->EntryType == FBindWidgetListEntry::EEntryType::Binding && Data->BindingType == Parent->BindingType)
				{
					FilteredChildren.Add(Data);
				}
			}
		}
	}

	TSharedPtr<SWidget> OnContextMenuOpening()
	{
		const TArray<FBindWidgetListEntryPtr> CurSelectedItems = GetSelectedItems();
		if (CurSelectedItems.Num() != 1)
		{
			return TSharedPtr<SWidget>();
		}

		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.AddMenuEntry(FBindWidgetCommands::Get().GotoNativeVarDefinition);

		return MenuBuilder.MakeWidget();
	}

	bool OnIsSelectableOrNavigable(FBindWidgetListEntryPtr InItem) const
	{
		return InItem->EntryType == FBindWidgetListEntry::EEntryType::Binding;
	}

	void GotoNativeCodeVarDefinition()
	{
		const TArray<FBindWidgetListEntryPtr> CurSelectedItems = GetSelectedItems();
		if (CurSelectedItems.Num() == 1)
		{
			if (CurSelectedItems[0]->Property
				&& FSourceCodeNavigation::CanNavigateToProperty(CurSelectedItems[0]->Property))
			{
				FSourceCodeNavigation::NavigateToProperty(CurSelectedItems[0]->Property);
			}
		}
	}
	
private:
	TArray<FBindWidgetListEntryPtr> AllSourceData;
	TArray<FBindWidgetListEntryPtr> CategorySourceData;
	TSharedPtr<FUICommandList> CommandList;
};


TArray<UWidget*> GetAllSourceWidgets(UWidgetBlueprint* WidgetBlueprint)
{
	TArray<UWidget*> Widgets;
	UWidgetBlueprint* WidgetBPToScan = WidgetBlueprint;
	while (WidgetBPToScan != nullptr)
	{
		Widgets = WidgetBPToScan->GetAllSourceWidgets();
		if (Widgets.Num() != 0)
		{
			break;
		}
		// Get the parent WidgetBlueprint
		WidgetBPToScan = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy ? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy) : nullptr;
	}
	return Widgets;
}


TArray<UWidgetAnimation*> GetAllSourceWidgetAnimations(UWidgetBlueprint* WidgetBlueprint)
{
	TArray<UWidgetAnimation*> WidgetAnimations;
	UWidgetBlueprint* WidgetBPToScan = WidgetBlueprint;
	while (WidgetBPToScan != nullptr)
	{
		WidgetAnimations.Append(WidgetBPToScan->Animations);
		WidgetBPToScan = WidgetBPToScan->ParentClass && WidgetBPToScan->ParentClass->ClassGeneratedBy ? Cast<UWidgetBlueprint>(WidgetBPToScan->ParentClass->ClassGeneratedBy) : nullptr;
	}
	return WidgetAnimations;
}


TArray<FBindWidgetListEntryPtr> BuildSourceDataList(TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	TArray<FBindWidgetListEntryPtr> Result;

	UWidgetBlueprint* WidgetBlueprint = CastChecked<UWidgetBlueprint>(InBlueprintEditor->GetBlueprintObj());
	TArray<UWidget*> AllWidgets = GetAllSourceWidgets(WidgetBlueprint);
	TArray<UWidgetAnimation*> AllWidgetAnimations = GetAllSourceWidgetAnimations(WidgetBlueprint);

	for (FObjectProperty* WidgetProperty : TFieldRange<FObjectProperty>(WidgetBlueprint->ParentClass, EFieldIterationFlags::IncludeSuper))
	{
		if (WidgetProperty->PropertyClass->IsChildOf(UWidget::StaticClass()))
		{
			bool bIsOptional = false;
			if (FWidgetBlueprintEditorUtils::IsBindWidgetProperty(WidgetProperty, bIsOptional))
			{
				TSharedRef<FBindWidgetListEntry> Ref = MakeShared<FBindWidgetListEntry>();
				Ref->Property = WidgetProperty;
				Ref->bIsOptional = bIsOptional;
				Ref->EntryType = FBindWidgetListEntry::EEntryType::Binding;
				Ref->BindingType = FBindWidgetListEntry::EBindingType::Widget;

				int32 FoundIndex = AllWidgets.IndexOfByPredicate([WidgetProperty](UWidget* Widget){ return Widget->GetFName() == WidgetProperty->GetFName();});
				Ref->bIsBound = FoundIndex != INDEX_NONE;
				if (Ref->bIsBound)
				{
					Ref->bIsCorrectClass = AllWidgets[FoundIndex]->IsA(WidgetProperty->PropertyClass);
				}

				Result.Add(Ref);
			}
		}
		else if (WidgetProperty->PropertyClass->IsChildOf(UWidgetAnimation::StaticClass()))
		{
			bool bIsOptional = false;
			if (FWidgetBlueprintEditorUtils::IsBindWidgetAnimProperty(WidgetProperty, bIsOptional))
			{
				TSharedRef<FBindWidgetListEntry> Ref = MakeShared<FBindWidgetListEntry>();
				Ref->Property = WidgetProperty;
				Ref->bIsOptional = bIsOptional;
				Ref->EntryType = FBindWidgetListEntry::EEntryType::Binding;
				Ref->BindingType = FBindWidgetListEntry::EBindingType::Animation;

				int32 FoundIndex = AllWidgetAnimations.IndexOfByPredicate([WidgetProperty](UWidgetAnimation* Widget) { return Widget->GetFName() == WidgetProperty->GetFName(); });
				Ref->bIsBound = FoundIndex != INDEX_NONE;
				if (Ref->bIsBound)
				{
					Ref->bIsCorrectClass = AllWidgetAnimations[FoundIndex]->IsA(WidgetProperty->PropertyClass);
				}

				Result.Add(Ref);
			}
		}
	}

	return Result;
}


UWidgetBlueprint* GetWidgetBlueprint(TWeakPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	if (TSharedPtr<FWidgetBlueprintEditor> Pin = InBlueprintEditor.Pin())
	{
		return CastChecked<UWidgetBlueprint>(Pin->GetBlueprintObj());
	}
	return nullptr;
}

} // namespace UMG
} // namespace UE


void SBindWidgetView::Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;
	bRefreshRequested = false;

	// register for any objects replaced
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SBindWidgetView::HandleObjectsReplaced);

	UWidgetBlueprint* Blueprint = CastChecked<UWidgetBlueprint>(InBlueprintEditor->GetBlueprintObj());
	Blueprint->OnChanged().AddRaw(this, &SBindWidgetView::HandleBlueprintChanged);
	Blueprint->OnCompiled().AddRaw(this, &SBindWidgetView::HandleBlueprintChanged);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBrush"))
			[
				SAssignNew(ListView, UE::UMG::SBindWidgetView, UE::UMG::BuildSourceDataList(InBlueprintEditor))
			]
		]
	];
}


SBindWidgetView::~SBindWidgetView()
{
	if (UWidgetBlueprint* Blueprint = UE::UMG::GetWidgetBlueprint(BlueprintEditor))
	{
		Blueprint->OnChanged().RemoveAll(this);
		Blueprint->OnCompiled().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}


void SBindWidgetView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRefreshRequested)
	{
		TSharedPtr<UE::UMG::SBindWidgetView> ListViewPin = ListView.Pin();
		TSharedPtr<FWidgetBlueprintEditor> BlueprintEditorPin = BlueprintEditor.Pin();
		if (ListViewPin && BlueprintEditorPin)
		{
			ListViewPin->SetSourceData(UE::UMG::BuildSourceDataList(BlueprintEditorPin));
		}
		bRefreshRequested = false;
	}
}


void SBindWidgetView::HandleObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	if (!bRefreshRequested)
	{
		for (const auto& Entry : ReplacementMap)
		{
			if (Entry.Key->IsA<UVisual>())
			{
				bRefreshRequested = true;
			}
		}
	}
}


void SBindWidgetView::HandleBlueprintChanged(UBlueprint* InBlueprint)
{
	bRefreshRequested = true;
}

#undef LOCTEXT_NAMESPACE
