// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FWidgetBlueprintEditor;
class FUICommandInfo;
class UBlueprint;
class UObject;

namespace UE
{
namespace UMG
{
	class SBindWidgetView;
}
}

class FBindWidgetCommands : public TCommands<FBindWidgetCommands>
{
public:
	FBindWidgetCommands() 
		: TCommands<FBindWidgetCommands>(TEXT("BindWidget"), NSLOCTEXT("Contexts", "Bind Widget", "Bind Widget"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> GotoNativeVarDefinition;
};

/**
 * 
 */
class SBindWidgetView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBindWidgetView){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FWidgetBlueprintEditor> InBlueprintEditor);
	virtual ~SBindWidgetView();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void HandleBlueprintChanged(UBlueprint* InBlueprint);
	void HandleObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

private:
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;
	TWeakPtr<UE::UMG::SBindWidgetView> ListView;
	bool bRefreshRequested;
};
