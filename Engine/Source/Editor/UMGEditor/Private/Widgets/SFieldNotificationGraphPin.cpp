// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFieldNotificationGraphPin.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SFieldNotificationPicker.h"

class SWidget;


#define LOCTEXT_NAMESPACE "SFieldNotificationGraphPin"

namespace UE::FieldNotification::Private
{
	UClass* GetPinClass(UEdGraphPin* Pin)
	{
		if (Pin == nullptr)
		{
			return nullptr;
		}
		check(Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object);

		UClass* PinClass = nullptr;
		if (UClass* DefaultClass = Cast<UClass>(Pin->DefaultObject))
		{
			PinClass = DefaultClass;
		}
		else
		{
			PinClass = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Pin->GetOwningNode())->GeneratedClass;
		}

		UClass* BaseClass = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get());
		if (!PinClass || !PinClass->IsChildOf(BaseClass))
		{
			PinClass = BaseClass;
		}

		if (Pin->LinkedTo.Num() > 0)
		{
			UClass* CommonInputClass = nullptr;
			for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				const FEdGraphPinType& LinkedPinType = LinkedPin->PinType;

				UClass* LinkClass = Cast<UClass>(LinkedPinType.PinSubCategoryObject.Get());
				if (LinkClass == nullptr && LinkedPinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self)
				{
					if (UK2Node* K2Node = Cast<UK2Node>(LinkedPin->GetOwningNode()))
					{
						LinkClass = K2Node->GetBlueprint()->GeneratedClass;
					}
				}

				if (LinkClass != nullptr)
				{
					if (CommonInputClass != nullptr)
					{
						while (!LinkClass->IsChildOf(CommonInputClass))
						{
							CommonInputClass = CommonInputClass->GetSuperClass();
						}
					}
					else
					{
						CommonInputClass = LinkClass;
					}
				}
			}

			PinClass = CommonInputClass;
		}
		return PinClass;
	}
} // namespace


namespace UE::FieldNotification
{

void SFieldNotificationGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	OnSetValue = InArgs._OnSetValue;
}


TSharedRef<SWidget> SFieldNotificationGraphPin::GetDefaultValueWidget()
{
	UEdGraphPin* SelfPin = GraphPinObj->GetOwningNode()->FindPin(UEdGraphSchema_K2::PSC_Self);
	if (UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(GraphPinObj->GetOwningNode()))
	{
		if (UFunction* Function = CallFunction->GetTargetFunction())
		{
			const FString& PinName = Function->GetMetaData("FieldNotifyInterfaceParam");
			if (PinName.Len() != 0)
			{
				SelfPin = GraphPinObj->GetOwningNode()->FindPin(*PinName);
			}
		}
	}

	return SNew(SFieldNotificationPicker)
		.Value(this, &SFieldNotificationGraphPin::GetValue)
		.OnValueChanged(this, &SFieldNotificationGraphPin::SetValue)
		.FromClass_Static(Private::GetPinClass, SelfPin)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}


FFieldNotificationId SFieldNotificationGraphPin::GetValue() const
{
	FFieldNotificationId Result;
	if (!GraphPinObj->GetDefaultAsString().IsEmpty())
	{
		FFieldNotificationId::StaticStruct()->ImportText(*GraphPinObj->GetDefaultAsString(), &Result, nullptr, EPropertyPortFlags::PPF_None, GLog, FFieldNotificationId::StaticStruct()->GetName());
	}
	return Result;
}


void SFieldNotificationGraphPin::SetValue(FFieldNotificationId NewValue)
{
	FString ValueString;
	FFieldNotificationId::StaticStruct()->ExportText(ValueString, &NewValue, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString);
}

} //namespace

#undef LOCTEXT_NAMESPACE
