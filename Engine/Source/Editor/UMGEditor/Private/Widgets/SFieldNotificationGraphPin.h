// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "FieldNotificationId.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
class UEdGraphPin;

namespace UE::FieldNotification
{

class SFieldNotificationGraphPin : public SGraphPin
{
public:
	DECLARE_DELEGATE_OneParam(FFieldNotificationGraphPinSetValue, FFieldNotificationId);

	SLATE_BEGIN_ARGS(SFieldNotificationGraphPin){}
		SLATE_EVENT(FFieldNotificationGraphPinSetValue, OnSetValue)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	FFieldNotificationId GetValue() const;
	void SetValue(FFieldNotificationId NewValue);

private:
	FFieldNotificationGraphPinSetValue OnSetValue;
};

} //namesapce