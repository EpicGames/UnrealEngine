// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFieldNotificationPicker.h"

#include "IFieldNotificationClassDescriptor.h"
#include "INotifyFieldValueChanged.h"
#include "Layout/Children.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ScriptInterface.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

namespace UE::FieldNotification
{

void SFieldNotificationPicker::Construct(const FArguments& InArgs)
{
	FromClassAttribute = InArgs._FromClass;
	ValueAttribute = InArgs._Value;
	OnValueChangedDelegate = InArgs._OnValueChanged;

	FieldNotificationIdsSource.Reset();
	ChildSlot
	[
		SAssignNew(PickerBox, SComboBox<TSharedPtr<FFieldNotificationId>>)
		.OptionsSource(&FieldNotificationIdsSource)
		.OnSelectionChanged(this, &SFieldNotificationPicker::HandleComboBoxChanged)
		.OnGenerateWidget(this, &SFieldNotificationPicker::HandleGenerateWidget)
		.OnComboBoxOpening(this, &SFieldNotificationPicker::HandleComboOpening)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SFieldNotificationPicker::HandleComboBoxValueAsText)
		]
	];
}


FFieldNotificationId SFieldNotificationPicker::GetCurrentValue() const
{
	return ValueAttribute.Get(FFieldNotificationId());
}


void SFieldNotificationPicker::HandleComboBoxChanged(TSharedPtr<FFieldNotificationId> InItem, ESelectInfo::Type InSeletionInfo)
{
	FFieldNotificationId NewItem = InItem ? *InItem.Get() : FFieldNotificationId();
	if (NewItem != GetCurrentValue())
	{
		if (!ValueAttribute.IsBound())
		{
			ValueAttribute.Set(NewItem);
		}
		OnValueChangedDelegate.ExecuteIfBound(NewItem);
	}
}


TSharedRef<SWidget> SFieldNotificationPicker::HandleGenerateWidget(TSharedPtr<FFieldNotificationId> InItem)
{
	return SNew(STextBlock)
		.Text(InItem ? FText::FromName(InItem->GetFieldName()) : FText::GetEmpty());
}


void SFieldNotificationPicker::HandleComboOpening()
{
	FieldNotificationIdsSource.Reset();
	if (UClass* Class = FromClassAttribute.Get(nullptr))
	{
		TScriptInterface<INotifyFieldValueChanged> ScriptObject = Class->GetDefaultObject();
		if (ScriptObject.GetInterface() && ScriptObject.GetObject())
		{
			SFieldNotificationPicker* Self = this;
			const UE::FieldNotification::IClassDescriptor& Descriptor = ScriptObject->GetFieldNotificationDescriptor();
			Descriptor.ForEachField(Class, [Self](const FFieldId& Id) ->bool
				{
					Self->FieldNotificationIdsSource.Add(MakeShared<FFieldNotificationId>(Id.GetName()));
					return true;
				});
		}

		if (FieldNotificationIdsSource.Num())
		{
			FieldNotificationIdsSource.Sort([](const TSharedPtr<FFieldNotificationId>& A, const TSharedPtr<FFieldNotificationId>& B)
				{
					return A->GetFieldName().LexicalLess(B->GetFieldName());
				});
			FFieldNotificationId CurrentValue = GetCurrentValue();
			const TSharedPtr<FFieldNotificationId>* FoundValue = FieldNotificationIdsSource.FindByPredicate([CurrentValue](const TSharedPtr<FFieldNotificationId>& Other)
				{
					return Other->GetFieldName() == CurrentValue.GetFieldName();
				});
			if (FoundValue)
			{
				PickerBox->SetSelectedItem(*FoundValue);
			}
		}
	}
}


FText SFieldNotificationPicker::HandleComboBoxValueAsText() const
{
	return FText::FromName(GetCurrentValue().GetFieldName());
}

} //namespace