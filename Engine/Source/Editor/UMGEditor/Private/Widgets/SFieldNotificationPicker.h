// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "FieldNotificationId.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;
class UClass;

namespace UE::FieldNotification
{

/**
 * A widget which allows the user to enter a FieldNotificationId or discover it from a drop menu.
 */
class SFieldNotificationPicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FFieldNotificationId);

	SLATE_BEGIN_ARGS(SFieldNotificationPicker)
	{}
		SLATE_ATTRIBUTE(FFieldNotificationId, Value)
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
		SLATE_ATTRIBUTE(UClass*, FromClass)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FFieldNotificationId GetCurrentValue() const;

private:
	void HandleComboBoxChanged(TSharedPtr<FFieldNotificationId> InItem, ESelectInfo::Type InSeletionInfo);
	TSharedRef<SWidget> HandleGenerateWidget(TSharedPtr<FFieldNotificationId> InItem);
	void HandleComboOpening();
	FText HandleComboBoxValueAsText() const;

private:
	TSharedPtr<SComboBox<TSharedPtr<FFieldNotificationId>>> PickerBox;
	TArray<TSharedPtr<FFieldNotificationId>> FieldNotificationIdsSource;
	FOnValueChanged OnValueChangedDelegate;
	TAttribute<FFieldNotificationId> ValueAttribute;
	TAttribute<UClass*> FromClassAttribute;
};

} // namespace