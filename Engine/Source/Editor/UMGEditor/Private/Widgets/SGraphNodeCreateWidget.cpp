// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SGraphNodeCreateWidget.h"

#include "Containers/Array.h"
#include "EdGraph/EdGraphPin.h"
#include "KismetPins/SGraphPinClass.h"
#include "Misc/AssertionMacros.h"
#include "Nodes/K2Node_CreateWidget.h"
#include "Templates/Casts.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;

TSharedPtr<SGraphPin> SGraphNodeCreateWidget::CreatePinWidget(UEdGraphPin* Pin) const
{
	UK2Node_CreateWidget* CreateWidgetNode = CastChecked<UK2Node_CreateWidget>(GraphNode);
	UEdGraphPin* ClassPin = CreateWidgetNode->GetClassPin();
	if ((ClassPin == Pin) && (!ClassPin->bHidden || (ClassPin->LinkedTo.Num() > 0)))
	{
		TSharedPtr<SGraphPinClass> NewPin = SNew(SGraphPinClass, ClassPin);
		check(NewPin.IsValid());
		NewPin->SetAllowAbstractClasses(false);
		return NewPin;
	}
	return SGraphNodeK2Default::CreatePinWidget(Pin);
}
