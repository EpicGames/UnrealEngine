// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Default.h"
#include "Templates/SharedPointer.h"

class SGraphPin;
class UEdGraphPin;

class SGraphNodeCreateWidget : public SGraphNodeK2Default
{
public:
	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	// End of SGraphNode interface
};
