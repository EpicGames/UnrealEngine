// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WidgetBlueprintEditor.h"
#include "BlueprintEditorModes.h"

#define UE_API UMGEDITOR_API

class UWidgetBlueprint;

/////////////////////////////////////////////////////
// FWidgetBlueprintApplicationMode

DECLARE_MULTICAST_DELEGATE_OneParam(FOnWidgetBlueprintModeTransition, class FWidgetBlueprintApplicationMode&);

class FWidgetBlueprintApplicationMode : public FBlueprintEditorApplicationMode
{
public:
	UE_API FWidgetBlueprintApplicationMode(TSharedPtr<class FWidgetBlueprintEditor> InWidgetEditor, FName InModeName);

	// FApplicationMode interface
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;
	// End of FApplicationMode interface

	/** Called at start of PostActivateMode */
	mutable FOnWidgetBlueprintModeTransition OnPostActivateMode;

	/** Called at end of PreDeactivateMode */
	mutable FOnWidgetBlueprintModeTransition OnPreDeactivateMode;

public:
	UE_API TSharedPtr<class FWidgetBlueprintEditor> GetBlueprintEditor() const;
	UE_API UWidgetBlueprint* GetBlueprint() const;

protected:
	TWeakPtr<class FWidgetBlueprintEditor> MyWidgetBlueprintEditor;

	// Set of spawnable tabs in the mode
	FWorkflowAllowedTabSet TabFactories;
};

#undef UE_API
