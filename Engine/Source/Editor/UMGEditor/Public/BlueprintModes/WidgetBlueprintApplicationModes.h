// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

#define UE_API UMGEDITOR_API

/////////////////////////////////////////////////////
// FWidgetBlueprintApplicationModes

// This is the list of IDs for widget blueprint editor modes
struct FWidgetBlueprintApplicationModes
{
	// Mode constants
	static UE_API const FName DesignerMode;
	static UE_API const FName GraphMode;
	UE_DEPRECATED(5.3, "DebugMode is deprecated. Use PreviewMode instead.")
	static UE_API const FName DebugMode;
	static UE_API const FName PreviewMode;

	static UE_API FText GetLocalizedMode(const FName InMode);

	UE_DEPRECATED(5.3, "IsDebugModeEnabled is deprecated. Use IsPreviewModeEnabled instead.")
	static UE_API bool IsDebugModeEnabled();
	static UE_API bool IsPreviewModeEnabled();	

private:
	FWidgetBlueprintApplicationModes() = delete;
};

#undef UE_API
