// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "AssetRegistry/AssetData.h"
#include "AssetThumbnailWidget.generated.h"

#define UE_API UMGEDITOR_API

class FAssetThumbnail;
class SBorder;
class SWidget;
struct FAssetThumbnailConfig;

UENUM(BlueprintType, meta = (DisplayName = "EThumbnailLabelType"))
enum class EThumbnailLabelType_BlueprintType : uint8
{
	ClassName,
	AssetName,
	NoLabel
};

UENUM(BlueprintType, meta = (DisplayName = "EThumbnailColorStripOrientation"))
enum class EThumbnailColorStripOrientation_BlueprintType : uint8
{
	/** Display the color strip as a horizontal line along the bottom edge */
	HorizontalBottomEdge,
	/** Display the color strip as a vertical line along the right edge */
	VerticalRightEdge,
};

DECLARE_DYNAMIC_DELEGATE_RetVal(FText, FGetHighlightTextDelegate);

/** Copied mostly from FAssetThumbnailConfig */
USTRUCT(BlueprintType)
struct FAssetThumbnailWidgetSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bForceGenericThumbnail = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bAllowHintText = true;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bAllowRealTimeOnHovered = true;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bAllowAssetSpecificThumbnailOverlay = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	EThumbnailLabelType_BlueprintType ThumbnailLabel = EThumbnailLabelType_BlueprintType::ClassName;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FGetHighlightTextDelegate HighlightedTextDelegate;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (EditCondition = "bAllowHintText", EditConditionHides))
	FLinearColor HintColorAndOpacity = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	/** Whether to override the asset type's colour */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bOverrideAssetTypeColor = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (EditCondition = "bOverrideAssetTypeColor", EditConditionHides))
	FLinearColor AssetTypeColorOverride = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	UE_DEPRECATED(5.6, "Border padding is handled by the thumbnail internally, this will no longer have any effect in future release.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (ToolTip = "Warning: Border padding is handled by the thumbnail internally, this will no longer have any effect in future release"))
	FMargin Padding;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	int32 GenericThumbnailSize = 64;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	EThumbnailColorStripOrientation_BlueprintType ColorStripOrientation = EThumbnailColorStripOrientation_BlueprintType::HorizontalBottomEdge;

	UE_API FAssetThumbnailConfig ToThumbnailConfig() const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAssetThumbnailWidgetSettings() = default;
	FAssetThumbnailWidgetSettings(const FAssetThumbnailWidgetSettings&) = default;
	FAssetThumbnailWidgetSettings& operator=(const FAssetThumbnailWidgetSettings&) = default;
	FAssetThumbnailWidgetSettings(FAssetThumbnailWidgetSettings&&) = default;
	FAssetThumbnailWidgetSettings& operator=(FAssetThumbnailWidgetSettings&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** This widget can be given an asset and it will render its thumbnail. Editor-only. */
UCLASS(MinimalAPI)
class UAssetThumbnailWidget : public UWidget
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UE_API void SetThumbnailSettings(const FAssetThumbnailWidgetSettings& InThumbnailSettings);
	FAssetThumbnailWidgetSettings GetThumbnailSettings() const { return ThumbnailSettings; }

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UE_API void SetAsset(const FAssetData& AssetData);
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UE_API void SetAssetByObject(UObject* Object);
	
	/** Sets the resolution for the rendered thumbnail. */
	UFUNCTION(Blueprintcallable, Category = "Appearance")
	UE_API void SetResolution(const FIntPoint& InResolution);

	/** Gets the resolution of the rendered thumbnail. */
	UFUNCTION(BlueprintPure, Category = "Appearance")
	FIntPoint GetResolution() const { return Resolution; }

	//~ Begin UWidget Interface
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget Interface
	
private:

	/** The asset of which to show the thumbnail. */
	UPROPERTY(EditAnywhere, Category = "Appearance")
	FAssetData AssetToShow;

	/** Desired size of the thumbnail */
	UPROPERTY(EditAnywhere, BlueprintGetter = "GetResolution", BlueprintSetter = "SetResolution", Category = "Appearance", meta = (ClampMin = "1", ClampMax = "1024"))
	FIntPoint Resolution = { 64, 64};
	
	/** Behaviour and style of the widget */
	UPROPERTY(EditAnywhere, Getter, Setter, BlueprintSetter = "SetThumbnailSettings", Category = "Appearance")
	FAssetThumbnailWidgetSettings ThumbnailSettings;

	/** Creates ThumbnailAssetWidget.  */
	TSharedPtr<FAssetThumbnail> ThumbnailRenderer;
	
	/** Wraps either an error widget or the contents of FAssetThumbnail::MakeThumbnailWidget. */
	TSharedPtr<SBorder> DisplayedWidget;

	/** Updates or creates ThumbnailRenderer and sets the contents of DisplayedWidget. */
	UE_API void UpdateNativeAssetThumbnailWidget();
};

#undef UE_API
