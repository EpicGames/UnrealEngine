// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDetailLayoutBuilder;
class UWidget;
class FWidgetBlueprintEditor;

/**
 * Details customization extender for UWidgets.
 */
class IBlueprintWidgetCustomizationExtender : public TSharedFromThis<IBlueprintWidgetCustomizationExtender>
{
public:
	virtual ~IBlueprintWidgetCustomizationExtender() = default;

	/**
	 * Customizes given widgets.
	 * @param InDetailLayout				detail layout builder for customization.
	 * @param InWidget						widgets to be customized.
	 * @param InWidgetBlueprintEditor		widget blueprint editor that is used for widgets being customized.
	 */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout, const TArrayView<UWidget*> InWidgets, const TSharedRef<FWidgetBlueprintEditor>& InWidgetBlueprintEditor) = 0;
};
