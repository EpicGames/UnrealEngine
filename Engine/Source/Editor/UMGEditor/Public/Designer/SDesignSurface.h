// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/SlateRect.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API UMGEDITOR_API

class FActiveTimerHandle;
class FPaintArgs;
class FSlateWindowElementList;
class FWidgetStyle;
class SWidget;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

class SDesignSurface : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDesignSurface )
		: _AllowContinousZoomInterpolation(false)
	{ }

		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ATTRIBUTE(bool, AllowContinousZoomInterpolation)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	// SWidget interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent) override;
	UE_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	// End of Swidget interface

	/** Gets the current zoom factor. */
	UE_API float GetZoomAmount() const;

	UE_API FVector2D GraphCoordToPanelCoord(const FVector2D& GraphSpaceCoordinate) const;
	UE_API FVector2D PanelCoordToGraphCoord(const FVector2D& PanelSpaceCoordinate) const;

protected:
	UE_API virtual void OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	UE_API void PaintBackgroundAsLines(const FSlateBrush* BackgroundImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;

	UE_API void ChangeZoomLevel(int32 ZoomLevelDelta, const FVector2D& WidgetSpaceZoomOrigin, bool bOverrideZoomLimiting);
	
	UE_API void PostChangedZoom();

	UE_API bool ScrollToLocation(const FGeometry& MyGeometry, FVector2D DesiredCenterPosition, const float InDeltaTime);

	UE_API bool ZoomToLocation(const FVector2D& CurrentSizeWithoutZoom, const FVector2D& DesiredSize, bool bDoneScrolling);

	UE_API void ZoomToFit(bool bInstantZoom);

	UE_API FText GetZoomText() const;
	UE_API FSlateColor GetZoomTextColorAndOpacity() const;

	UE_API FVector2D GetViewOffset() const;

	UE_API FSlateRect ComputeSensibleBounds() const;

protected:
	UE_API virtual FSlateRect ComputeAreaBounds() const;
	UE_API virtual float GetGridScaleAmount() const;
	UE_API virtual int32 GetGraphRulePeriod() const;
	virtual int32 GetSnapGridSize() const = 0;

protected:
	/** The position within the graph at which the user is looking */
	FVector2D ViewOffset;

	/** The position in the grid to begin drawing at. */
	FVector2D GridOrigin;

	/** Should we render the grid lines? */
	bool bDrawGridLines;

	/** Previous Zoom Level */
	int32 PreviousZoomLevel;

	/** How zoomed in/out we are. e.g. 0.25f results in quarter-sized nodes. */
	int32 ZoomLevel;

	/** Are we panning the view at the moment? */
	bool bIsPanning;

	/** Are we zooming the view with trackpad at the moment? */
	bool bIsZooming;

	/** Allow continuous zoom interpolation? */
	TAttribute<bool> AllowContinousZoomInterpolation;

	/** Fade on zoom for graph */
	FCurveSequence ZoomLevelGraphFade;

	/** Curve that handles fading the 'Zoom +X' text */
	FCurveSequence ZoomLevelFade;

	// The interface for mapping ZoomLevel values to actual node scaling values
	TUniquePtr<FZoomLevelsContainer> ZoomLevels;

	bool bAllowContinousZoomInterpolation;

	bool bTeleportInsteadOfScrollingWhenZoomingToFit;

	FVector2D ZoomTargetTopLeft;
	FVector2D ZoomTargetBottomRight;
	FVector2D ZoomToFitPadding;

	/** The Y component of mouse drag (used when zooming) */
	double TotalMouseDelta;

	/** Offset in the panel the user started the LMB+RMB zoom from */
	FVector2D ZoomStartOffset;

	/**  */
	FVector2D ViewOffsetStart;

	/**  */
	FVector2D MouseDownPositionAbsolute;

	/** Cumulative magnify delta from trackpad gesture */
	float TotalGestureMagnify;

	/** Does the user need to press Control in order to over-zoom. */
	bool bRequireControlToOverZoom;

private:
	/** Active timer that handles deferred zooming until the target zoom is reached */
	UE_API EActiveTimerReturnType HandleZoomToFit(double InCurrentTime, float InDeltaTime);

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	// A flag noting if we have a pending zoom to extents operation to perform next tick.
	bool bDeferredZoomToExtents;
};

#undef UE_API
