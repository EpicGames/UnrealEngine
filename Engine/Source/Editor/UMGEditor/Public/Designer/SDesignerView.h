// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Styling/SlateColor.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Layout/ArrangedWidget.h"
#include "Widgets/SWidget.h"
#include "UObject/GCObject.h"
#include "Types/SlateStructs.h"
#include "Animation/CurveSequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateBrush.h"
#include "Components/Widget.h"
#include "WidgetReference.h"
#include "WidgetBlueprintEditor.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Layout/WidgetPath.h"
#include "IUMGDesigner.h"
#include "DesignerExtension.h"
#include "Designer/SDesignSurface.h"
#include "UMGEditorProjectSettings.h"

#define UE_API UMGEDITOR_API

class FMenuBuilder;
class FScopedTransaction;
class SBox;
class SCanvas;
class SPaintSurface;
class SRuler;
class SZoomPan;
class UPanelWidget;
class UWidgetBlueprint;
class FHittestGrid;
struct FOnPaintHandlerParams;
struct FWidgetHitResult;
class UWidgetEditingProjectSettings;

/**
 * The designer for widgets.  Allows for laying out widgets in a drag and drop environment.
 */
class SDesignerView : public SDesignSurface, public FGCObject, public IUMGDesigner
{
public:

	SLATE_BEGIN_ARGS( SDesignerView ) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<class FWidgetBlueprintEditor> InBlueprintEditor);
	UE_API virtual ~SDesignerView();

	UE_API EActiveTimerReturnType EnsureTick(double InCurrentTime, float InDeltaTime);

	UE_API TSharedRef<SWidget> CreateOverlayUI();

	// SWidget interface
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	UE_API void Register(TSharedRef<FDesignerExtension> Extension);
	UE_API void Unregister(TSharedRef<FDesignerExtension> Extension);

	// IUMGDesigner interface
	UE_API virtual float GetPreviewScale() const override;
	UE_API virtual const TSet<FWidgetReference>& GetSelectedWidgets() const override;
	UE_API virtual FWidgetReference GetSelectedWidget() const override;
	UE_API virtual ETransformMode::Type GetTransformMode() const override;
	UE_API virtual FGeometry GetDesignerGeometry() const override;
	UE_API virtual FVector2D GetWidgetOriginAbsolute() const;
	UE_API virtual bool GetWidgetGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const override;
	UE_API virtual bool GetWidgetGeometry(const UWidget* PreviewWidget, FGeometry& Geometry) const override;
	UE_API virtual bool GetWidgetParentGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const override;
	UE_API virtual FGeometry MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const override;
	UE_API virtual void MarkDesignModifed(bool bRequiresRecompile) override;
	UE_API virtual void PushDesignerMessage(const FText& Message) override;
	UE_API virtual void PopDesignerMessage() override;
	// End of IUMGDesigner interface

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDesignerView");
	}
	// End of FGCObject interface

public:

	/** The width of the preview screen for the UI */
	UE_API FOptionalSize GetPreviewAreaWidth() const;

	/** The height of the preview screen for the UI */
	UE_API FOptionalSize GetPreviewAreaHeight() const;

	/** The width of the preview widget for the UI */
	UE_API FOptionalSize GetPreviewSizeWidth() const;

	/** The height of the preview widget for the UI */
	UE_API FOptionalSize GetPreviewSizeHeight() const;

	/** Gets the DPI scale that would be applied given the current preview width and height */
	UE_API float GetPreviewDPIScale() const;

	/** Set the size of the preview screen for the UI */
	UE_API void SetPreviewAreaSize(int32 Width, int32 Height);

	UE_API void BeginResizingArea();
	UE_API void EndResizingArea();
	
	UE_API const UWidgetEditingProjectSettings* GetRelevantSettings() const;

protected:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual void OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;
	UE_API void DrawResolution(const FDebugResolution& Resolution, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	UE_API virtual FSlateRect ComputeAreaBounds() const override;
	UE_API virtual int32 GetGraphRulePeriod() const override;
	UE_API virtual float GetGridScaleAmount() const override;
	UE_API virtual int32 GetSnapGridSize() const override;

private:
	UE_API void RegisterExtensions();

	/** Establishes the resolution and aspect ratio to use on construction from config settings */
	UE_API void SetStartupResolution();

	UE_API FVector2D GetAreaResizeHandlePosition() const;
	UE_API EVisibility GetAreaResizeHandleVisibility() const;

	UE_API const FSlateBrush* GetPreviewBackground() const;

	/** Adds any pending selected widgets to the selection set */
	UE_API void ResolvePendingSelectedWidgets();

	/** Updates the designer to display the latest preview widget */
	UE_API void UpdatePreviewWidget(bool bForceUpdate);

	UE_API void BroadcastDesignerChanged();

	UE_API void ClearExtensionWidgets();
	UE_API void CreateExtensionWidgetsForSelection();

	UE_API EVisibility GetInfoBarVisibility() const;
	UE_API FText GetInfoBarText() const;

	/** Displays the context menu when you right click */
	UE_API void ShowContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	UE_API void OnEditorSelectionChanged();

	/** Called when a new widget is being hovered */
	UE_API void OnHoveredWidgetSet(const FWidgetReference& InHoveredWidget);

	/** Called when a widget is no longer being hovered */
	UE_API void OnHoveredWidgetCleared();

	/** Gets the blueprint being edited by the designer */
	UE_API UWidgetBlueprint* GetBlueprint() const;

	/** Called whenever the blueprint is recompiled or the preview is updated */
	UE_API void OnPreviewNeedsRecreation();

	UE_API void PopulateWidgetGeometryCache(FArrangedWidget& Root);
	UE_API void PopulateWidgetGeometryCache_Loop(FArrangedWidget& Parent);

	/** @return Formatted text for the given resolution params */
	UE_API FText GetResolutionText(int32 Width, int32 Height, const FString& AspectRatio) const;

	/** Move the selected widget by the nudge amount. */
	UE_API FReply NudgeSelectedWidget(FVector2D Nudge);

	UE_API FText GetCurrentResolutionText() const;
	UE_API FText GetCurrentDPIScaleText() const;
	UE_API FSlateColor GetCurrentDPIScaleColor() const;
	UE_API FText GetCurrentScaleFactorText() const;
	UE_API FText GetCurrentSafeZoneText() const;
	UE_API FSlateColor GetResolutionTextColorAndOpacity() const;
	UE_API EVisibility GetResolutionTextVisibility() const;

	UE_API TOptional<int32> GetCustomResolutionWidth() const;
	UE_API TOptional<int32> GetCustomResolutionHeight() const;
	UE_API void OnCustomResolutionWidthChanged(int32 InValue);
	UE_API void OnCustomResolutionHeightChanged(int32 InValue);
	UE_API EVisibility GetCustomResolutionEntryVisibility() const;
	
	// Handles selecting a common screen resolution.
	UE_API void HandleOnCommonResolutionSelected(const FPlayScreenResolution InResolution);
	UE_API bool HandleIsCommonResolutionSelected(const FPlayScreenResolution InResolution) const;
	UE_API FUIAction GetResolutionMenuAction( const FPlayScreenResolution& ScreenResolution );
	UE_API TSharedRef<SWidget> GetResolutionsMenu();

	UE_API TSharedRef<SWidget> GetScreenSizingFillMenu();
	UE_API void CreateScreenFillEntry(FMenuBuilder& MenuBuilder, EDesignPreviewSizeMode SizeMode);
	UE_API FText GetScreenSizingFillText() const;
	UE_API bool GetIsScreenFillRuleSelected(EDesignPreviewSizeMode SizeMode) const;
	UE_API void OnScreenFillRuleSelected(EDesignPreviewSizeMode SizeMode);

	UE_API const FSlateBrush* GetAspectRatioSwitchImage() const;
	UE_API bool GetAspectRatioSwitchEnabled() const;
	UE_API bool GetFlipDeviceEnabled() const;
	UE_API EVisibility GetDesignerOutlineVisibility() const;
	UE_API FSlateColor GetDesignerOutlineColor() const;
	UE_API FText GetDesignerOutlineText() const;

	UE_API FText GetCursorPositionText() const;
	UE_API EVisibility GetCursorPositionTextVisibility() const;
	
	UE_API FText GetSelectedWidgetDimensionsText() const;
	UE_API EVisibility GetSelectedWidgetDimensionsVisibility() const;

	// Handles drawing selection and other effects a SPaintSurface widget injected into the hierarchy.
	UE_API int32 HandleEffectsPainting(const FOnPaintHandlerParams& PaintArgs);
	UE_API void DrawSelectionAndHoverOutline(const FOnPaintHandlerParams& PaintArgs);
	UE_API void DrawSafeZone(const FOnPaintHandlerParams& PaintArgs);

	UE_API FReply HandleDPISettingsClicked();

	UE_API UUserWidget* GetDefaultWidget() const;

	UE_API void BeginTransaction(const FText& SessionName);
	UE_API bool InTransaction() const;
	UE_API void EndTransaction(bool bCancel);

	UE_API UWidget* GetWidgetInDesignScopeFromSlateWidget(TSharedRef<SWidget>& InWidget);

	UE_API EVisibility GetExtensionCanvasVisibility() const;

private:
	struct FWidgetHitResult
	{
	public:
		FWidgetReference Widget;
		FArrangedWidget WidgetArranged;

		FName NamedSlot;

	public:
		FWidgetHitResult();
	};

	/** @returns Gets the widget under the cursor based on a mouse pointer event. */
	UE_API bool FindWidgetUnderCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSubclassOf<UWidget> FindType, FWidgetHitResult& HitResult);

private:
	UE_API FReply HandleZoomToFitClicked();
	UE_API FReply HandleSwapAspectRatioClicked();
	UE_API FReply HandleFlipSafeZonesClicked();
	UE_API EVisibility GetRulerVisibility() const;

private:
	static UE_API const FString ConfigSectionName;
	static UE_API const FString DefaultPreviewOverrideName;

	/** Extensions for the designer to allow for custom widgets to be inserted onto the design surface as selection changes. */
	TArray< TSharedRef<FDesignerExtension> > DesignerExtensions;

private:
	UE_API void BindCommands();

	UE_API void SetTransformMode(ETransformMode::Type InTransformMode);
	UE_API bool CanSetTransformMode(ETransformMode::Type InTransformMode) const;
	UE_API bool IsTransformModeActive(ETransformMode::Type InTransformMode) const;

	UE_API void ToggleShowingOutlines();
	UE_API bool IsShowingOutlines() const;

	UE_API void ToggleRespectingLocks();
	UE_API bool IsRespectingLocks() const;

	UE_API void ProcessDropAndAddWidget(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const bool bIsPreview);
	UE_API void MoveWidgets(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const bool bIsPreview, UWidget* Target, const bool bAnyWidgetChangingParent);

	UE_API FVector2D GetExtensionPosition(TSharedRef<FDesignerSurfaceElement> ExtensionElement) const;

	UE_API FVector2D GetExtensionSize(TSharedRef<FDesignerSurfaceElement> ExtensionElement) const;
	
	UE_API void ClearDropPreviews();

	UE_API void DetermineDragDropPreviewWidgets(TArray<UWidget*>& OutWidgets, const FDragDropEvent& DragDropEvent, UWidgetTree* RootWidgetTree);

	UE_API void SwapSafeZoneTypes();

	UE_API bool IsSelectableInSequencer(UWidget* const InWidget) const;

	UE_API void OnSelectedAnimationChanged();

	UE_API void OnSelectionLimitedChanged(const bool bInEnabled);

	UE_API void DeselectNonSequencerWidgets();

	UE_API void ToggleViewportSelectionLimited();
	
	UE_API bool CanToggleViewportSelectionLimited() const;

	UE_API bool IsViewportSelectionLimited() const;

private:
	/** A reference to the BP Editor that owns this designer */
	TWeakPtr<FWidgetBlueprintEditor> BlueprintEditor;

	/** The designer command list */
	TSharedPtr<FUICommandList> CommandList;

	/** The transaction used to commit undoable actions from resize, move...etc */
	FScopedTransaction* ScopedTransaction;

	/** The current preview widget */
	TObjectPtr<UUserWidget> PreviewWidget;

	/** The current preview widget's slate widget */
	TWeakPtr<SWidget> PreviewSlateWidget;

	/** Cache last mouse position to be used as a paste drop location */
	FVector2D CachedMousePosition;
	
	struct FDropPreview
	{
		TObjectPtr<UWidget> Widget;
		TObjectPtr<UPanelWidget> Parent;
		TWeakPtr<FDragDropOperation> DragOperation;
	};

	TArray<FDropPreview> DropPreviews;

	TSharedPtr<SWidget> PreviewHitTestRoot;
	TSharedPtr<SBox> PreviewAreaConstraint;
	TSharedPtr<SDPIScaler> PreviewSurface;
	TSharedPtr<SBox> PreviewSizeConstraint;

	TSharedPtr<SCanvas> DesignerControls;
	TSharedPtr<SCanvas> DesignerWidgetCanvas;
	TSharedPtr<SCanvas> ExtensionWidgetCanvas;
	TSharedPtr<SPaintSurface> EffectsLayer;

	/** The currently selected preview widgets in the preview GUI, just a cache used to determine changes between selection changes. */
	TSet< FWidgetReference > SelectedWidgetsCache;

	/** The location in selected widget local space where the context menu was summoned. */
	FVector2D SelectedWidgetContextMenuLocation;

	/**
	 * Holds onto a temporary widget that the user may be getting ready to select, or may just 
	 * be the widget that got hit on the initial mouse down before moving the parent.
	 */
	FWidgetReference PendingSelectedWidget;

	/** The position in screen space where the user began dragging a widget */
	FVector2D DraggingStartPositionScreenSpace;

	/** An existing widget is being moved in its current container, or in to a new container. */
	bool bMovingExistingWidget;

	/** The configured Width of the preview area, simulates screen size. */
	int32 PreviewWidth;

	/** The configured Height of the preview area, simulates screen size. */
	int32 PreviewHeight;

	/** The original Width of the preview area, read from the settings file. */
	int32 WidthReadFromSettings;

	/** The original Height of the preview area, read from the settings file. */
	int32 HeightReadFromSettings;

	// Whether we have selected one of the common resolutions for the preview.
	bool bCommonResolutionSelected;

	/***/
	bool bShowResolutionOutlines;

	/** The slate brush we use to hold the background image shown in the designer. */
	mutable FSlateBrush BackgroundImage;

	/** We cache the desired preview desired size to maintain the same size between compiles when it lags a frame behind and no widget is available. */
	FVector2D CachedPreviewDesiredSize;

	// Resolution Info
	FString PreviewAspectRatio;

	// Whether or not the resolution can flip between portrait and landscape
	bool bCanPreviewSwapAspectRatio;

	FString PreviewOverrideName;

	float ScaleFactor;

	bool bSafeZoneFlipped;

	// Whether the preview is currently in portrait mode
	bool bPreviewIsPortrait;

	/** Curve to handle fading of the resolution */
	FCurveSequence ResolutionTextFade;

	/** Curve to handle the fade-in of the border around the hovered widget */
	FCurveSequence HoveredWidgetOutlineFade;

	/**  */
	FWeakWidgetPath SelectedWidgetPath;

	/** The ruler bar at the top of the designer. */
	TSharedPtr<SRuler> TopRuler;

	/** The ruler bar on the left side of the designer. */
	TSharedPtr<SRuler> SideRuler;

	/** */
	ETransformMode::Type TransformMode;

	/**  */
	TMap<TSharedRef<SWidget>, FArrangedWidget> CachedWidgetGeometry;

	/** */
	TSharedPtr<FHittestGrid> DesignerHittestGrid;

	/** The message stack to display the last item to the user in a non-modal fashion. */
	TArray<FText> DesignerMessageStack;

	TArray<FVector2D> CustomSafeZoneStarts;

	TArray<FVector2D> CustomSafeZoneDimensions;

	FMargin DesignerSafeZoneOverride;
};

#undef UE_API
