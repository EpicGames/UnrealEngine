// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FArrangedChildren;
class SWidget;
struct FGeometry;

class SZoomPan : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SZoomPan) {}
		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)
		
		SLATE_ATTRIBUTE(FVector2D, ViewOffset)
		
		SLATE_ATTRIBUTE(float, ZoomAmount)
	SLATE_END_ARGS()

	UMGEDITOR_API void Construct(const FArguments& InArgs);
	
	/**
	 * Sets the content for this border
	 *
	 * @param	InContent	The widget to use as content for the border
	 */
	void SetContent( const TSharedRef< SWidget >& InContent );

protected:
	UMGEDITOR_API void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	UMGEDITOR_API virtual float GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const override;

	/** The position within the panel at which the user is looking */
	TAttribute<FVector2D> ViewOffset;

	/** How zoomed in/out we are. e.g. 0.25f results in quarter-sized widgets. */
	TAttribute<float> ZoomAmount;
};
