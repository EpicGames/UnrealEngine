// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WidgetReference.h"
#include "Widgets/SWidget.h"

#define UE_API UMGEDITOR_API

class FScopedTransaction;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class IUMGDesigner;
class SWidget;
class UWidgetBlueprint;
struct FGeometry;
struct FWidgetReference;

/**
 * The location where the widget should appear
 */
enum class EExtensionLayoutLocation : uint8
{
	/** Slate unit position relative from the parent. */
	RelativeFromParent,

	TopLeft,
	TopCenter,
	TopRight,

	CenterLeft,
	CenterCenter,
	CenterRight,

	BottomLeft,
	BottomCenter,
	BottomRight,
};



/**
 * The basic element returned for extending the design surface.
 */
class FDesignerSurfaceElement : public TSharedFromThis < FDesignerSurfaceElement >
{
public:
	FDesignerSurfaceElement(TSharedRef<SWidget> InWidget, EExtensionLayoutLocation InLocation, TAttribute<FVector2D> InOffset = FVector2D(0, 0), TAttribute<FVector2D> InAlignment = FVector2D(0, 0))
		: Widget(InWidget)
		, Location(InLocation)
		, Offset(InOffset)
		, Alignment(InAlignment)
	{
	}

	/** Gets the widget that will be laid out in the design surface for extending the capability of the selected widget */
	TSharedRef<SWidget> GetWidget() const
	{
		return Widget;
	}

	/** Gets the location where the widget will appear */
	EExtensionLayoutLocation GetLocation() const
	{
		return Location;
	}

	/** Sets the offset after laid out in that location */
	void SetOffset(TAttribute<FVector2D> InOffset)
	{
		Offset = InOffset;
	}

	/** Gets the offset after being laid out. */
	FVector2D GetOffset() const
	{
		return Offset.Get();
	}

	/** Sets the alignment to use, a normalized value representing position inside the parent. */
	void SetAlignment(TAttribute<FVector2D> InAlignment)
	{
		Alignment = InAlignment;
	}

	/** Gets the alignment to use, a normalized value representing position inside the parent. */
	FVector2D GetAlignment() const
	{
		return Alignment.Get();
	}

protected:
	TSharedRef<SWidget> Widget;

	EExtensionLayoutLocation Location;

	TAttribute<FVector2D> Offset;

	TAttribute<FVector2D> Alignment;
};

/**
 * The Designer extension allows developers to provide additional widgets and custom painting to the designer surface for
 * specific widgets.  Allowing for a more customized and specific editors for the different widgets.
 */
class FDesignerExtension : public TSharedFromThis<FDesignerExtension>
{
public:
	/** Constructor */
	UE_API FDesignerExtension();
	UE_API virtual ~FDesignerExtension();

	/** Initializes the designer extension, this is called the first time a designer extension is registered */
	UE_API virtual void Initialize(IUMGDesigner* InDesigner, UWidgetBlueprint* InBlueprint);

	/** Uninitializes the designer extension, this is called when the designer is removed. */
	UE_API virtual void Uninitialize();

	/** Returns true if the designer extension can extend the current selection. */
	virtual bool CanExtendSelection(const TArray< FWidgetReference >& Selection) const
	{
		return false;
	}

	/** Called every time a element the designer can extend is selected. */
	virtual void ExtendSelection(const TArray< FWidgetReference >& Selection, TArray< TSharedRef<FDesignerSurfaceElement> >& SurfaceElements)
	{
	}

	/** Called every time the designer creates new preview. */
	virtual void PreviewContentCreated(UUserWidget* PreviewWidget)
	{
	}

	/** Called every time the content of the designer changed. */
	virtual void PreviewContentChanged(TSharedRef<SWidget> NewContent)
	{
	}

	/** Called each frames to tick the extension. */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
	}

	/** Called to paint the extension. */
	virtual void Paint(const TSet< FWidgetReference >& Selection, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
	}

	/** Gets the ID identifying this extension. */
	UE_API FName GetExtensionId() const;

	bool operator==(const FDesignerExtension& Other) const
	{
		return ExtensionId.IsEqual(Other.GetExtensionId());
	}

protected:
	UE_API void BeginTransaction(const FText& SessionName);

	UE_API void EndTransaction();

protected:
	FName ExtensionId;
	TWeakObjectPtr<UWidgetBlueprint> Blueprint;
	IUMGDesigner* Designer;

	TArray< FWidgetReference > SelectionCache;

private:
	FScopedTransaction* ScopedTransaction;
};

#undef UE_API
