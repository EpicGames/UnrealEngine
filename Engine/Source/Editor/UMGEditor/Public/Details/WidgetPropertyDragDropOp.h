// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "UObject/Field.h"

class FScopedTransaction;
class UWidget;
class UWidgetBlueprint;

#define LOCTEXT_NAMESPACE "WidgetPropertyDragDropOp"
/** Drag-and-drop operation that stores data about the a widget property being dragged on. */
class FWidgetPropertyDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWidgetPropertyDragDropOp, FDecoratedDragDropOp);

	FWidgetPropertyDragDropOp();
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	static TSharedRef<FWidgetPropertyDragDropOp> New(UWidget* InOwnerWidget, FName InPropertyName, TArray<FFieldVariant> InPropertyPath, UWidgetBlueprint* InWidgetBP);

	TWeakObjectPtr<UWidget> OwnerWidget;
	TArray<FFieldVariant> DraggedPropertyPath;
	TWeakObjectPtr<UWidgetBlueprint> WidgetBP;
	TUniquePtr<FScopedTransaction> Transaction;
};
#undef LOCTEXT_NAMESPACE

