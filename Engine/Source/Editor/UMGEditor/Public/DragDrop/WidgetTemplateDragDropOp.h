// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "Templates/SharedPointer.h"

#define UE_API UMGEDITOR_API

class FWidgetTemplate;

/**
 * This drag drop operation allows widget templates from the palate to be dragged and dropped into the designer
 * or the widget hierarchy in order to spawn new widgets.
 */
class FWidgetTemplateDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWidgetTemplateDragDropOp, FDecoratedDragDropOp)

	/** The template to create an instance */
	TSharedPtr<FWidgetTemplate> Template;

	/** Constructs the drag drop operation */
	static UE_API TSharedRef<FWidgetTemplateDragDropOp> New(const TSharedPtr<FWidgetTemplate>& InTemplate);
};

#undef UE_API
