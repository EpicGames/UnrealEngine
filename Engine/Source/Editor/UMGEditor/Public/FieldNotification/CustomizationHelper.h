// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Styling/SlateTypes.h"
#include "UObject/WeakFieldPtr.h"
#include "WidgetBlueprint.h"

#define UE_API UMGEDITOR_API

class IDetailLayoutBuilder;
class IBlueprintEditor;
class UWidgetBlueprint;
class UK2Node_FunctionEntry;

namespace UE::FieldNotification
{

class FCustomizationHelper
{
public:
	static UE_API const FName MetaData_FieldNotify;

	FCustomizationHelper(UWidgetBlueprint* InBlueprint)
		: Blueprint(InBlueprint)
	{}

	UE_API void CustomizeVariableDetails(IDetailLayoutBuilder& DetailLayout);
	UE_API void CustomizeFunctionDetails(IDetailLayoutBuilder& DetailLayout);

private:
	UE_API ECheckBoxState IsPropertyFieldNotifyChecked() const;
	UE_API void HandlePropertyFieldNotifyCheckStateChanged(ECheckBoxState CheckBoxState);
	UE_API ECheckBoxState IsFunctionFieldNotifyChecked() const;
	UE_API void HandleFunctionFieldNotifyCheckStateChanged(ECheckBoxState CheckBoxState);

	UE_API UK2Node_FunctionEntry* FindFunctionEntry(UObject* Obj) const;

private:
	/** The blueprint we are editing */
	TWeakObjectPtr<UWidgetBlueprint> Blueprint;

	/** The property we are editing */
	TArray<TWeakFieldPtr<FProperty>> PropertiesBeingCustomized;

	/** The function we are editing */
	TArray<TWeakObjectPtr<UK2Node_FunctionEntry>> FunctionsBeingCustomized;
};

} //namespace

#undef UE_API
