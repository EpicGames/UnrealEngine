// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "WidgetReference.h"

class FWidgetBlueprintEditor;
class UWidgetBlueprint;

/**
* This drag drop operation allows widgets from the hierarchy rows to be dragged and dropped around the editor.
*/
class FHierarchyWidgetDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FHierarchyWidgetDragDropOp, FDecoratedDragDropOp)

	bool UMGEDITOR_API HasOriginatedFrom(const TSharedPtr<FWidgetBlueprintEditor>& BlueprintEditor) const;
	
	const TArrayView<const FWidgetReference> UMGEDITOR_API GetWidgetReferences() const;

	/** Constructs a new drag/drop operation */
	static TSharedRef<FHierarchyWidgetDragDropOp> New(UWidgetBlueprint* Blueprint, const TArray<FWidgetReference>& InWidgets);

private:
	TArray<FWidgetReference> WidgetReferences;
};