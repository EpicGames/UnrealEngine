// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"

class IHierarchyModel
{
public:
	virtual ~IHierarchyModel() = default;
	virtual bool IsHovered() const = 0;
	virtual bool IsVisible() const = 0;
	virtual bool CanControlVisibility() const = 0;
	virtual bool CanControlLockedInDesigner() const = 0;
	virtual bool IsLockedInDesigner() const = 0;
};

template <typename HierarchyModel>
class IHierarchyViewItem : public STableRow< TSharedPtr<HierarchyModel> >
{
public:
	virtual ~IHierarchyViewItem() = default;
	virtual FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual void ToggleLockedInDesigner(const bool bRecursive) = 0;
	virtual bool ShouldAppearHovered() const = 0;
};
