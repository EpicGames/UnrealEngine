// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "Framework/Views/TreeFilterHandler.h"
#include "Misc/TextFilter.h"
#include "Framework/SlateDelegates.h"
#include "UObject/ObjectPtr.h"
#include "WidgetBlueprint.h"
#include "Widgets/SCompoundWidget.h"

#include "SReadOnlyHierarchyView.generated.h"

#define UE_API UMGEDITOR_API

class ITableRow;
class SSearchBox;
class STableViewBase;

template <typename ItemType> class STreeView;
template <typename ItemType> class TreeFilterHandler;
template <typename ItemType> class TTextFilter;

UENUM()
enum class ERootSelectionMode : uint8
{
	/** The Root Widget is not selectable */
	Disabled,

	/** The Root Widget is selectable and will show it's name as the display text */
	Enabled,

	/** The Root Widget is selectable and it will show Self as the display text */
	Self
};

UENUM()
enum class EInheritedNamedSlotSelectionMode : uint8
{
	/** Inherited Named slots are not selectable */
	Disabled,

	/** Inherited Named slots are selectable */
	Enabled,
};

class SReadOnlyHierarchyView : public SCompoundWidget
{
private:

	struct FItem
	{
		FItem(const UWidget* InWidget) : Widget(InWidget) {}
		FItem(const UWidgetBlueprint* InWidgetBlueprint) : WidgetBlueprint(InWidgetBlueprint) {}
		FItem(const FName InInheritedNamedSlotName) : InheritedNamedSlotName(InInheritedNamedSlotName) {}

		TWeakObjectPtr<const UWidget> Widget;
		TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
		const FName InheritedNamedSlotName;

		TArray<TSharedPtr<FItem>> Children;
	};

public:

	using FOnSelectionChanged = typename TSlateDelegates<FName>::FOnSelectionChanged;

	SLATE_BEGIN_ARGS(SReadOnlyHierarchyView) {}
		SLATE_ARGUMENT_DEFAULT(bool, ShowSearch) = true;
		SLATE_ARGUMENT_DEFAULT(ERootSelectionMode, RootSelectionMode) = ERootSelectionMode::Enabled;
		SLATE_ARGUMENT_DEFAULT(EInheritedNamedSlotSelectionMode, InheritedNamedSlotSelectionMode) = EInheritedNamedSlotSelectionMode::Enabled;		
		SLATE_ARGUMENT_DEFAULT(ESelectionMode::Type, SelectionMode) = ESelectionMode::Single;
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_ARGUMENT(TArray<FName>, ShowOnly)
		SLATE_ARGUMENT_DEFAULT(bool, ExpandAll) = true;
	SLATE_END_ARGS()

	UE_API virtual ~SReadOnlyHierarchyView();

	UE_API void Construct(const FArguments& InArgs, const UWidgetBlueprint* WidgetBlueprint);
	UE_API void Refresh();

	UE_API void SetSelectedWidget(FName WidgetName);
	UE_API void SetRawFilterText(const FText& Text);

	UE_API TArray<FName> GetSelectedWidgets() const;
	UE_API void ClearSelection();

private:

	UE_API TSharedRef<ITableRow> GenerateRow(TSharedPtr<FItem> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	UE_API void GetItemChildren(TSharedPtr<FItem> Item, TArray<TSharedPtr<FItem>>& OutChildren) const;
	UE_API FText GetItemText(TSharedPtr<FItem> Item) const;
	UE_API const FSlateBrush* GetIconBrush(TSharedPtr<FItem> Item) const;
	UE_API FText GetIconToolTipText(TSharedPtr<FItem> Item) const;
	UE_API FText GetWidgetToolTipText(TSharedPtr<FItem> Item) const;
	UE_API void GetFilterStringsForItem(TSharedPtr<FItem> Item, TArray<FString>& OutStrings) const;
	UE_API TSharedPtr<FItem> FindItem(const TArray<TSharedPtr<FItem>>& RootItems, FName Name) const;
	UE_API void OnSelectionChanged(TSharedPtr<FItem> Selected, ESelectInfo::Type SelectionType);

	UE_API void RebuildTree();
	UE_API void BuildWidgetChildren(const UWidget* Widget, TSharedPtr<FItem> Parent);

	UE_API void SetItemExpansionRecursive(TSharedPtr<FItem> Item, bool bShouldBeExpanded);
	UE_API void ExpandAll();

	UE_API FName GetItemName(const TSharedPtr<FItem>& Item) const;

private:

	TWeakObjectPtr<const UWidgetBlueprint> WidgetBlueprint;
	TArray<FName> ShowOnly;

	TArray<TSharedPtr<FItem>> RootWidgets;
	TArray<TSharedPtr<FItem>> FilteredRootWidgets;

	ERootSelectionMode RootSelectionMode;
	EInheritedNamedSlotSelectionMode InheritedNamedSlotSelectionMode;

	FOnSelectionChanged OnSelectionChangedDelegate;

	TSharedPtr<SSearchBox> SearchBox;

	using FTextFilter = TTextFilter<TSharedPtr<FItem>>;
	TSharedPtr<FTextFilter> SearchFilter;
	using FTreeFilterHandler = TreeFilterHandler<TSharedPtr<FItem>>;
	TSharedPtr<FTreeFilterHandler> FilterHandler;

	TSharedPtr<STreeView<TSharedPtr<FItem>>> TreeView;

	bool bExpandAll;
};

#undef UE_API
