// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/OutputDevice.h"
#include "Templates/SharedPointer.h"
#include "UObject/PropertyPortFlags.h"

class FExportObjectInnerContext;
class UExporter;
class UWidget;
class UWidgetBlueprint;

class IClipboardExtension
{
public:
	struct FExportArgs
	{
		const FExportObjectInnerContext* Context = nullptr;
		UExporter* Exporter = nullptr; 
		FOutputDevice* Out = nullptr; 
		const TCHAR* FileType = nullptr;
		int32 Indent = 0;
		uint32 PortFlags = PPF_None;
		bool bSelectedOnly = false;
		UObject* ExportRootScope = nullptr;
	};

	virtual void AppendToClipboard(const UWidget* Widget, const FExportArgs& ExportArgs) = 0;
	virtual bool CanAppendToClipboard(const UWidget* Widget) const = 0;
	virtual void ProcessImportedText(const UWidgetBlueprint* WidgetBlueprint, const FString& TextToImport, UPackage*& TempPackage) = 0;
	virtual bool CanImportFromClipboard(const UWidget* Widget) const = 0;
	virtual void ImportDataToWidget(const UWidget* Widget, FName OldWidgetName) = 0;
	virtual bool CanWidgetAcceptPaste(const UWidget* Widget) const = 0;

	virtual ~IClipboardExtension() = default;
};

/**
 * Clipboard extensibility manager holds a list of registered clipboard extensions.
 */
class FClipboardExtensibilityManager
{
public:
	void AddExtension(const TSharedRef<IClipboardExtension>& Extension)
	{
		if (ensure(!Extensions.Contains(Extension)))
		{
			Extensions.Add(Extension);
		}
	}

	void RemoveExtension(const TSharedRef<IClipboardExtension>& Extension)
	{
		int32 NumRemoved = Extensions.RemoveSingleSwap(Extension);
		ensure(NumRemoved == 1);
	}

	const TArrayView<const TSharedPtr<IClipboardExtension>> GetExtensions() const
	{
		return Extensions;
	}

private:
	TArray<TSharedPtr<IClipboardExtension>> Extensions;
};

/** Indicates that a class has data to append to clipboard */
class IHasClipboardExtensibility
{
public:
	virtual TSharedPtr<FClipboardExtensibilityManager> GetClipboardExtensibilityManager() = 0;
};

