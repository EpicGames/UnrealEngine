// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DesignerExtension.h"

/**
 * Factory that creates a unique DesignerExtension when a UMG designer is created.
 */
class IDesignerExtensionFactory
{
public:
	virtual ~IDesignerExtensionFactory() = default;
	virtual TSharedRef<FDesignerExtension> CreateDesignerExtension() const = 0;
};

class FDesignerExtensibilityManager;

/** Indicates that a class has a designer that is extensible */
class IHasDesignerExtensibility
{
public:
	virtual ~IHasDesignerExtensibility() = default;
	virtual TSharedPtr<FDesignerExtensibilityManager> GetDesignerExtensibilityManager() = 0;
};

/**
 * Designer Extensibility Manager keep a series of Designer Extensions. See FDesignerExtension class for more information.
 */
class FDesignerExtensibilityManager
{
public:
	UE_DEPRECATED(4.26, "AddDesignerExtension is deprecated, use the IDesignerExtensibilityFactory instead.")
	void AddDesignerExtension(const TSharedRef<FDesignerExtension>& Extension)
	{
		ExternalExtensions.AddUnique(Extension);
	}

	UE_DEPRECATED(4.26, "RemoveDesignerExtension is deprecated, use the IDesignerExtensibilityFactory instead.")
	void RemoveDesignerExtension(const TSharedRef<FDesignerExtension>& Extension)
	{
		ExternalExtensions.RemoveSingle(Extension);
	}

	UE_DEPRECATED(4.26, "RemoveDesignerExtension is deprecated, use the IDesignerExtensibilityFactory instead.")
	const TArray<TSharedRef<FDesignerExtension>>& GetExternalDesignerExtensions() const
	{
		return ExternalExtensions;
	}

	void AddDesignerExtensionFactory(const TSharedRef<IDesignerExtensionFactory>& Extension)
	{
		ExternalExtensionFactories.AddUnique(Extension);
	}

	void RemoveDesignerExtensionFactory(const TSharedRef<IDesignerExtensionFactory>& Extension)
	{
		ExternalExtensionFactories.RemoveSingle(Extension);
	}

	const TArray<TSharedRef<IDesignerExtensionFactory>>& GetExternalDesignerExtensionFactories() const
	{
		return ExternalExtensionFactories;
	}

private:
	TArray<TSharedRef<FDesignerExtension>> ExternalExtensions;
	TArray<TSharedRef<IDesignerExtensionFactory>> ExternalExtensionFactories;
};
