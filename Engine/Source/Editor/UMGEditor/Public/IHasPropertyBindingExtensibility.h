// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Input/DragAndDrop.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SNullWidget.h"

class IDetailsView;
class IPropertyHandle;
class UWidget;
class UWidgetBlueprint;
struct FSlateBrush;

class IPropertyBindingExtension
{
public:
	enum class EDropResult
	{
		HandledContinue,	// The drop event was handled by the OnDrop call and it can still be re-used for OnDrop calls in other extensions.
		HandledBreak,		// The drop event was handled by the OnDrop call but it is not reusable for OnDrop calls in other extensions (e.g. it was modified).
		Unhandled,			// The drop event was not handled by this extension.
	};

	/** Does this extension want to extend this property in the widget? */
	virtual bool CanExtend(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<const IPropertyHandle> PropertyHandle) const = 0;
	virtual TSharedPtr<FExtender> CreateMenuExtender(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) = 0;
	virtual void ClearCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) = 0;
	virtual TOptional<FName> GetCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const = 0;
	virtual const FSlateBrush* GetCurrentIcon(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const = 0;
	virtual TOptional<FLinearColor> GetCurrentIconColor(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const = 0;
	virtual EDropResult OnDrop(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) = 0;
	virtual TSharedRef<SWidget> OnGenerateMenu(const UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle, TWeakPtr<IDetailsView> DetailsView) { return SNullWidget::NullWidget; };
	virtual bool ShouldAllowEditingValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, TSharedPtr<IPropertyHandle> PropertyHandle) const { return false; };

	UE_DEPRECATED(5.3, "The function CreateMenuExtender with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual TSharedPtr<FExtender> CreateMenuExtender(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) { return nullptr; }
	UE_DEPRECATED(5.6, "The function CanExtend with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual bool CanExtend(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) const { return false; }
	UE_DEPRECATED(5.6, "The function ClearCurrentValue with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual void ClearCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) {}
	UE_DEPRECATED(5.6, "The function GetCurrentValue with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual TOptional<FName> GetCurrentValue(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) const { return TOptional<FName>(); }
	UE_DEPRECATED(5.6, "The function GetCurrentIcon with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual const FSlateBrush* GetCurrentIcon(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) const { return nullptr; }
	UE_DEPRECATED(5.6, "The function GetCurrentIconColor with FProperty parameter is deprecated. Please pass the property handler as the parameter instead.")
	virtual TOptional<FLinearColor> GetCurrentIconColor(const UWidgetBlueprint* WidgetBlueprint, const UWidget* Widget, const FProperty* Property) const { return TOptional<FLinearColor>(); };
};

/**
 * Bindings menu extensibility manager holds a list of registered bindings menu extensions.
 */
class FPropertyBindingExtensibilityManager
{
public:
	void AddExtension(const TSharedRef<IPropertyBindingExtension>& Extension)
	{
		Extensions.AddUnique(Extension);
	}

	void RemoveExtension(const TSharedRef<IPropertyBindingExtension>& Extension)
	{
		Extensions.RemoveSingle(Extension);
	}

	const TArray<TSharedPtr<IPropertyBindingExtension>>& GetExtensions() const
	{
		return Extensions;
	}

private:
	TArray<TSharedPtr<IPropertyBindingExtension>> Extensions;
};

/** Indicates that a class has a bindings menu that is extensible */
class IHasPropertyBindingExtensibility
{
public:
	virtual TSharedPtr<FPropertyBindingExtensibilityManager> GetPropertyBindingExtensibilityManager() = 0;
};

