// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FMenuBuilder;
class FWidgetBlueprintEditor;

class IWidgetContextMenuExtension
{
public:
	virtual ~IWidgetContextMenuExtension() = default;

	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, FVector2D TargetLocation) const = 0;
};

/**
 * Widget Context Menu extensibility manager holds a list of registered widget context menu extensions.
 */
class FWidgetContextMenuExtensibilityManager
{
public:
	void AddExtension(const TSharedRef<IWidgetContextMenuExtension>& Extension)
	{
		if (ensure(!Extensions.Contains(Extension)))
		{
			Extensions.Add(Extension);
		}
	}

	void RemoveExtension(const TSharedRef<IWidgetContextMenuExtension>& Extension)
	{
		int32 NumRemoved = Extensions.RemoveSingleSwap(Extension);
		ensure(NumRemoved == 1);
	}

	TArrayView<const TSharedPtr<IWidgetContextMenuExtension>> GetExtensions() const
	{
		return Extensions;
	}

private:
	TArray<TSharedPtr<IWidgetContextMenuExtension>> Extensions;
};

/** Indicates that a class can extend drag & drop functionality */
class IHasWidgetContextMenuExtensibility
{
public:
	virtual TSharedPtr<FWidgetContextMenuExtensibilityManager> GetWidgetContextMenuExtensibilityManager() = 0;
};
