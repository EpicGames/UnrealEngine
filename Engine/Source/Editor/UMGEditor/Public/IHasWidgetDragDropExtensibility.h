// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

class FDragDropOperation;
class UWidget;

class IWidgetDragDropExtension
{
public:
	virtual ~IWidgetDragDropExtension() = default;

	virtual bool ShouldPreventDropOnTarget(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const = 0;
	virtual FText GetDropFailureText(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp) const = 0;
};

/**
 * Drag & drop extensibility manager holds a list of registered drag and drop extensions.
 */
class FWidgetDragDropExtensibilityManager
{
public:
	void AddExtension(const TSharedRef<IWidgetDragDropExtension>& Extension)
	{
		if (ensure(!Extensions.Contains(Extension)))
		{
			Extensions.Add(Extension);
		}
	}

	void RemoveExtension(const TSharedRef<IWidgetDragDropExtension>& Extension)
	{
		int32 NumRemoved = Extensions.RemoveSingleSwap(Extension);
		ensure(NumRemoved == 1);
	}

	TArrayView<const TSharedPtr<IWidgetDragDropExtension>> GetExtensions() const
	{
		return Extensions;
	}

private:
	TArray<TSharedPtr<IWidgetDragDropExtension>> Extensions;
};

/** Indicates that a class can extend drag & drop functionality */
class IHasWidgetDragDropExtensibility
{
public:
	virtual TSharedPtr<FWidgetDragDropExtensibilityManager> GetWidgetDragDropExtensibilityManager() = 0;
};
