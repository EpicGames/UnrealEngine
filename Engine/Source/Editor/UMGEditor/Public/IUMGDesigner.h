// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "WidgetReference.h"

/**
 * The logical type of transform that can be applied to a widget.
 */
namespace ETransformMode
{
	enum Type
	{
		/** Allows parent transfers */
		Layout,
		/** Only effects the rendered appearance of the widget */
		Render,
	};
}

/**
 * The public interface implemented by the UMG Designer to allow extensions to call methods
 * on the designer.
 */
class IUMGDesigner
{
public:
	/** @return the effective preview scale after both the DPI and Zoom scale has been applied. */
	virtual float GetPreviewScale() const = 0;

	/** @return The currently selected widgets */
	virtual const TSet<FWidgetReference>& GetSelectedWidgets() const = 0;

	/** @return The currently selected widget. */
	virtual FWidgetReference GetSelectedWidget() const = 0;

	/** @return Get the transform mode currently in use in the designer. */
	virtual ETransformMode::Type GetTransformMode() const = 0;

	/** @return The Geometry representing the designer area, useful for when you need to convert mouse into designer space. */
	virtual FGeometry GetDesignerGeometry() const = 0;

	/**
	 * Gets the previous frames widget geometry.
	 */
	virtual bool GetWidgetGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const = 0;

	/**
	 * Gets the previous frames widget geometry.
	 */
	virtual bool GetWidgetGeometry(const UWidget* PreviewWidget, FGeometry& Geometry) const = 0;

	/**
	 * Takes geometry and adds the inverse of the window transform to get the geometry in the space of the window.
	 */
	virtual FGeometry MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const = 0;

	/**
	 * Gets the previous frames widget geometry of the parent of the provided widget.
	 */
	virtual bool GetWidgetParentGeometry(const FWidgetReference& Widget, FGeometry& Geometry) const = 0;

	/**
	 * Marks the designer content as being modified.
	 */
	virtual void MarkDesignModifed(bool bRequiresRecompile) = 0;

	/**
	 * Push a new designer message to show at the bottom of the screen.  Don't forget to call PopDesignerMessage when complete.
	 */
	virtual void PushDesignerMessage(const FText& Message) = 0;

	/**
	 * Removes the last message from the message stack.
	 */
	virtual void PopDesignerMessage() = 0;
};
