// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API UMGEDITOR_API

class UObject;
class UUserWidget;

namespace UE::UMG::Editor
{

	class FPreviewMode
	{
	public:
		UE_API void SetSelectedObject(TArray<TWeakObjectPtr<UObject>> Objects);
		UE_API void SetSelectedObject(TArrayView<UObject*> Objects);
		UE_API TArray<UObject*> GetSelectedObjectList() const;

		FSimpleMulticastDelegate& OnSelectedObjectChanged()
		{
			return SelectedObjectChangedDelegate;
		}

		UE_API void SetPreviewWidget(UUserWidget* Widget);

		UUserWidget* GetPreviewWidget() const
		{
			return PreviewWidget.Get();
		}

		FSimpleMulticastDelegate& OnPreviewWidgetChanged()
		{
			return PreviewWidgetChangedDelegate;
		}

	private:
		TArray<TWeakObjectPtr<UObject>> SelectedObjects;
		TWeakObjectPtr<UUserWidget> PreviewWidget;
		FSimpleMulticastDelegate SelectedObjectChangedDelegate;
		FSimpleMulticastDelegate PreviewWidgetChangedDelegate;
	};

} // namespace UE::UMG

#undef UE_API
