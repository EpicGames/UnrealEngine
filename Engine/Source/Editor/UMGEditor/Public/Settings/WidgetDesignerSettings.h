// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "WidgetPaletteFavorites.h"

#include "WidgetDesignerSettings.generated.h"

#define UE_API UMGEDITOR_API

class UObject;
class UWidgetPaletteFavorites;

UENUM()
enum EDisplayOnCompile : int
{
	DoC_ErrorsOrWarnings,
	DoC_ErrorsOnly,
	DoC_WarningsOnly,
	DoC_Never,
};

/**
 * Implements the settings for the Widget Blueprint Designer.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UWidgetDesignerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UWidgetDesignerSettings();

#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif

public:

	/** If enabled, actor positions will snap to the grid. */
	UPROPERTY(EditAnywhere, config, Category = GridSnapping, meta = (DisplayName = "Enable Grid Snapping"))
	uint32 GridSnapEnabled:1;

	/**
	 * 
	 */
	UPROPERTY(config)
	int32 GridSnapSize;

	/**
	 * 
	 */
	UPROPERTY(EditAnywhere, config, Category = Dragging)
	bool bLockToPanelOnDragByDefault;
	
	/** The default preview resolution in the designer.*/
	UPROPERTY(EditAnywhere, config, Category = Visuals)
	FUintVector2 DefaultPreviewResolution;

	/**
	 * Should the designer show outlines by default?
	 */
	UPROPERTY(EditAnywhere, config, Category = Visuals, meta = ( DisplayName = "Show Dashed Outlines By Default" ))
	bool bShowOutlines;

	/**
	 * Should the designer run the design event?  Disable this if you're seeing crashes in the designer,
	 * you may have some unsafe code running in the designer.
	 */
	UPROPERTY(EditAnywhere, config, Category = Visuals)
	bool bExecutePreConstructEvent;

	/**
	 * Should the designer respect locked widgets?  If true, the designer by default
	 * will not allow you to select locked widgets in the designer view.
	 */
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bRespectLocks;

	/** Setting to automatically create compile tab based on compile results */
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	TEnumAsByte<EDisplayOnCompile> CreateOnCompile;

	/** Setting to automatically dismiss compile tab based on compile results */
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	TEnumAsByte<EDisplayOnCompile> DismissOnCompile;
	
	/**
	 * List of Favorites widgets used to populate the Favorites Palette
	 */
	UPROPERTY()
	TObjectPtr<UWidgetPaletteFavorites> Favorites;

};

#undef UE_API
