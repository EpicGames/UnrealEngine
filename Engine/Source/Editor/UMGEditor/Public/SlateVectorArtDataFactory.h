// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SlateVectorArtDataFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(hidecategories = Object, collapsecategories, MinimalAPI)
class USlateVectorArtDataFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// ~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// ~ UFactory Interface	
};
