// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Input/Reply.h"
#include "AssetRegistry/AssetData.h"
#include "Blueprint/UserWidget.h"
#include "Widgets/IToolTip.h"
#include "Templates/WidgetTemplateClass.h"

#define UE_API UMGEDITOR_API

class UWidgetTree;

/**
* A template for classes generated by UWidgetBlueprint assets
* Accounts for the possibility that a the blueprint may not be loaded when the palette is created
*/
class FWidgetTemplateBlueprintClass : public FWidgetTemplateClass
{
public:
	/** 
	* Constructor
	* @param InWidgetAssetData The FAssetData for the widget blueprint asset
	* @param InUserWidgetClass The user-created widget class that will be created by this template
	* @param bCacheParentClass True means we will search for parent class and cache, this is a slow operation, but useful for BPGCs.
	*/
	UE_API FWidgetTemplateBlueprintClass(const FAssetData& InWidgetAssetData, TSubclassOf<UUserWidget> InUserWidgetClass = {});

	//UE_DEPRECATED(5.3, "FWidgetTemplateBlueprintClass with bInIsBlueprintGeneratedClass is deprecated.")
	//FWidgetTemplateBlueprintClass(const FAssetData& InWidgetAssetData, TSubclassOf<UUserWidget> InUserWidgetClass, bool bInIsBlueprintGeneratedClass);

	/** Destructor */
	UE_API virtual ~FWidgetTemplateBlueprintClass();

	/** Gets the category for the widget */
	UE_API virtual FText GetCategory() const override;

	/** Creates an instance of the widget for the tree. */
	UE_API virtual UWidget* Create(UWidgetTree* Tree) override;

	/** The icon coming from the default object of the class */
	UE_API virtual const FSlateBrush* GetIcon() const override;

	/** Gets the tooltip widget for this palette item. */
	UE_API virtual TSharedRef<IToolTip> GetToolTip() const override;

	/** Opens the widget blueprint for edit */
	UE_API virtual FReply OnDoubleClicked() override;

	/** Returns true if the supplied class is supported by this template */
	static UE_API bool Supports(UClass* InClass);

protected:
	/** Parent Class of this widget template, may not be valid */
	TWeakObjectPtr<UClass> CachedParentClass;

	/** True if this template is for a cooked BP generated class */
	bool bIsBlueprintGeneratedClass;
};

#undef UE_API
