// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Components/Widget.h"
#include "Widgets/IToolTip.h"
#include "WidgetTemplate.h"
#include "AssetRegistry/AssetData.h"

#define UE_API UMGEDITOR_API

class UWidgetTree;

/**
 * A template that can spawn any widget derived from the UWidget class.
 */
class FWidgetTemplateClass : public FWidgetTemplate
{
public:
	/** Constructor */
	UE_API explicit FWidgetTemplateClass(TSubclassOf<UWidget> InWidgetClass);

	UE_API explicit FWidgetTemplateClass(const FAssetData& InWidgetAssetData, TSubclassOf<UWidget> InWidgetClass);

	/** Destructor */
	UE_API virtual ~FWidgetTemplateClass();

	/** Gets the category for the widget */
	UE_API virtual FText GetCategory() const override;

	/** Creates an instance of the widget for the tree. */
	UE_API virtual UWidget* Create(UWidgetTree* Tree) override;

	/** The icon coming from the default object of the class */
	UE_API virtual const FSlateBrush* GetIcon() const override;

	/** Gets the tooltip widget for this palette item. */
	UE_API virtual TSharedRef<IToolTip> GetToolTip() const override;

	/** Gets the tooltip text for this palette item (the text that GetToolTip wraps into an IToolTip widget). */
	UE_API virtual FText GetToolTipText() const;

	/** @param OutStrings - Returns an array of strings used for filtering/searching this widget template. */
	UE_API virtual void GetFilterStrings(TArray<FString>& OutStrings) const override;

	/** Gets the WidgetClass which might be null. */
	TWeakObjectPtr<UClass> GetWidgetClass() const { return WidgetClass; }

	/** Returns the asset data for this widget which might be invalid. */
	const FAssetData& GetWidgetAssetData() { return WidgetAssetData; }

protected:
	/** Creates a widget template class without any class reference */
	UE_API FWidgetTemplateClass();

	/** Called when objects need to be swapped out for new versions, like after a blueprint recompile. */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Constructs the widget template with an overridden object name. */
	UE_API UWidget* CreateNamed(class UWidgetTree* Tree, FName NameOverride);

protected:
	/** The widget class that will be created by this template */
	TWeakObjectPtr<UClass> WidgetClass;

	/** The asset data for the widget blueprint */
	FAssetData WidgetAssetData;

	/** Parent Class of this widget template, may not be valid */
	TWeakObjectPtr<UClass> CachedParentClass;
};

#undef UE_API
