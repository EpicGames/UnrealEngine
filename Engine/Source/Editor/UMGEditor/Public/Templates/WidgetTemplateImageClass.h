// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Templates/WidgetTemplateClass.h"

#define UE_API UMGEDITOR_API

class UWidgetTree;

/**
 * A template for classes generated by UTexture or UMaterial classes, or implements a USlateTextureAtlasInterface
 */

class FWidgetTemplateImageClass : public FWidgetTemplateClass
{

public:
	/**
	 * Constructor.
	 * @param	InAssetData		The asset data used to create the widget
	 */
	UE_API FWidgetTemplateImageClass(const FAssetData& InAssetData);

	UE_API virtual ~FWidgetTemplateImageClass();

	/** Creates an instance of the widget for the tree */
	UE_API virtual UWidget* Create(UWidgetTree* WidgetTree) override;

	/** Returns true if the supplied class is supported by this template */
	static UE_API bool Supports(UClass* InClass);
};

#undef UE_API
