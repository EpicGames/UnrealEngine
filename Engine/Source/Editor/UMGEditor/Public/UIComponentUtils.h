// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ClassViewerFilter.h"
#include "UObject/ObjectMacros.h"

#define UE_API UMGEDITOR_API

class UUIComponent;
class UWidget;
class UWidgetBlueprint;
class FWidgetBlueprintEditor;
class FUIComponentUtils
{
public:
	class FUIComponentClassFilter : public IClassViewerFilter
	{
	public:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet <const UClass*> AllowedChildrenOfClasses;
		TSet<const UClass*> ExcludedChildrenOfClasses;

		/** Classes in the same hierarchy as any of these will be excluded (both ancestors and descendants). */
		TSet<const UClass*> ExcludedHierarchyClasses;

		/** Disallowed class flags. */
		EClassFlags DisallowedClassFlags;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override;

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override;
	};

	static UE_API FClassViewerInitializationOptions CreateClassViewerInitializationOptions(const TSet<const UClass*>& ExistingComponentClasses = TSet<const UClass*>());

	static UE_API void OnWidgetRenamed(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const FName& OldVarName, const FName& NewVarName);

	UE_DEPRECATED(5.9, "Use the FWidgetBlueprintEditor* overload instead.")
	static void OnWidgetRenamed(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const FName& OldVarName, const FName& NewVarName)
	{
		OnWidgetRenamed(&BlueprintEditor.Get(), WidgetBlueprint, OldVarName, NewVarName);
	}

	static UE_API void ReplaceComponentVariableReferences(UWidgetBlueprint* WidgetBlueprint, const FName& OldWidgetName, const FName& NewWidgetName);
	
	static UE_API UWidget* AddComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClass, const FName WidgetName, FText& OutErrorMessage);
	
	UE_DEPRECATED(5.9, "Use the FWidgetBlueprintEditor* overload instead.")
	static UE_API void AddComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClass, const FName WidgetName);

	static UE_API bool RemoveComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClass, const FName WidgetName, FText& OutErrorMessage);

	UE_DEPRECATED(5.9, "Use the FWidgetBlueprintEditor* overload instead.")
	static UE_API void RemoveComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClass, const FName WidgetName);

	static UE_API bool MoveComponent(FWidgetBlueprintEditor* BlueprintEditor, UWidgetBlueprint* WidgetBlueprint, const UClass* ComponentClassToMove, const UClass* RelativeToComponentClass, const FName WidgetName, bool bMoveAfter, FText& OutErrorMessage);

	UE_DEPRECATED(5.9, "Use the FWidgetBlueprintEditor* overload instead.")
	static UE_API void MoveComponent(const TSharedRef<FWidgetBlueprintEditor>& BlueprintEditor, const UClass* ComponentClassToMove, const UClass* RelativeToComponentClass, const FName WidgetName, bool bMoveAfter);
	
};

#undef UE_API
