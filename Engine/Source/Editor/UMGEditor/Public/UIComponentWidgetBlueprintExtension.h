// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WidgetBlueprintExtension.h"

#include "Extensions/UIComponent.h"

#include "UIComponentWidgetBlueprintExtension.generated.h"

#define UE_API UMGEDITOR_API

class UUIComponentUserWidgetExtension;
class UUIComponentContainer;

UCLASS()
/***
 * Extension to the Widget Blueprint that will save the Components and will be used in the editor for compilation.
 */
class UUIComponentWidgetBlueprintExtension : public UWidgetBlueprintExtension
{
	GENERATED_BODY()
	
	UE_API explicit UUIComponentWidgetBlueprintExtension(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

public:
	static UE_API const FName MD_ComponentVariable;

	UE_API UUIComponent* AddComponent(const UClass* ComponentClass, FName OwnerName, FText& OutErrorMessage);
	UE_API UUIComponent* AddOrReplaceComponent(UUIComponent* Component, FName OwnerName);
	UE_API void RemoveComponent(const UClass* ComponentClass, FName OwnerName);
	UE_API void MoveComponent(FName OwnerName, const UClass* ComponentClass, const UClass* RelativeToComponentClass, bool bMoveAfter);
	UE_API TArray<UUIComponent*> GetComponentsFor(const UWidget* Target) const;
	UE_API UUIComponent* GetComponent(const UClass* ComponentClass, FName OwnerName) const;
	UE_API EDataValidationResult ValidateComponents(FDataValidationContext& Context) const;

	UE_API UUIComponentUserWidgetExtension* GetOrCreateExtension(UUserWidget* PreviewWidget);
	UE_API void RenameWidget(const FName& OldVarName, const FName& NewVarName);
	UE_API void ReplaceComponentVariableReferences(const FName& OldWidgetName, const FName& NewWidgetName);

	UE_API bool VerifyContainer(UUserWidget* UserWidget) const;

protected:
	UE_API virtual void HandleBeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext) override;	
	UE_API virtual void HandleCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO) override;
	UE_API virtual void HandlePopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context) override;
	UE_API virtual void HandleFinishCompilingClass(UWidgetBlueprintGeneratedClass* Class) override;
	UE_API virtual bool HandleValidateGeneratedClass(UWidgetBlueprintGeneratedClass* Class) override; 

	UE_API virtual void HandleEndCompilation() override;

private:	
	UE_API UUIComponentContainer* DuplicateContainer(UObject* Outer) const;

	FWidgetBlueprintCompilerContext* CompilerContext = nullptr;

#if WITH_EDITORONLY_DATA	
	UPROPERTY()
	TObjectPtr<UUIComponentContainer> ComponentContainer;
#endif //WITH_EDITORONLY_DATA
};

#undef UE_API
