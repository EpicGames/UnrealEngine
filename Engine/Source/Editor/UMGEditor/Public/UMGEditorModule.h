// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IHasClipboardExtensibility.h"
#include "IHasDesignerExtensibility.h"
#include "IHasPropertyBindingExtensibility.h"
#include "IHasWidgetDragDropExtensibility.h"
#include "IHasWidgetContextMenuExtensibility.h"
#include "UObject/TopLevelAssetPath.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUMGEditor, Log, All);

extern const FName UMGEditorAppIdentifier;

class FWidgetBlueprintApplicationMode;
class FWidgetBlueprintEditor;
class FWorkflowAllowedTabSet;
class IBlueprintWidgetCustomizationExtender;
class IPropertyTypeCustomization;

/** The public interface of the UMG editor module. */
class IUMGEditorModule : 
	public IModuleInterface, 
	public IHasMenuExtensibility, 
	public IHasToolBarExtensibility, 
	public IHasDesignerExtensibility,
	public IHasPropertyBindingExtensibility,
	public IHasClipboardExtensibility,
	public IHasWidgetDragDropExtensibility,
	public IHasWidgetContextMenuExtensibility
{
public:
	virtual class FWidgetBlueprintCompiler* GetRegisteredCompiler() = 0;

	DECLARE_EVENT_TwoParams(IUMGEditorModule, FOnRegisterTabs, const FWidgetBlueprintApplicationMode&, FWorkflowAllowedTabSet&);
	virtual FOnRegisterTabs& OnRegisterTabsForEditor() = 0;

	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FWidgetEditorToolbarExtender, const TSharedRef<FUICommandList>, TSharedRef<FWidgetBlueprintEditor>);

	/** Add Toolbar extender for widget editor, used by widget tool palette. */
	virtual void AddWidgetEditorToolbarExtender(FWidgetEditorToolbarExtender&& InToolbarExtender) = 0;

	/** Get current toolbar extenders for widget editor */
	virtual TArrayView<FWidgetEditorToolbarExtender> GetAllWidgetEditorToolbarExtenders() = 0;

	/** Add customization for widget */
	virtual void AddWidgetCustomizationExtender(const TSharedRef<IBlueprintWidgetCustomizationExtender>& WidgetCustomizationExtender) = 0;

	/** Remove customization for widget */
	virtual void RemoveWidgetCustomizationExtender(const TSharedRef<IBlueprintWidgetCustomizationExtender>& WidgetCustomizationExtender) = 0;

	/** Get current customization extenders for widget */
	virtual TArrayView<TSharedRef<IBlueprintWidgetCustomizationExtender>> GetAllWidgetCustomizationExtenders() = 0;

	/** Support for general layout extenders */
	DECLARE_EVENT_OneParam(IUMGEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() = 0;

	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<IPropertyTypeCustomization>, FOnGetInstancePropertyTypeCustomizationInstance, TWeakPtr<FWidgetBlueprintEditor> Editor);
	/** Add an instance customization to the widget property view. */
	virtual void RegisterInstancedCustomPropertyTypeLayout(FTopLevelAssetPath Type, FOnGetInstancePropertyTypeCustomizationInstance) = 0;

	/** Remove an instance customization to the widget property view. */
	virtual void UnregisterInstancedCustomPropertyTypeLayout(FTopLevelAssetPath Type) = 0;

	struct FCustomPropertyTypeLayout
	{
		FTopLevelAssetPath Type;
		FOnGetInstancePropertyTypeCustomizationInstance Delegate;
	};

	/** Remove an instance customization to the widget property view. */
	virtual TArrayView<const FCustomPropertyTypeLayout> GetAllInstancedCustomPropertyTypeLayout() const = 0;

	/** Arguments for the OnBlueprintCreated callback. */
	struct FWidgetBlueprintCreatedArgs
	{
		UClass* ParentClass = nullptr;
		UWidgetBlueprint* Blueprint = nullptr;
	};
	DECLARE_EVENT_OneParam(IUMGEditorModule, FOnWidgetBlueprintCreated, FWidgetBlueprintCreatedArgs);

	/** Callback when a WidgetBlueprint is created by the factory. */
	virtual FOnWidgetBlueprintCreated& OnWidgetBlueprintCreated() = 0;
};
