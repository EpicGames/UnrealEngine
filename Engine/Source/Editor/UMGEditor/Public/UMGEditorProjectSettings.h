// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "WidgetEditingProjectSettings.h"
#include "Engine/EngineTypes.h"
#include "Misc/NamePermissionList.h"
#include "UMGEditorProjectSettings.generated.h"

#define UE_API UMGEDITOR_API

class UWidgetCompilerRule;
class UUserWidget;
class UWidgetBlueprint;
class UPanelWidget;


/**
 * Implements the settings for the UMG Editor Project Settings
 */
UCLASS(MinimalAPI, config=Editor, defaultconfig)
class UUMGEditorProjectSettings : public UWidgetEditingProjectSettings
{
	GENERATED_BODY()

public:
	UE_API UUMGEditorProjectSettings();

#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif

#if WITH_EDITOR
	// Begin UObject Interface
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End UObject Interface
#endif

};

#undef UE_API
