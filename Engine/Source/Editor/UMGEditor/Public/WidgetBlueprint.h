// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "BaseWidgetBlueprint.h"
#include "Binding/DynamicPropertyPath.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Animation/WidgetAnimationBinding.h"
#include "Templates/ValueOrError.h"

#include "WidgetBlueprint.generated.h"

#define UE_API UMGEDITOR_API

class FCompilerResultsLog;
class UEdGraph;
class UMovieScene;
class UTexture2D;
class UUserWidget;
class UWidget;
class UWidgetAnimation;
class FKismetCompilerContext;
class UWidgetBlueprint;
enum class EWidgetTickFrequency : uint8;
enum class EWidgetCompileTimeTickPrediction : uint8;
class UWidgetEditingProjectSettings;


/** Widget Delegates */
class FWidgetBlueprintDelegates
{
public:
	// delegate for generating widget asset registry tags.
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FGetAssetTagsWithContext, const UWidgetBlueprint*, FAssetRegistryTagsContext);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FGetAssetTags, const UWidgetBlueprint*, TArray<UObject::FAssetRegistryTag>&);

	// called by UWdgetBlueprint::GetAssetRegistryTags()
	static UE_API FGetAssetTagsWithContext GetAssetTagsWithContext;
};


/** */
USTRUCT()
struct FEditorPropertyPathSegment
{
	GENERATED_USTRUCT_BODY()

public:
	UE_API FEditorPropertyPathSegment();
	UE_API FEditorPropertyPathSegment(const FProperty* InProperty);
	UE_API FEditorPropertyPathSegment(const UFunction* InFunction);
	UE_API FEditorPropertyPathSegment(const UEdGraph* InFunctionGraph);

	UStruct* GetStruct() const { return Struct; }
	UE_API FFieldVariant GetMember() const;

	UE_API void Rebase(UBlueprint* SegmentBase);
	UE_API bool ValidateMember(FDelegateProperty* DelegateProperty, FText& OutError) const;

	UE_API FName GetMemberName() const;
	UE_API FText GetMemberDisplayText() const;
	UE_API FGuid GetMemberGuid() const;

private:

	/** The owner of the path segment (ie. What class or structure was this property from) */
	UPROPERTY()
	TObjectPtr<UStruct> Struct;

	/** The member name in the structure this segment represents. */
	UPROPERTY()
	FName MemberName;

	/**
	 * The member guid in this structure this segment represents.  If this is valid it should 
	 * be used instead of Name to get the true name.
	 */
	UPROPERTY()
	FGuid MemberGuid;

	/** true if property, false if function */
	UPROPERTY()
	bool IsProperty;
};


/**  */
USTRUCT()
struct FEditorPropertyPath
{
	GENERATED_USTRUCT_BODY()

public:

	/**  */
	UE_API FEditorPropertyPath();

	/**  */
	UE_API FEditorPropertyPath(const TArray<FFieldVariant>& BindingChain);

	/**  */
	UE_API bool Rebase(UBlueprint* SegmentBase);

	/**  */
	bool IsEmpty() const { return Segments.Num() == 0; }

	/**  */
	UE_API bool Validate(FDelegateProperty* Destination, FText& OutError) const;

	/**  */
	UE_API FText GetDisplayText() const;

	/**  */
	UE_API FDynamicPropertyPath ToPropertyPath() const;

public:

	/** The path of properties. */
	UPROPERTY()
	TArray<FEditorPropertyPathSegment> Segments;
};

/** */
USTRUCT()
struct FDelegateEditorBinding
{
	GENERATED_USTRUCT_BODY()

	/** The member widget the binding is on, must be a direct variable of the UUserWidget. */
	UPROPERTY()
	FString ObjectName;

	/** The property on the ObjectName that we are binding to. */
	UPROPERTY()
	FName PropertyName;

	/** The function that was generated to return the SourceProperty */
	UPROPERTY()
	FName FunctionName;

	/** The property we are bindings to directly on the source object. */
	UPROPERTY()
	FName SourceProperty;

	/**  */
	UPROPERTY()
	FEditorPropertyPath SourcePath;

	/** If it's an actual Function Graph in the blueprint that we're bound to, there's a GUID we can use to lookup that function, to deal with renames better.  This is that GUID. */
	UPROPERTY()
	FGuid MemberGuid;

	UPROPERTY()
	EBindingKind Kind = EBindingKind::Property;

	bool operator==( const FDelegateEditorBinding& Other ) const
	{
		// NOTE: We intentionally only compare object name and property name, the function is irrelevant since
		// you're only allowed to bind a property on an object to a single function.
		return ObjectName == Other.ObjectName && PropertyName == Other.PropertyName;
	}

	UE_API bool IsAttributePropertyBinding(class UWidgetBlueprint* Blueprint) const;

	UE_API bool DoesBindingTargetExist(UWidgetBlueprint* Blueprint) const;

	UE_API bool IsBindingValid(UClass* Class, class UWidgetBlueprint* Blueprint, FCompilerResultsLog& MessageLog) const;

	UE_API FDelegateRuntimeBinding ToRuntimeBinding(class UWidgetBlueprint* Blueprint) const;
};

UENUM()
enum class EWidgetSupportsDynamicCreation : uint8
{
	Default,
	Yes,
	No,
};


UENUM()
enum class EThumbnailPreviewSizeMode : uint8
{
	MatchDesignerMode,
	FillScreen,
	Custom,
	Desired
};



/**
 * This represents the tickability of a widget computed at compile time
 * It is designed as a hint so the runtime can determine if ticking needs to be enabled
 * A lot of widgets set to WillTick means you might have a performance problem
 */
UENUM()
enum class EWidgetCompileTimeTickPrediction : uint8
{
	/** The widget is manually set to never tick or we dont detect any animations, latent actions, and/or script or possible native tick methods */
	WontTick,

	/** This widget is set to auto tick and we detect animations, latent actions but not script or native tick methods*/
	OnDemand,

	/** This widget has an implemented script tick or native tick */
	WillTick,
};

/**
 * The widget blueprint enables extending UUserWidget the user extensible UWidget.
 */
UCLASS(MinimalAPI, BlueprintType)
class UWidgetBlueprint : public UBaseWidgetBlueprint
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray< FDelegateEditorBinding > Bindings;

	UPROPERTY()
	TArray<TObjectPtr<UWidgetAnimation>> Animations;
	
	/**
	 * Saved Guids for variables generated by this blueprint (Widgets and Animations).
	 *
	 * Used to fixup external references to renamed widgets/animations in this blueprint
	 *
	 * See: OnVariableAdded, OnVariableRenamed, OnVariableRemoved
	 */
	UPROPERTY()
	TMap<FName, FGuid> WidgetVariableNameToGuidMap;

	/**
	 * Don't directly modify this property to change the palette category.  The actual value is stored 
	 * in the CDO of the UUserWidget, but a copy is stored here so that it's available in the serialized 
	 * Tag data in the asset header for access in the FAssetData.
	 */
	UPROPERTY(AssetRegistrySearchable)
	FString PaletteCategory;

	/**
	 * Determines whether this widget blueprint can be initialized without
	 * a valid player context (PlayerController, etc.).
	 * Required to be true for use with UMG Widget Preview.
	 */
	UPROPERTY(EditAnywhere, Category="Widget")
	bool bCanCallInitializedWithoutPlayerContext;

	UE_API void OnVariableAdded(const FName& VariableName);
	UE_API void OnVariableRenamed(const FName& OldName, const FName& NewName);
	UE_API void OnVariableRemoved(const FName& VariableName);
#endif

public:

	/** UObject interface */
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

#if WITH_EDITORONLY_DATA
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API virtual void NotifyGraphRenamed(class UEdGraph* Graph, FName OldName, FName NewName) override;
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	UE_API bool DetectSlateWidgetLeaks(class FDataValidationContext& Context) const;
	UE_API virtual bool FindDiffs(const UBlueprint* OtherBlueprint, FDiffResults& Results) const override;
#endif

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API UPackage* GetWidgetTemplatePackage() const;

	UE_API virtual void ReplaceDeprecatedNodes() override;
	
	//~ Begin UBlueprint Interface
	UE_API virtual UClass* GetBlueprintClass() const override;

	UE_API virtual bool AllowsDynamicBinding() const override;

	UE_API virtual bool SupportsInputEvents() const override;

	virtual bool SupportedByDefaultBlueprintFactory() const override
	{
		return false;
	}

	UE_API virtual void GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const override;

	/** UWidget blueprints are never data only, should always compile on load (data only blueprints cannot declare new variables) */
	virtual bool AlwaysCompileOnLoad() const override { return true; }
	//~ End UBlueprint Interface

	UE_API virtual void GatherDependencies(TSet<TWeakObjectPtr<UBlueprint>>& InDependencies) const override;

	/** Returns true if the supplied user widget will not create a circular reference when added to this blueprint */
	UE_API bool IsWidgetFreeFromCircularReferences(UUserWidget* UserWidget) const;
	
	/**  */
	UE_API TValueOrError<void, UWidget*> HasCircularReferences() const;

	UE_API TValueOrError<void, TSet<UWidget*>> HasConflictingWidgetNamesFromInheritance() const;


	static UE_API bool ValidateGeneratedClass(const UClass* InClass);
	
	static UE_API TSharedPtr<FKismetCompilerContext> GetCompilerForWidgetBP(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);

	UE_DEPRECATED(5.8, "Use UpdateTickabilityStatsAndInputRegistration instead.")
	UE_API void UpdateTickabilityStats(bool& OutHasLatentActions, bool& OutHasAnimations, bool& OutClassRequiresNativeTick);
	
	/** Queries the parent widget chain to determine which features we can enable/disable based on what is needed. */
	UE_API void UpdateTickabilityStatsAndInputRegistration(bool& OutHasLatentActions, bool& OutHasAnimations, bool& OutClassRequiresNativeTick, bool& OutClassRequiresInputReg);

	UE_API bool ArePropertyBindingsAllowed() const;

	/** Gets any named slots exposed by the parent generated class that can be slotted into by the subclass. */
	UE_API TArray<FName> GetInheritedAvailableNamedSlots() const;

	/** Gets named slots exposed by the parent generated class that already have content in the parent widget tree. */
	UE_API TSet<FName> GetInheritedNamedSlotsWithContentInSameTree() const;

	UE_API virtual UWidgetEditingProjectSettings* GetRelevantSettings();
	UE_API virtual const UWidgetEditingProjectSettings* GetRelevantSettings() const;

protected:

	UE_API virtual void LoadModulesRequiredForCompilation() override;

private:

	UE_API void FixupLegacySequencePlayerCalls();

private:
	/**
	 * The desired tick frequency set by the user on the UserWidget's CDO.
	 */
	UPROPERTY(AssetRegistrySearchable)
	EWidgetTickFrequency TickFrequency;

	/**
	 * The computed frequency that the widget will need to be ticked at.  You can find the reasons for
	 * this decision by looking at TickPredictionReason.
	 */
	UPROPERTY(AssetRegistrySearchable)
	EWidgetCompileTimeTickPrediction TickPrediction;

	/**
	 * The reasons we may need to tick this widget.
	 */
	UPROPERTY(AssetRegistrySearchable)
	FString TickPredictionReason;

public:

	/**
	 * The total number of property bindings.  Consider this as a performance warning.
	 */
	UPROPERTY(AssetRegistrySearchable)
	int32 PropertyBindings;

	UPROPERTY(EditDefaultsOnly, Category = ThumbnailSettings)
	EThumbnailPreviewSizeMode ThumbnailSizeMode;

	UPROPERTY(EditDefaultsOnly, Category = ThumbnailSettings)
	FVector2D ThumbnailCustomSize;

	UPROPERTY(EditDefaultsOnly, Category = ThumbnailSettings)
	TObjectPtr<UTexture2D> ThumbnailImage;

};

#undef UE_API
