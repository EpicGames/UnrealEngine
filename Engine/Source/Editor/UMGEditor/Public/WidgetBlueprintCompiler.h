// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineLogs.h"
#include "Engine/Blueprint.h"
#include "WidgetBlueprint.h"
#include "KismetCompiler.h"
#include "KismetCompilerModule.h"
#include "ComponentReregisterContext.h"
#include "Components/WidgetComponent.h"

#define UE_API UMGEDITOR_API

class FProperty;
class UEdGraph;
class UWidget;
class UWidgetAnimation;
class UWidgetGraphSchema;

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompiler 

class FWidgetBlueprintCompiler : public IBlueprintCompiler
{

public:
	UE_API FWidgetBlueprintCompiler();

	UE_API bool CanCompile(const UBlueprint* Blueprint) override;
	UE_API void PreCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions) override;
	UE_API void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
	UE_API void PostCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions) override;
	UE_API bool GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const override;

private:

	/** The temporary variable that captures and reinstances components after compiling finishes. */
	TComponentReregisterContext<UWidgetComponent>* ReRegister;

	/**
	* The current count on the number of compiles that have occurred.  We don't want to re-register components until all
	* compiling has stopped.
	*/
	int32 CompileCount;

};

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintCompilerContext


class FWidgetBlueprintCompilerContext : public FKismetCompilerContext
{
protected:
	typedef FKismetCompilerContext Super;

public:
	UE_API FWidgetBlueprintCompilerContext(UWidgetBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions);
	UE_API virtual ~FWidgetBlueprintCompilerContext();

protected:
	/**
	 * Checks if the animations' bindings are valid. Shows a compiler warning if any of the animations have a null track, urging the user
	 * to delete said track to avoid performance hitches when playing said null tracks in large user widgets.
	*/
	UE_API void ValidateWidgetAnimations();

	/** Validates the Desired Focus name to make sure it's part of the Widget Tree. */
	UE_API void ValidateDesiredFocusWidgetName();

	//~ Begin FKismetCompilerContext
	UE_API virtual UEdGraphSchema_K2* CreateSchema() override;
	UE_API virtual void CreateFunctionList() override;
	UE_API virtual void SpawnNewClass(const FString& NewClassName) override;
	UE_API virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) override;
	UE_API virtual void PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags) override;
	UE_API virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOutOldCDO) override;
	UE_API virtual void SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean) override;
	UE_API virtual void EnsureProperGeneratedClass(UClass*& TargetClass) override;
	UE_API virtual void PopulateBlueprintGeneratedVariables() override;
	UE_API virtual void CreateClassVariablesFromBlueprint() override;
	UE_API virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject);
	UE_API virtual void FinishCompilingClass(UClass* Class) override;
	UE_API virtual bool ValidateGeneratedClass(UBlueprintGeneratedClass* Class) override;
	UE_API virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context) override;
	//~ End FKismetCompilerContext

	UE_API void SanitizeBindings(UBlueprintGeneratedClass* Class);
	UE_API void ValidateAndFixUpVariableGuids();

	UE_API void VerifyEventReplysAreNotEmpty(FKismetFunctionContext& Context);
	UE_API void VerifyFieldNotifyFunction(FKismetFunctionContext& Context);

public:
	UWidgetBlueprint* WidgetBlueprint() const { return Cast<UWidgetBlueprint>(Blueprint); }

	UE_API void AddExtension(UWidgetBlueprintGeneratedClass* Class, UWidgetBlueprintGeneratedClassExtension* Extension);
	UE_API void AddGeneratedVariable(FBPVariableDescription&& VariableDescription) const;
	
	struct FPopulateGeneratedVariablesContext
	{
	public:
		UE_API void AddGeneratedVariable(FBPVariableDescription&& VariableDescription) const;
		UE_API UWidgetBlueprint* GetWidgetBlueprint() const;

	private:
		friend FWidgetBlueprintCompilerContext;
		UE_API FPopulateGeneratedVariablesContext(FWidgetBlueprintCompilerContext& InContext);
		FWidgetBlueprintCompilerContext& Context;
	};

	struct FCreateVariableContext
	{
	public:
		UE_API FProperty* CreateVariable(const FName Name, const FEdGraphPinType& Type) const;
		UE_API FMulticastDelegateProperty* CreateMulticastDelegateVariable(const FName Name, const FEdGraphPinType& Type) const;
		UE_API FMulticastDelegateProperty* CreateMulticastDelegateVariable(const FName Name) const;
		UE_API void AddGeneratedFunctionGraph(UEdGraph* Graph) const;
		UE_API UWidgetBlueprint* GetWidgetBlueprint() const;
		UE_DEPRECATED(5.4, "GetSkeletonGeneratedClass renamed to GetGeneratedClass")
		UE_API UWidgetBlueprintGeneratedClass* GetSkeletonGeneratedClass() const;
		UE_API UWidgetBlueprintGeneratedClass* GetGeneratedClass() const;
		UE_API EKismetCompileType::Type GetCompileType() const;

	private:
		friend FWidgetBlueprintCompilerContext;
		UE_API FCreateVariableContext(FWidgetBlueprintCompilerContext& InContext);
		FWidgetBlueprintCompilerContext& Context;
	};

	struct FCreateFunctionContext
	{
	public:
		UE_API void AddGeneratedFunctionGraph(UEdGraph* Graph) const;
		UE_API void AddGeneratedUbergraphPage(UEdGraph* Graph) const;
		UE_API UWidgetBlueprintGeneratedClass* GetGeneratedClass() const;

	private:
		friend FWidgetBlueprintCompilerContext;
		UE_API FCreateFunctionContext(FWidgetBlueprintCompilerContext& InContext);
		FWidgetBlueprintCompilerContext& Context;
	};

	/** Duplicate the widget tree of the passed widget blueprint and assign it as the widget tree of the passed generated class. */
	UE_API void static UpdateGeneratedClassWidgetTree(UWidgetBlueprint* WidgetBP, UWidgetBlueprintGeneratedClass* BPGClass, UWidgetTree* OldWidgetTree, FCompilerResultsLog* MessageLog);
	UE_API void static ClearGeneratedClassWidgetTree(UWidgetBlueprintGeneratedClass* WBC_ToClean, bool bRecompilingOnLoad);

protected:
	UE_API void static FixAbandonedWidgetTree(UWidgetBlueprint* WidgetBP);

	UWidgetBlueprintGeneratedClass* NewWidgetBlueprintClass;

	UWidgetTree* OldWidgetTree;

	TArray<UWidgetAnimation*> OldWidgetAnimations;

	UWidgetGraphSchema* WidgetSchema;

	// Map of properties created for widgets; to aid in debug data generation
	TMap<UWidget*, FProperty*> WidgetToMemberVariableMap;

	// Map of properties created in parent widget for bind widget validation
	TMap<UWidget*, FProperty*> ParentWidgetToBindWidgetMap;

	// Map of properties created for widget animations; to aid in debug data generation
	TMap<UWidgetAnimation*, FProperty*> WidgetAnimToMemberVariableMap;

	///----------------------------------------------------------------
};

#undef UE_API
