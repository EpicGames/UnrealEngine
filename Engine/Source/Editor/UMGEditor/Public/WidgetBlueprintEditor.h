// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Settings/WidgetDesignerSettings.h"
#include "AssetRegistry/AssetData.h"
#include "PreviewScene.h"
#include "GraphEditor.h"
#include "BlueprintEditor.h"
#include "ISequencer.h"
#include "WidgetReference.h"
#include "Blueprint/UserWidget.h"

#define UE_API UMGEDITOR_API

class FMenuBuilder;
class FWidgetBlueprintEditorToolbar;
class IMessageLogListing;
class STextBlock;
class UPanelSlot;
class UWidgetAnimation;
class UWidgetBlueprint;
class FPaletteViewModel;
class FLibraryViewModel;
struct FPropertyChangedEvent;
namespace UE::UMG::Editor { class FPreviewMode; }

struct FNamedSlotSelection
{
	FWidgetReference NamedSlotHostWidget;
	FName SlotName;
};

/**
 * widget blueprint editor (extends Blueprint editor)
 */
class FWidgetBlueprintEditor : public FBlueprintEditor
{
private:
	using Super = FBlueprintEditor;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHoveredWidgetSet, const FWidgetReference&)
	DECLARE_MULTICAST_DELEGATE(FOnHoveredWidgetCleared);

	DECLARE_MULTICAST_DELEGATE(FOnSelectedWidgetsChanging)
	DECLARE_MULTICAST_DELEGATE(FOnSelectedWidgetsChanged)

	/** Called after the widget preview has been updated */
	DECLARE_MULTICAST_DELEGATE(FOnWidgetPreviewUpdated)

	/** Called when animation list changes */
	DECLARE_MULTICAST_DELEGATE(FOnWidgetAnimationsUpdated)

	/** Called when animation selection changes */
	DECLARE_MULTICAST_DELEGATE(FOnSelectedAnimationChanged)

	DECLARE_EVENT(FWidgetBlueprintEditor, FOnEnterWidgetDesigner)
	
	/** Called after the widget preview has fully recreated */
	DECLARE_MULTICAST_DELEGATE(FOnWidgetPreviewReady)

public:
	UE_API FWidgetBlueprintEditor();

	UE_API virtual ~FWidgetBlueprintEditor();

	UE_API void InitWidgetBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode);

	//~ Begin FBlueprintEditor interface
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual void PostUndo(bool bSuccessful) override;
	UE_API virtual void PostRedo(bool bSuccessful) override;
	UE_API virtual void Compile() override;
	//~ End FBlueprintEditor interface
	
	//~ Begin FAssetEditorToolkit Interface
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	// End of FAssetEditorToolkit 

	//~ Begin FGCObjectInterface interface
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	//~ End FGCObjectInterface interface

	//~ Begin IToolkit interface
	UE_API virtual FName GetToolkitContextFName() const override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	UE_API void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	UE_API void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	//~ End IToolkit interface

	/** @return The widget blueprint currently being edited in this editor */
	UE_API class UWidgetBlueprint* GetWidgetBlueprintObj() const;

	/** @return The preview widget. */
	UE_API UUserWidget* GetPreview() const;

	/** @return The preview scene that owns the preview widget. */
	UE_API FPreviewScene* GetPreviewScene();

	/**  */
	UE_API bool IsSimulating() const;

	/**  */
	UE_API void SetIsSimulating(bool bSimulating);

	/** Causes the preview to be destroyed and a new one to be created next tick */
	UE_API void InvalidatePreview(bool bViewOnly = false);

	/** Immediately rebuilds the preview widget. */
	UE_API void RefreshPreview();

	/** Creates a widget reference using the template. */
	UE_API FWidgetReference GetReferenceFromTemplate(UWidget* TemplateWidget);

	/** Creates a widget reference using the preview.  Which is used to lookup the stable template pointer. */
	UE_API FWidgetReference GetReferenceFromPreview(UWidget* PreviewWidget);

	/** @return The currently active sequencer used to create widget animations */
	UE_API TSharedPtr<ISequencer>& GetSequencer();

	/** @return The sequencer used to create widget animations */
	UE_API TSharedPtr<ISequencer>& GetTabSequencer();

	/** @return The sequencer used to create widget animations in drawer */
	UE_API TSharedPtr<ISequencer>& GetDrawerSequencer();

	/** Callback to dock anim tab into layout */
	UE_API void DockInLayoutClicked();

	/** Changes the currently viewed animation in Sequencer to the new one*/
	UE_API void ChangeViewedAnimation( UWidgetAnimation& InAnimationToView );

	/** Get the current animation*/
	UWidgetAnimation* GetCurrentAnimation() { return CurrentAnimation.Get(); }

	/** Updates the current animation if it is invalid */
	UE_API const UWidgetAnimation* RefreshCurrentAnimation();

	/** Sets the currently selected set of widgets */
	UE_API void SelectWidgets(const TSet<FWidgetReference>& Widgets, bool bAppendOrToggle);

	/** Sets the currently selected set of objects */
	UE_API void SelectObjects(const TSet<UObject*>& Objects);

	/** Called to determine whether a binding is selected in the tree view */
	UE_API bool IsBindingSelected(const FMovieSceneBinding& InBinding);

	/** Sets the selected named slot */
	UE_API void SetSelectedNamedSlot(TOptional<FNamedSlotSelection> SelectedNamedSlot);

	/** Removes removed widgets from the selection set. */
	UE_API void CleanSelection();

	/** @return The selected set of widgets */
	UE_API const TSet<FWidgetReference>& GetSelectedWidgets() const;

	/** @return The selected named slot */
	UE_API TOptional<FNamedSlotSelection> GetSelectedNamedSlot() const;

	/** @return The selected set of Objects */
	UE_API const TSet< TWeakObjectPtr<UObject> >& GetSelectedObjects() const;

	/** @return the selected template widget */
	const TWeakObjectPtr<UClass> GetSelectedTemplate() const { return SelectedTemplate; }

	/** @return the selected user widget */
	const FAssetData GetSelectedUserWidget() const { return SelectedUserWidget; }

	/** Set the selected template widget */
	void SetSelectedTemplate(TWeakObjectPtr<UClass> TemplateClass) { SelectedTemplate = TemplateClass; }

	/** Set the selected user widget */
	void SetSelectedUserWidget(FAssetData InSelectedUserWidget) { SelectedUserWidget = InSelectedUserWidget; }

	TSharedPtr<class FWidgetBlueprintEditorToolbar> GetWidgetToolbarBuilder() { return WidgetToolbar; }

	/** Migrate a property change from the preview GUI to the template GUI. */
	UE_API void MigrateFromChain(const FPropertyChangedEvent* PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged, bool bIsModify);

	/** Event called when an undo/redo transaction occurs */
	DECLARE_EVENT(FWidgetBlueprintEditor, FOnWidgetBlueprintTransaction)
	FOnWidgetBlueprintTransaction& GetOnWidgetBlueprintTransaction() { return OnWidgetBlueprintTransaction; }

	/** Creates a sequencer widget */
	UE_API TSharedRef<SWidget> CreateSequencerTabWidget();

	/** Creates a sequencer widget, for anim drawer */
	UE_API TSharedRef<SWidget> CreateSequencerDrawerWidget();

	/** Gets sequencer widget, for anim drawer */
	UE_API TSharedRef<SWidget> OnGetWidgetAnimSequencer();

	/** Adds external widget whose lifetime should be managed by this editor */
	UE_API void AddExternalEditorWidget(FName ID, TSharedRef<SWidget> InExternalWidget);

	/** Removes external widget whose lifetime should be managed by this editor */
	UE_API int32 RemoveExternalEditorWidget(FName ID);

	/** Gets external widget whose lifetime should be managed by this editor, NullWidget if not found */
	UE_API TSharedPtr<SWidget> GetExternalEditorWidget(FName ID);

	/** Toggles Anim Drawer, only used for keyboard shortcut */
	UE_API void ToggleAnimDrawer();

	/** Callback for when widget animation list has been updated */
	UE_API void NotifyWidgetAnimListChanged();

	UE_DEPRECATED(5.2, "OnWidgetAnimSequencerOpened is deprecated, please use OnWidgetAnimDrawerSequencerOpened instead.")
	UE_API void OnWidgetAnimSequencerOpened(FName StatusBarWithDrawerName);

	UE_DEPRECATED(5.2, "OnWidgetAnimSequencerDismissed is deprecated, please use OnWidgetAnimDrawerSequencerDismissed instead.")
	UE_API void OnWidgetAnimSequencerDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget);

	/** Callback for anim drawer opening */
	UE_API void OnWidgetAnimDrawerSequencerOpened(FName StatusBarWithDrawerName);

	/** Callback for anim drawer closing */
	UE_API void OnWidgetAnimDrawerSequencerDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget);

	/** Callback for anim tab closing */
	UE_API void OnWidgetAnimTabSequencerClosed(TSharedRef<SDockTab> ClosedTab);

	/** Callback for anim tab opening */
	UE_API void OnWidgetAnimTabSequencerOpened();

	/**
	 * The widget we're now hovering over in any particular context, allows multiple views to 
	 * synchronize feedback on where that widget is in their representation.
	 */
	UE_API void SetHoveredWidget(FWidgetReference& InHoveredWidget);

	UE_API void ClearHoveredWidget();

	/** @return The widget that is currently being hovered over (either in the designer or hierarchy) */
	UE_API const FWidgetReference& GetHoveredWidget() const;

	UE_API void AddPostDesignerLayoutAction(TFunction<void()> Action);

	UE_API void OnEnteringDesigner();

	UE_API TArray< TFunction<void()> >& GetQueuedDesignerActions();

	/** Get the current designer flags that are in effect for the current user widget we're editing. */
	UE_API EWidgetDesignFlags GetCurrentDesignerFlags() const;

	UE_API bool GetShowDashedOutlines() const;
	UE_API void SetShowDashedOutlines(bool Value);

	UE_API bool GetIsRespectingLocks() const;
	UE_API void SetIsRespectingLocks(bool Value);

	TSharedPtr<FPaletteViewModel> GetPaletteViewModel() { return PaletteViewModel; };
	TSharedPtr<FLibraryViewModel> GetLibraryViewModel() { return LibraryViewModel; };

	UE_API void CreateEditorModeManager() override;

	/** Get the relative info for the Debug mode. */
	TSharedPtr<UE::UMG::Editor::FPreviewMode> GetPreviewMode() const { return PreviewMode; }

public:
	/** Fires whenever a new widget is being hovered over */
	FOnHoveredWidgetSet OnHoveredWidgetSet;

	/** Fires when there is no longer any widget being hovered over */
	FOnHoveredWidgetCleared OnHoveredWidgetCleared;	

	/** Fires whenever the selected set of widgets changing */
	FOnSelectedWidgetsChanged OnSelectedWidgetsChanging;

	/** Fires whenever the selected set of widgets changes */
	FOnSelectedWidgetsChanged OnSelectedWidgetsChanged;

	/** Fires whenever the selected animation changes */
	FOnSelectedAnimationChanged OnSelectedAnimationChanged;

	/** Notification for when the preview widget has been updated */
	FOnWidgetPreviewUpdated OnWidgetPreviewUpdated;

	/** Notification for when animations have been updated */
	FOnWidgetAnimationsUpdated OnWidgetAnimationsUpdated;

	/** Notification for when the preview widget has been fully recreated */
	FOnWidgetPreviewReady OnWidgetPreviewReady;
	
	/** Fires after the mode change to Designer*/
	FOnEnterWidgetDesigner OnEnterWidgetDesigner;

	/** Command list for handling widget actions in the WidgetBlueprintEditor */
	TSharedPtr< FUICommandList > DesignerCommandList;

	/** Commands for switching between tool palettes */
	TArray< TSharedPtr<FUICommandInfo>> ToolPaletteCommands;

	/** Paste Metadata */
	FVector2D PasteDropLocation;

protected:
	UE_API virtual void InitalizeExtenders() override;
	UE_API TSharedPtr<FExtender> CreateMenuExtender();
	UE_API void FillFileMenu(FMenuBuilder& MenuBuilder);
	UE_API void FillAssetMenu(FMenuBuilder& MenuBuilder);
	UE_API void BindToolkitCommands();

	UE_API void TakeSnapshot();
	UE_API void CaptureThumbnail();
	UE_API void ClearThumbnail();
	UE_API bool IsImageUsedAsThumbnail();
	UE_API bool IsPreviewWidgetInitialized();

	UE_API void CustomizeWidgetCompileOptions(FMenuBuilder& InMenuBuilder);

	static UE_API void AddCreateCompileTabSubMenu(FMenuBuilder& InMenuBuilder);
	static UE_API void AddDismissCompileTabSubMenu(FMenuBuilder& InMenuBuilder);
	static UE_API void SetDismissOnCompileSetting(EDisplayOnCompile InDismissOnCompile);
	static UE_API void SetCreateOnCompileSetting(EDisplayOnCompile InCreateOnCompile);
	static UE_API bool IsDismissOnCompileSet(EDisplayOnCompile InDismissOnCompile);
	static UE_API bool IsCreateOnCompileSet(EDisplayOnCompile InCreateOnCompile);

	UE_API void OpenCreateNativeBaseClassDialog();
	UE_API void OnCreateNativeBaseClassSuccessfully(const FString& InClassName, const FString& InClassPath, const FString& InModuleName);
	UE_API TSharedPtr<ISequencer> CreateSequencerWidgetInternal();

	//~ Begin FBlueprintEditor interface
	UE_API virtual void RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated = false) override;
	UE_API virtual FGraphAppearanceInfo GetGraphAppearance(class UEdGraph* InGraph) const override;
	UE_API virtual TSubclassOf<UEdGraphSchema> GetDefaultSchemaClass() const override;
	//~ End FBlueprintEditor interface

private:
	UE_API bool CanDeleteSelectedWidgets();
	UE_API void DeleteSelectedWidgets();

	UE_API bool CanCopySelectedWidgets();
	UE_API void CopySelectedWidgets();

	UE_API bool CanPasteWidgets();
	UE_API void PasteWidgets();

	UE_API bool CanCutSelectedWidgets();
	UE_API void CutSelectedWidgets();

	UE_API bool CanDuplicateSelectedWidgets();
	UE_API void DuplicateSelectedWidgets();

	UE_API void OnFindWidgetReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags);
	UE_API bool CanFindWidgetReferences() const;

	/** Is creating a native base class for the current widget blueprint allowed */
	UE_API bool CanCreateNativeBaseClass() const;

	/** Whether menu to create a native base class is visible */
	UE_API bool IsCreateNativeBaseClassVisible() const;

private:
	/** Called whenever the blueprint is structurally changed. */
	UE_API virtual void OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled = false ) override;

	/** Called when objects need to be swapped out for new versions, like after a blueprint recompile. */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Destroy the current preview GUI object */
	UE_API void DestroyPreview();

	/** Tick the current preview GUI object */
	UE_API void UpdatePreview(UBlueprint* InBlueprint, bool bInForceFullUpdate);

	/** Populates the sequencer add menu. */
	UE_API void OnGetAnimationAddMenuContent(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer);

	/** Populates sequencer menu when added with objects. Used to handle case where widget is deleted so it's 
	    Object * is null.*/
	UE_API void OnBuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder,FGuid ObjectBinding);

	/** Populates the sequencer add submenu for the big list of widgets. */
	UE_API void OnGetAnimationAddMenuContentAllWidgets(FMenuBuilder& MenuBuilder);

	/** Adds the supplied UObject to the current animation. */
	UE_API void AddObjectToAnimation(UObject* ObjectToAnimate);

	/** Gets the extender to use for sequencers context sensitive menus and toolbars. */
	UE_API TSharedRef<FExtender> GetAddTrackSequencerExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects);

	/** Extends the sequencer add track menu. */
	UE_API void ExtendSequencerAddTrackMenu( FMenuBuilder& AddTrackMenuBuilder, const TArray<UObject*> ContextObjects );

	/** Binds additional widgets to a track of the same type */
	UE_API void AddWidgetsToTrack(const TArray<FWidgetReference> Widgets, FGuid ObjectId);

	/** Unbind widgets from a track*/
	UE_API void RemoveWidgetsFromTrack(const TArray<FWidgetReference> Widgets, FGuid ObjectId);

	/** Remove all bindings from a track */
	UE_API void RemoveAllWidgetsFromTrack(FGuid ObjectId);

	/** Remove any missing bindings from a track */
	UE_API void RemoveMissingWidgetsFromTrack(FGuid ObjectId);
	
	/** Replace current widget bindings on a track with new widget bindings */
	UE_API void ReplaceTrackWithWidgets(const TArray<FWidgetReference> Widgets, FGuid ObjectId);

	/** Dynamic binding */
	UE_API bool CanAddDynamicPossessionMenu();
	UE_API void AddDynamicPossessionMenu(FMenuBuilder& MenuBuilder, FGuid ObjectId);

	/** Add an animation track for the supplied slot to the current animation. */
	UE_API void AddSlotTrack( UPanelSlot* Slot );

	/** Add an animation track for the supplied slot to the current animation. */
	void AddSlotTrack(TObjectPtr<UPanelSlot> Slot) { return AddSlotTrack(Slot.Get()); }

	/** Add an animation track for the supplied material property path to the current animation. */
	UE_API void AddMaterialTrack( UWidget* Widget, TArray<FProperty*> MaterialPropertyPath, FText MaterialPropertyDisplayName );

	/** Handler which is called whenever sequencer movie scene data changes. */
	UE_API void OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType);

	/** Handler which is called whenever sequencer binding is pasted. */
	UE_API void OnMovieSceneBindingsPasted(const TArray<FMovieSceneBinding>& BindingsPasted);

	/** Fire off when sequencer selection changed */
	UE_API void SyncSelectedWidgetsWithSequencerSelection(TArray<FGuid> ObjectGuids);

	/** Tell sequencer the selected widgets changed */
	UE_API void SyncSequencerSelectionToSelectedWidgets();

	/** Tell sequencers movie data changed */
	UE_API void SyncSequencersMovieSceneData();

	/** Get the animation playback context */
	UObject* GetAnimationPlaybackContext() const { return GetPreview(); }

	/** Get the animation playback event contexts */
	TArray<UObject*> GetAnimationEventContexts() const { TArray<UObject*> EventContexts; EventContexts.Add(GetPreview()); return EventContexts; }
	
	/** Update the name of a track to reflect changes in bindings */
	UE_API void UpdateTrackName(FGuid ObjectId);

	/** Called after changing dynamic binding properties*/
	UE_API void OnFinishedChangingDynamicBindingProperties(const FPropertyChangedEvent& ChangeEvent, TSharedPtr<FStructOnScope> ValueStruct, FGuid ObjectId);

	/** If editing the matching widget blueprint, sets the blueprint's current debug instance to the given object. */
	void OnDebugObjectAccess(UObject* DebugObject);

private:
	/** The preview scene that owns the preview GUI */
	FPreviewScene PreviewScene;

	/** For external widgets to have lifetimes bound to this editor instance */
	TMap<FName, TSharedPtr<SWidget>> ExternalEditorWidgets;

	/** List of created sequencers */
	TArray<TWeakPtr<ISequencer>> Sequencers;

	/** Sequencer for creating and previewing widget animations in tabs */
	TSharedPtr<ISequencer> TabSequencer;

	/** Overlay used to display UI on top of tab sequencer */
	TWeakPtr<SOverlay> TabSequencerOverlay;

	/** Sequencer for creating and previewing widget animations in drawer */
	TSharedPtr<ISequencer> DrawerSequencer;

	/** Overlay used to display UI on top of drawer sequencer */
	TWeakPtr<SOverlay> DrawerSequencerOverlay;

	/** Widget created for anim drawer */
	TSharedPtr<SWidget> AnimDrawerWidget;

	/** A text block which is displayed in the overlay when no animation is selected. */
	TWeakPtr<STextBlock> NoAnimationTextBlockTab;

	/** A text block which is displayed in the overlay when no animation is selected in drawer. */
	TWeakPtr<STextBlock> NoAnimationTextBlockDrawer;

	/** The Blueprint associated with the current preview */
	UWidgetBlueprint* PreviewBlueprint;

	/** The currently selected preview widgets in the preview GUI */
	TSet< FWidgetReference > SelectedWidgets;

	/** The currently selected objects in the designer */
	TSet< TWeakObjectPtr<UObject> > SelectedObjects;

	/** The last selected template widget in the palette view */
	TWeakObjectPtr<UClass> SelectedTemplate;

	/** AssetData of Selected UserWidget */
	FAssetData SelectedUserWidget;

	/** The currently selected named slot */
	TOptional<FNamedSlotSelection> SelectedNamedSlot;

	/** The preview GUI object */
	mutable TWeakObjectPtr<UUserWidget> PreviewWidgetPtr;

	/** Delegate called when a undo/redo transaction happens */
	FOnWidgetBlueprintTransaction OnWidgetBlueprintTransaction;

	/** The toolbar builder associated with this editor */
	TSharedPtr<class FWidgetBlueprintEditorToolbar> WidgetToolbar;

	/** Used to spawn sidebar tool palette */
	TSharedPtr<class FWidgetEditorModeUILayer> ModeUILayer;

	/** The widget references out in the ether that may need to be updated after being issued. */
	TArray< TWeakPtr<FWidgetHandle> > WidgetHandlePool;

	/** The widget currently being hovered over */
	FWidgetReference HoveredWidget;

	/** The preview becomes invalid and needs to be rebuilt on the next tick. */
	bool bPreviewInvalidated;

	/**  */
	bool bIsSimulateEnabled;

	/**  */
	bool bIsRealTime;

	/** Should the designer show outlines when it creates widgets? */
	bool bShowDashedOutlines;

	/**  */
	bool bRespectLocks;

	TArray< TFunction<void()> > QueuedDesignerActions;

	/** The currently viewed animation, if any. */
	TWeakObjectPtr<UWidgetAnimation> CurrentAnimation;

	FDelegateHandle SequencerAddTrackExtenderHandle;

	/** True if sequencer drawer is open */
	bool bIsSequencerDrawerOpen;

	/** When true the animation data in the generated class should be replaced with the current animation data. */
	bool bRefreshGeneratedClassAnimations;

	/** ViewModel used by the Palette and Palette Favorite Views */
	TSharedPtr<FPaletteViewModel> PaletteViewModel;

	/** ViewModel used by the Library View */
	TSharedPtr<FLibraryViewModel> LibraryViewModel;

	TSharedPtr<UE::UMG::Editor::FPreviewMode> PreviewMode;

	/** When true the sequencer selection is being updated from changes to the external selection. */
	bool bUpdatingSequencerSelection;

	/** When true the external selection is being updated from changes to the sequencer selection. */
	bool bUpdatingExternalSelection;

	/** True when the selected object is the preview widget. Used to restore selection after refreshing the preview.*/
	bool bIsPreviewWidgetSelected;
};

#undef UE_API
