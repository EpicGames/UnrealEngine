// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprint.h"
#include "WidgetReference.h"
#include "Misc/StringOutputDevice.h"

#define UE_API UMGEDITOR_API

class FDragDropOperation;
class FHittestGrid;
class FMenuBuilder;
class UWidgetBlueprint;
class UWidgetSlotPair;
class UWidgetTree;
class SWindow;
class UWidgetEditingProjectSettings;
class FWidgetObjectTextFactory;
class ULocalPlayer;
class FWidgetTemplateClass;

//////////////////////////////////////////////////////////////////////////
// FWidgetBlueprintEditorUtils

class FWidgetBlueprintEditorUtils
{
	friend class FWidgetBlueprintOperationUtils;

public:
	struct FWidgetThumbnailProperties
	{
		FVector2D ScaledSize;
		FVector2D Offset;
	};

	struct FCreateWidgetFromBlueprintParams
	{
		EWidgetDesignFlags FlagsToApply;
		ULocalPlayer* LocalPlayer = nullptr; // Optionally specify if available
	};

	static UE_API bool VerifyWidgetRename(TSharedRef<class FWidgetBlueprintEditor> BlueprintEditor, FWidgetReference Widget, const FText& NewName, FText& OutErrorMessage);

	static UE_API bool RenameWidget(TSharedRef<class FWidgetBlueprintEditor> BlueprintEditor, const FName& OldObjectName, const FString& NewDisplayName);

	static UE_API void ReplaceDesiredFocus(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, const FName& OldName, const FName& NewName);
	static UE_API void ReplaceDesiredFocus(UWidgetBlueprint* Blueprint, const FName& OldName, const FName& NewName);
	
	static UE_API void SetDesiredFocus(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, const FName DesiredFocusWidgetName);
	static UE_API void SetDesiredFocus(UWidgetBlueprint* Blueprint, const FName& DesiredFocusWidgetName);

	static UE_API void CreateWidgetContextMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, FVector2D TargetLocation);

	static UE_API void CopyWidgets(UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API TArray<UWidget*> PasteWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference ParentWidget, FName SlotName, FVector2D PasteLocation);

	enum class EReplaceWidgetNamingMethod
	{
		// Will give the new widget the same name as the replaced widget if the widget classes are compatible
		// If it's using a generated name, the new generated name will be used and FBlueprintEditorUtils::ReplaceVariableReferences will be called
		MaintainNameAndReferences,

		// Same as MaintainNameAndReferences but doesn't check for matching classes or generated names
		MaintainNameAndReferencesForUnmatchingClass,

		// Will use the new widget's generated name and not make an effort to maintain references
		UseNewGeneratedName
	};
	static UE_API void ReplaceWidgets(UWidgetBlueprint* BP, TSet<UWidget*> Widgets, UClass* WidgetClass, EReplaceWidgetNamingMethod NewWidgetNamingMethod);

	UE_DEPRECATED(5.6, "FWidgetBlueprintEditorUtils::DeleteWidgets no longer takes in the BlueprintEditor, use the version without it instead")
	static UE_API void DeleteWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets, bool bSilentDelete = false);

	enum class EDeleteWidgetWarningType
	{
		// If the widget being deleted is referenced in the graph, ask the user if the deletion should continue
		WarnAndAskUser,
		// Don't notify the user at all and delete the widget even if it is referenced
		DeleteSilently,
	};
	static UE_API void DeleteWidgets(UWidgetBlueprint* BP, TSet<UWidget*> Widgets, EDeleteWidgetWarningType WarningType);

	static UE_API void CutWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API TArray<UWidget*> DuplicateWidgets(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API UUserWidget* CreateUserWidgetFromBlueprint(UObject* Outer, UWidgetBlueprint* BP, const FCreateWidgetFromBlueprintParams& Params);

	/** Performs cleanup on the specified UserWidget. */
	static UE_API void DestroyUserWidget(UUserWidget* UserWidget);

	static UE_API bool IsAnySelectedWidgetLocked(TSet<FWidgetReference> SelectedWidgets);

	static UE_API bool CanPasteWidgetsExtension(TSet<FWidgetReference> SelectedWidgets);

	static UE_API UWidget* GetWidgetTemplateFromDragDrop(UWidgetBlueprint* Blueprint, UWidgetTree* RootWidgetTree, TSharedPtr<FDragDropOperation>& DragDropOp);

	static UE_API bool ShouldPreventDropOnTargetExtensions(const UWidget* Target, const TSharedPtr<FDragDropOperation>& DragDropOp, FText& OutFailureText);

	static UE_API bool IsBindWidgetProperty(const FProperty* InProperty);
	static UE_API bool IsBindWidgetProperty(const FProperty* InProperty, bool& bIsOptional);

	static UE_API bool IsBindWidgetAnimProperty(const FProperty* InProperty);
	static UE_API bool IsBindWidgetAnimProperty(const FProperty* InProperty, bool& bIsOptional);

	struct FUsableWidgetClassResult
	{
		const UClass* NativeParentClass = nullptr;
		EClassFlags AssetClassFlags = EClassFlags::CLASS_None;
	};

	UE_DEPRECATED(5.3, "This function has been deprecated. Use the version of IsUsableWidgetClass that takes a second argument of TSharedRef<FWidgetBlueprintEditor>.")
	static UE_API bool IsUsableWidgetClass(const UClass* WidgetClass);
	UE_DEPRECATED(5.3, "This function has been deprecated. Use the version of IsUsableWidgetClass that takes a second argument of TSharedRef<FWidgetBlueprintEditor>.")
	static UE_API TValueOrError<FUsableWidgetClassResult, void> IsUsableWidgetClass(const FAssetData& WidgetAsset);

	static UE_API bool IsUsableWidgetClass(const UClass* WidgetClass, TSharedRef<FWidgetBlueprintEditor> InCurrentActiveBlueprintEditor);
	static UE_API TValueOrError<FUsableWidgetClassResult, void> IsUsableWidgetClass(const FAssetData& WidgetAsset, TSharedRef<FWidgetBlueprintEditor> InCurrentActiveBlueprintEditor);

	static UE_API void ExportWidgetsToText(TArray<UWidget*> WidgetsToExport, /*out*/ FString& ExportedText);

	static UE_API void ImportWidgetsFromText(UWidgetBlueprint* BP, const FString& TextToImport, /*out*/ TSet<UWidget*>& ImportedWidgetSet, /*out*/ TMap<FName, UWidgetSlotPair*>& PastedExtraSlotData);

	/** Exports the individual properties of an object to text and stores them in a map. */
	static UE_API void ExportPropertiesToText(UObject* Object, TMap<FName, FString>& ExportedProperties);

	/** Attempts to import any property in the map and apply it to a property with the same name on the object. */
	static UE_API void ImportPropertiesFromText(UObject* Object, const TMap<FName, FString>& ExportedProperties);

	static UE_API bool DoesClipboardTextContainWidget(UWidgetBlueprint* BP);

	static UE_API TScriptInterface<INamedSlotInterface> FindNamedSlotHostForContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree);

	static UE_API UWidget* FindNamedSlotHostWidgetForContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree);

	static UE_API void FindAllAncestorNamedSlotHostWidgetsForContent(TArray<FWidgetReference>& OutSlotHostWidgets, UWidget* WidgetTemplate, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor);

	static UE_API bool RemoveNamedSlotHostContent(UWidget* WidgetTemplate, TScriptInterface<INamedSlotInterface> NamedSlotHost);

	static UE_API bool ReplaceNamedSlotHostContent(UWidget* WidgetTemplate, TScriptInterface<INamedSlotInterface> NamedSlotHost, UWidget* NewContentWidget);

	static UE_API UWidgetTree* FindLatestWidgetTree(UWidgetBlueprint* Blueprint, UUserWidget* UserWidget);

	static UE_API int32 UpdateHittestGrid(FHittestGrid& HitTestGrid, TSharedRef<SWindow> Window, float Scale, FVector2D DrawSize, float DeltaTime);

	static UE_API TTuple<FVector2D, FVector2D> GetWidgetPreviewAreaAndSize(UUserWidget* UserWidget, FVector2D DesiredSize, FVector2D PreviewSize, EDesignPreviewSizeMode ThumbnailSizeMode, TOptional<FVector2D> ThumbnailCustomSize);

	static UE_API float GetWidgetPreviewDPIScale(UUserWidget* UserWidget, FVector2D PreviewSize);

	static UE_API EDesignPreviewSizeMode ConvertThumbnailSizeModeToDesignerSizeMode(EThumbnailPreviewSizeMode ThumbnailSizeMode, UUserWidget* WidgetInstance);

	static UE_API FVector2D GetWidgetPreviewUnScaledCustomSize(FVector2D DesiredSize, UUserWidget* UserWidget, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode = EThumbnailPreviewSizeMode::MatchDesignerMode);

	static UE_API TTuple<float, FVector2D> GetThumbnailImageScaleAndOffset(FVector2D WidgetSize, FVector2D ThumbnailSize);

	static UE_API void SetTextureAsAssetThumbnail(UWidgetBlueprint* WidgetBlueprint, UTexture2D* ThumbnailTexture);

	static UE_API FText GetPaletteCategory(const TSubclassOf<UWidget> Widget);
	static UE_API FText GetPaletteCategory(const FAssetData& WidgetAsset, const TSubclassOf<UWidget> NativeClass);

	static UE_API TOptional<FWidgetThumbnailProperties> DrawSWidgetInRenderTargetForThumbnail(UUserWidget* WidgetInstance, FRenderTarget* RenderTarget2D, FVector2D ThumbnailSize, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode = EThumbnailPreviewSizeMode::MatchDesignerMode);

	static UE_API TOptional<FWidgetThumbnailProperties> DrawSWidgetInRenderTargetForThumbnail(UUserWidget* WidgetInstance, UTextureRenderTarget2D* RenderTarget2D, FVector2D ThumbnailSize, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode);

	static UE_API TOptional<FWidgetThumbnailProperties> DrawSWidgetInRenderTarget(UUserWidget* WidgetInstance, UTextureRenderTarget2D* RenderTarget2D);

	static UE_API UWidgetEditingProjectSettings* GetRelevantMutableSettings(TWeakPtr<FWidgetBlueprintEditor> CurrentEditor);
	static UE_API const UWidgetEditingProjectSettings* GetRelevantSettings(TWeakPtr<FWidgetBlueprintEditor> CurrentEditor);

	static UE_API UWidgetBlueprint* GetWidgetBlueprintFromWidget(const UWidget* Widget);
	
	static UE_API TSet<UWidget*> ResolveWidgetTemplates(const TSet<FWidgetReference>& Widgets);

private:

	static UE_API FString CopyWidgetsInternal(UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API TArray<UWidget*> PasteWidgetsInternal(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, const FString& TextToImport, FWidgetReference ParentWidget, FName SlotName, FVector2D PasteLocation, bool bForceSibling, bool& TransactionSuccesful);

	static UE_API bool DisplayPasteWarningAndEarlyExit();

	static UE_API void ExecuteOpenSelectedWidgetsForEdit( TSet<FWidgetReference> SelectedWidgets );

	static UE_API bool FindAndRemoveNamedSlotContent(UWidget* WidgetTemplate, UWidgetTree* WidgetTree);

	static UE_API bool CanOpenSelectedWidgetsForEdit( TSet<FWidgetReference> SelectedWidgets );

	static UE_API void BuildWrapWithMenu(FMenuBuilder& Menu, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API void BuildReplaceWithMenu(FMenuBuilder& Menu, TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, TSet<FWidgetReference> Widgets);

	static UE_API void ReplaceWidgetWithSelectedTemplate(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference Widget);

	static UE_API bool CanBeReplacedWithTemplate(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidgetBlueprint* BP, FWidgetReference Widget);

	static UE_API FString FindNextValidName(UWidgetTree* WidgetTree, const FString& Name);

	static UE_API void FindUsedVariablesForWidgets(const TSet<UWidget*>& Widgets, const UWidgetBlueprint* BP, TArray<UWidget*>& UsedVariables, TArray<FText>& WidgetNames, bool bIncludeVariablesOnChildren);

	static UE_API bool ShouldContinueDeleteOperation(UWidgetBlueprint* BP, const TArray<FText>& WidgetNames);

	static UE_API bool ShouldContinueReplaceOperation(UWidgetBlueprint* BP, const TArray<FText>& WidgetNames);	

	static UE_API TOptional<FWidgetThumbnailProperties> DrawSWidgetInRenderTargetInternal(UUserWidget* WidgetInstance, FRenderTarget* RenderTarget2D, UTextureRenderTarget2D* TextureRenderTarget,FVector2D ThumbnailSize, bool bIsForThumbnail, TOptional<FVector2D> ThumbnailCustomSize, EThumbnailPreviewSizeMode ThumbnailSizeMode);
	
	static UE_API bool IsDesiredFocusWidget(TSharedRef<FWidgetBlueprintEditor> BlueprintEditor, UWidget* Widget);
	static UE_API bool IsDesiredFocusWidget(UWidgetBlueprint* Blueprint, UWidget* Widget);

	static UE_API FWidgetObjectTextFactory ProcessImportedText(UWidgetBlueprint* BP, const FString& TextToImport, /*out*/ UPackage*& TempPackage);
};

#undef UE_API
