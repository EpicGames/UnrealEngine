// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Blueprint/BlueprintExtension.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintCompiler.h"

class FSubobjectCollection;
class UWidgetBlueprintGeneratedClass;

#include "WidgetBlueprintExtension.generated.h"

#define UE_API UMGEDITOR_API

/** Extension that allows per-system data to be held on the widget blueprint, and per-system logic to be executed during compilation */
UCLASS(MinimalAPI, Within=WidgetBlueprint)
class UWidgetBlueprintExtension : public UBlueprintExtension
{
	GENERATED_BODY()

public:
	/**
	 * Request an WidgetBlueprintExtension for an WidgetBlueprint.
	 * @note It is illegal to perform this operation once compilation has commenced, use GetExtension instead.
	 */
	template<typename ExtensionType>
	static ExtensionType* RequestExtension(UWidgetBlueprint* InWidgetBlueprint)
	{
		return CastChecked<ExtensionType>(RequestExtension(InWidgetBlueprint, ExtensionType::StaticClass()));
	}

	/**
	 * Request an WidgetBlueprintExtension for an WidgetBlueprint.
	 * @note It is illegal to perform this operation once compilation has commenced, use GetExtension instead.
	 */
	static UE_API UWidgetBlueprintExtension* RequestExtension(UWidgetBlueprint* InWidgetBlueprint, TSubclassOf<UWidgetBlueprintExtension> InExtensionType);

	/** Get an already-requested extension for an WidgetBlueprint. */
	template<typename ExtensionType>
    static ExtensionType* GetExtension(const UWidgetBlueprint* InWidgetBlueprint)
	{
		return Cast<ExtensionType>(GetExtension(InWidgetBlueprint, ExtensionType::StaticClass()));
	}

	/** Get an already-requested extension for an WidgetBlueprint. */
	static UE_API UWidgetBlueprintExtension* GetExtension(const UWidgetBlueprint* InWidgetBlueprint, TSubclassOf<UWidgetBlueprintExtension> InExtensionType);

	/** Get all subsystems currently present on an WidgetBlueprint */
	static UE_API TArray<UWidgetBlueprintExtension*> GetExtensions(const UWidgetBlueprint* InWidgetBlueprint);

	/** Iterate over all registered WidgetBlueprintExtensions in an WidgetBlueprint */
	template<typename Predicate>
	static void ForEachExtension(const UWidgetBlueprint* InWidgetBlueprint, Predicate Pred)
	{
		for (const TObjectPtr<UBlueprintExtension>& BlueprintExtension : InWidgetBlueprint->GetExtensions())
		{
			if (UWidgetBlueprintExtension* WidgetBlueprintExtension = Cast<UWidgetBlueprintExtension>(BlueprintExtension))
			{
				Pred(WidgetBlueprintExtension);
			}
		}
	}

	/** Get the WidgetBlueprint that hosts this extension */
	UE_API UWidgetBlueprint* GetWidgetBlueprint() const;

#if WITH_EDITOR
	virtual void FindDiffs(const UWidgetBlueprint* OwningBlueprint, const UWidgetBlueprint* OtherBlueprint, FDiffResults& Results) const {}

	/**
	 * Report references that this extension is holding into a specific widget's class members.
	 *
	 * @param WidgetVariableName       Variable name of the widget on the owning blueprint.
	 * @param WidgetClass              Current class of the widget (use to resolve member kind).
	 * @param OutReferencedProperties  Property/delegate names on WidgetClass that this extension references.
	 * @param OutReferencedFunctions   Function names on WidgetClass that this extension references.
	 */
	virtual void GatherWidgetMemberReferences(FName WidgetVariableName, const UClass* WidgetClass, TSet<FName>& OutReferencedProperties, TSet<FName>& OutReferencedFunctions) const {}
#endif //WITH_EDITOR

protected:
	/** 
	 * Override point called when a compiler context is created for the WidgetBlueprint
	 * @param	InCreationContext	The compiler context for the current compilation
	 */
	virtual void HandleBeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext) {}

	virtual void HandleCreateFunctionList(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& InCreationContext) {}
	virtual void HandleCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO) {}
	virtual TArray<UObject*> HandleSaveSubObjectsFromCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean) { return TArray<UObject*>();  }
	virtual void HandlePopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context) {}
	virtual void HandleCreateClassVariablesFromBlueprint(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context) {}
	virtual void HandleCopyTermDefaultsToDefaultObject(UObject* DefaultObject) {}
	virtual void HandleFinishCompilingClass(UWidgetBlueprintGeneratedClass* Class) {}
	virtual bool HandleValidateGeneratedClass(UWidgetBlueprintGeneratedClass* Class) { return true; }

	/**
	 * Override point called when a compiler context is destroyed for the WidgetBlueprint.
	 * Can be used to clean up resources.
	 */
	virtual void HandleEndCompilation() {}


private:
	friend FWidgetBlueprintCompilerContext;

	void BeginCompilation(FWidgetBlueprintCompilerContext& InCreationContext)
	{
		HandleBeginCompilation(InCreationContext);
	}

	void CreateFunctionList(const FWidgetBlueprintCompilerContext::FCreateFunctionContext& InCreationContext)
	{
		HandleCreateFunctionList(InCreationContext);
	}

	void CleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean, UObject* OldCDO)
	{
		HandleCleanAndSanitizeClass(ClassToClean, OldCDO);
	}

	TArray<UObject*> SaveSubObjectsFromCleanAndSanitizeClass(UWidgetBlueprintGeneratedClass* ClassToClean)
	{
		return HandleSaveSubObjectsFromCleanAndSanitizeClass(ClassToClean);
	}

	void PopulateGeneratedVariables(const FWidgetBlueprintCompilerContext::FPopulateGeneratedVariablesContext& Context)
	{
		HandlePopulateGeneratedVariables(Context);
	}

	void CreateClassVariablesFromBlueprint(const FWidgetBlueprintCompilerContext::FCreateVariableContext& Context)
	{
		HandleCreateClassVariablesFromBlueprint(Context);
	}

	void CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
	{
		HandleCopyTermDefaultsToDefaultObject(DefaultObject);
	}

	void FinishCompilingClass(UWidgetBlueprintGeneratedClass* Class)
	{
		HandleFinishCompilingClass(Class);
	}

	bool ValidateGeneratedClass(UWidgetBlueprintGeneratedClass* Class)
	{
		return HandleValidateGeneratedClass(Class);
	}

	void EndCompilation()
	{
		HandleEndCompilation();
	}
};

#undef UE_API
