// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blueprint/UserWidget.h"
#include "Components/Widget.h"
#include "Engine/Blueprint.h"
#include "WidgetBlueprintEditorUtils.h"

#define UE_API UMGEDITOR_API

class UWidgetBlueprint;
class UPanelWidget;
class FWidgetBlueprintEditor;

/**
 * Utility functions for programmatic Widget Blueprint manipulation.
 *
 */
class FWidgetBlueprintOperationUtils
{
	friend class FWidgetBlueprintEditorUtils;

public:

	/**
	 * Lower-level overload that creates a Widget Blueprint directly into InParent with the given Name.
	 * Intended for asset factories that have already resolved the package and name.
	 *
	 * @param InParent            Package or outer object to create the blueprint in.
	 * @param Name                Name for the blueprint UObject.
	 * @param BlueprintType       Type of blueprint (typically BPTYPE_Normal).
	 * @param ParentClass         The parent UUserWidget class.
	 * @param RootWidgetClass     Optional root panel widget class. Pass null to leave the root unset.
	 * @param CallingContext      Opaque context name passed to FKismetEditorUtilities (used for analytics).
	 * @param bRegisterAndCompile When true (default), marks the package dirty, registers missing GUIDs,
	 *                            compiles the blueprint, and notifies the asset registry. Pass false when
	 *                            the calling framework (e.g. UFactory) already handles these steps.
	 */
	static UE_API UWidgetBlueprint* CreateWidgetBlueprint(UObject* InParent, FName Name, EBlueprintType BlueprintType, TSubclassOf<UUserWidget> ParentClass, TSubclassOf<UWidget> RootWidgetClass = nullptr, FName CallingContext = NAME_None, bool bRegisterAndCompile = true);

	/**
	 * Adds a widget to the blueprint's widget tree.
	 * - If ParentWidget is null and the tree has no root, sets the new widget as the root.
	 * - Otherwise adds to ParentWidget at ChildIndex.
	 * Returns true on success, false on failure (OutErrorMessage will contain details).
	 */
	static UE_API bool AddWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* NewWidget, UWidget* ParentWidget, int32 ChildIndex, FText& OutErrorMessage);

	/** Returns true if a widget can be added to the specified parent in the given blueprint. */
	static UE_API bool CanAddToParent(UWidgetBlueprint* WidgetBlueprint, UWidget* ParentWidget, FText& OutErrorMessage);

	/** Creates a widget instance based on the class of the asset. */
	static UE_API UWidget* CreateWidgetFromAsset(UWidgetBlueprint* WidgetBlueprint, const FAssetData& AssetData, UWidgetTree* RootWidgetTree, FText& OutErrorMessage);

	/** Returns true if making ChildWidget a child of ParentWidget would not create a cycle in the widget hierarchy. */
	static UE_API bool IsParentChildCycleFree(UWidgetBlueprint* WidgetBlueprint, UWidget* ChildWidget, UWidget* ParentWidget);

	/**
	 * Moves a widget to a new parent panel at the specified child index (-1 appends).
	 * Returns true on success.
	 */
	static UE_API bool MoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, UPanelWidget* NewParent, int32 ChildIndex, FText& OutErrorMessage);

	/**
	 * Removes a widget and all of its children from the blueprint's widget tree.
	 * Returns true if the widget was found and removed.
	 */
	static UE_API bool RemoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, FText& OutErrorMessage);

	/**
	 * Renames a widget in the blueprint.
	 * Returns true on success.
	 */
	static UE_API bool RenameWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FString& NewDisplayName);

	/** Validates that a widget can be renamed to NewName. Returns true if the rename is valid; fills OutErrorMessage on failure. */
	static UE_API bool VerifyWidgetRename(UWidgetBlueprint* WidgetBlueprint, UWidget* TemplateWidget, const FText& NewName, FText& OutErrorMessage);

	/** Sets the bIsVariable flag on a widget. */
	static UE_API void ToggleWidgetAsVariable(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, bool bIsVariable, bool bMarkBlueprintModified = true);

	/**
	 * Adds a Blueprint event handler graph node bound to a widget's multicast delegate event,
	 * equivalent to clicking an event button on a widget variable in the Details panel.
	 */
	static UE_API bool BindToEventProperty(UWidgetBlueprint* WidgetBlueprint, FName EventName, FName PropertyName, UClass* PropertyClass, bool bShouldJumpToNode, FText& OutErrorMessage);

	/**
	 * Replaces a widget instance in the blueprint's widget tree with a new instance created from a
	 * different template widget class. Performs hard validations (params, widget identity in the
	 * tree, circular blueprint reference, BindWidget contract on the parent class chain, panel
	 * rules and child-capacity for panel-to-panel swaps) and aborts without mutating on failure.
	 * On success, references to compatible members on the new class are retargeted; references to
	 * members that don't exist on the new class are left pointing at the old class so the user
	 * sees them as orphaned in the graph.
	 *
	 * Panel handling: if WidgetToReplace is a panel with children, TemplateClass must also be a
	 * panel widget that can hold every existing child. Slot properties are preserved across the
	 * move where the new slot accepts them.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the widget to replace.
	 * @param WidgetToReplace    The widget instance to replace (must be in WidgetBlueprint's tree).
	 * @param TemplateClass      The widget class to create the replacement from.
	 * @param OutErrorMessage    Filled on failure with a description of what went wrong.
	 * @return true on success, false on hard failure (no side effects on failure).
	 */
	static UE_API bool ReplaceWidgetWithTemplate(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, TSubclassOf<UWidget> TemplateClass, FText& OutErrorMessage);

	/**
	 * Removes a widget instance that was freshly added to the tree. Use RemoveWidget for any delete operations after a widget has been added.
	 * RemoveTransientWidgetFromTree assumes the widget has no references. RemoveWidget is a complete version that cleans all references.
	 */
	static UE_API void RemoveTransientWidgetFromTree(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget);

	/**
	 * Adds a UI component of the given class to the named widget.
	 * Returns the UWidget the component was added to.
	 */
	static UE_API UWidget* AddUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClass, FName WidgetName, FText& OutErrorMessage);

	/**
	 * Removes a UI component of the given class from the named widget.
	 * Returns true on success.
	 */
	static UE_API bool RemoveUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClass, FName WidgetName, FText& OutErrorMessage);

	/**
	 * Moves a UI component before or after another component on the same widget.
	 * Returns true on success.
	 */
	static UE_API bool MoveUIComponent(UWidgetBlueprint* WidgetBlueprint, UClass* ComponentClassToMove, UClass* RelativeToComponentClass, FName WidgetName, bool bMoveAfter, FText& OutErrorMessage);

	/**
	 * Returns true if Class declares (or inherits) a property with Property's name and a compatible
	 * type. On failure OutError is populated with the reason ("no member of that name on the new
	 * class" vs "name exists but the type is incompatible"). On success OutError is left untouched.
	 * Used both internally by FixupWidgetBlueprintReferences and by higher layers (e.g. UMG toolset)
	 * that need to audit member compatibility ahead of a widget replacement.
	 */
	static UE_API bool DoesPropertyExistInClass(UClass* Class, FProperty* Property, FText& OutError);

	/**
	 * Returns true if Class declares (or inherits) a function with Function's name and a signature
	 * compatible with Function. On failure OutError is populated with the reason ("no function of
	 * that name on the new class" vs "name exists but the signature is incompatible"). On success
	 * OutError is left untouched.
	 */
	static UE_API bool DoesFunctionExistInClass(UClass* Class, UFunction* Function, FText& OutError);

	/**
	 * Wraps selected widgets in a new parent widget of the specified class.
	 * Returns the newly created wrapper widget(s).
	 */
	static UE_API TArray<UWidget*> WrapWidgets(UWidgetBlueprint* BP, TArray<UWidget*> Widgets, UClass* WidgetClass);

	/**
	 * Replaces a single-child panel widget with its child, preserving named-slot host wiring and
	 * parent linkage. The widget to replace must be a UPanelWidget with exactly one child.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the panel widget.
	 * @param WidgetToReplace    The panel widget (must be a UPanelWidget with exactly one child and in WidgetBlueprint's tree).
	 * @param OutErrorMessage    Filled on failure with a description of what went wrong.
	 * @return true on success, false if validation failed (no side effects on failure).
	 */
	static UE_API bool ReplaceWidgetWithChild(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FText& OutErrorMessage);

	/**
	 * Replaces a host widget with the content of one of its named slots, preserving parent
	 * linkage and root-widget wiring. Validates inputs and aborts without side effects on failure.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the host widget.
	 * @param WidgetToReplace    The host widget (must implement INamedSlotInterface and be in WidgetBlueprint's tree).
	 * @param NamedSlot          The slot whose content will replace WidgetToReplace.
	 * @param OutErrorMessage    Filled on failure with a description of what went wrong.
	 * @param DeleteWarningType  Controls whether the user is prompted when the replaced host
	 *                           is referenced as a Blueprint variable. Defaults to DeleteSilently
	 *                           (suitable for headless / toolset callers); the editor menu path
	 *                           passes WarnAndAskUser.
	 * @return true on success, false if validation failed (no side effects on failure).
	 */
	static UE_API bool ReplaceWidgetWithNamedSlot(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FName NamedSlot, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType DeleteWarningType, FText& OutErrorMessage);

	// ---- Widget Tree Traversal ----

	/**
	 * Depth-first walk of a widget tree. Calls Visitor once per widget in order.
	 * Handles panel children and named slot content. Cycle-safe.
	 *
	 * @param WidgetBlueprint  Blueprint to traverse.
	 * @param StartWidget      nullptr = walk from root widget.
	 * @param MaxDepth         -1 = no limit; 0 = StartWidget only; N = N levels of descent.
	 * @param Visitor          Called per widget: (Widget, Depth, SlotName).
	 *                         Depth is 0 at StartWidget. SlotName is NAME_None for panel
	 *                         children; the named slot name for named slot content.
	 */
	static UE_API void WalkWidgetTree(
		const UWidgetBlueprint* WidgetBlueprint,
		const UWidget* StartWidget,
		int32 MaxDepth,
		TFunctionRef<void(const UWidget* Widget, int32 Depth, FName SlotName)> Visitor);

	/**
	 * Returns the maximum depth of the widget tree from StartWidget.
	 * A leaf = 0; a node with one level of children = 1.
	 * Returns -1 if WidgetBlueprint or its WidgetTree is null.
	 */
	static UE_API int32 ComputeWidgetTreeDepth(
		const UWidgetBlueprint* WidgetBlueprint,
		const UWidget* StartWidget = nullptr);

private:
	static void FixupWidgetBlueprintReferences(UWidgetBlueprint* WidgetBlueprint, UClass* OldClass, UClass* NewWidgetClass, FName VariableName);

	static bool ReplaceWidgetsWithTemplateClass(UWidgetBlueprint* BP, TSet<UWidget*> Widgets, TSharedPtr<FWidgetTemplateClass> TemplateClass, FWidgetBlueprintEditorUtils::EReplaceWidgetNamingMethod NewWidgetNamingMethod);
};

#undef UE_API
