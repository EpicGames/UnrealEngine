// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "WidgetBlueprintThumbnailRenderer.generated.h"

#define UE_API UMGEDITOR_API

class FCanvas;
class FRenderTarget;
class UTextureRenderTarget2D;
class FWidgetBlueprintThumbnailPool;

UCLASS(MinimalAPI)
class UWidgetBlueprintThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	UE_API UWidgetBlueprintThumbnailRenderer();
	UE_API virtual ~UWidgetBlueprintThumbnailRenderer();

	//~ Begin UThumbnailRenderer Object
	UE_API virtual bool CanVisualizeAsset(UObject* Object) override;
	UE_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	UE_API virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	//~ End UThumbnailRenderer Object

private:
	UE_API void OnBlueprintUnloaded(UBlueprint* Blueprint);

private:
	struct FWidgetBlueprintThumbnailPoolDeleter
	{
		void operator()(FWidgetBlueprintThumbnailPool* Pointer);
	};

	TUniquePtr<FWidgetBlueprintThumbnailPool, FWidgetBlueprintThumbnailPoolDeleter> ThumbnailPool;
};

#undef UE_API
