// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WidgetBlueprintToolMenuContext.generated.h"

class FWidgetBlueprintEditor;

UCLASS(MinimalAPI)
class UWidgetBlueprintToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor;
};
