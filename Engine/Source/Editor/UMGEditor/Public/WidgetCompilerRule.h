// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "WidgetCompilerRule.generated.h"

#define UE_API UMGEDITOR_API

class FCompilerResultsLog;
class UWidgetBlueprint;

/**
 * 
 */
UCLASS(MinimalAPI, Abstract)
class UWidgetCompilerRule : public UObject
{
	GENERATED_BODY()
public:
	UE_API UWidgetCompilerRule();

	UE_API virtual void ExecuteRule(UWidgetBlueprint* WidgetBlueprint, FCompilerResultsLog& MessageLog);
};

#undef UE_API
