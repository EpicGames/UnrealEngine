// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "Misc/NamePermissionList.h"
#include "WidgetEditingProjectSettings.generated.h"

#define UE_API UMGEDITOR_API

class UWidgetCompilerRule;
class UUserWidget;
class UWidgetBlueprint;
class UPanelWidget;

USTRUCT()
struct FDebugResolution
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Resolution)
	int32 Width = 0;

	UPROPERTY(EditAnywhere, Category = Resolution)
	int32 Height = 0;

	UPROPERTY(EditAnywhere, Category=Resolution)
	FString Description;

	UPROPERTY(EditAnywhere, Category = Resolution)
	FLinearColor Color = FLinearColor::Black;
};

/** Controls the level of support you want to have for widget property binding. */
UENUM()
enum class EPropertyBindingPermissionLevel : uint8
{
	/** Freely allow the use of property binding. */
	Allow,
	/**
	 * Prevent any new property binding, will still allow you to edit widgets with property binding, but
	 * the buttons will be missing on all existing widgets that don't have bindings.
	 */
	Prevent,
	/**
	 * Prevent any new property binding, and warn when compiling any existing bindings.
	 */
	PreventAndWarn,
	/**
	* Prevent any new property binding, and error when compiling any existing bindings.
	*/
	PreventAndError
};

USTRUCT()
struct FWidgetCompilerOptions
{
	GENERATED_BODY()

public:

	/**
	 * If you disable this, these widgets these compiler options apply to will not be allowed to implement Tick.
	 */
	UPROPERTY(EditAnywhere, Category = Compiler)
	bool bAllowBlueprintTick = true;

	/**
	 * If you disable this, these widgets these compiler options apply to will not be allowed to implement Paint.
	 */
	UPROPERTY(EditAnywhere, Category = Compiler)
	bool bAllowBlueprintPaint = true;

	/**
	 * Controls if you allow property bindings in widgets.  They can have a large performance impact if used.
	 */
	UPROPERTY(EditAnywhere, Category = Compiler)
	EPropertyBindingPermissionLevel PropertyBindingRule = EPropertyBindingPermissionLevel::Allow;

	/**
	 * Custom rules.
	 */
	UPROPERTY(EditAnywhere, Category = Compiler)
	TArray<TSoftClassPtr<UWidgetCompilerRule>> Rules;

	//TODO Allow limiting delay nodes?
	//TODO Allow limiting 'size of blueprint'
	//TODO Allow preventing blueprint inheritance
};


/**  */
USTRUCT()
struct FDirectoryWidgetCompilerOptions
{
	GENERATED_BODY()

public:

	/** The directory to limit the rules effects to. */
	UPROPERTY(EditAnywhere, Category = Compiler, meta = (ContentDir))
	FDirectoryPath Directory;

#if WITH_EDITORONLY_DATA
	/** These widgets are ignored, and they will use the next most applicable directory to determine their rules. */
	UPROPERTY(EditAnywhere, Category = Compiler)
	TArray<TSoftObjectPtr<UWidgetBlueprint>> IgnoredWidgets;
#endif // WITH_EDITORONLY_DATA

	/** The directory specific compiler options for these widgets. */
	UPROPERTY(EditAnywhere, Category = Compiler)
	FWidgetCompilerOptions Options;
};

/**
 * Implements the settings for the UMG Editor Project Settings
 */
UCLASS(MinimalAPI, Abstract, config=Editor, defaultconfig)
class UWidgetEditingProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UWidgetEditingProjectSettings();

	UE_API virtual void PostInitProperties() override;

public:
	UPROPERTY(EditAnywhere, config, Category = "Compiler")
	FWidgetCompilerOptions DefaultCompilerOptions;

protected:
	UPROPERTY(EditAnywhere, config, Category = "Compiler")
	TArray<FDirectoryWidgetCompilerOptions> DirectoryCompilerOptions;

public:

	UPROPERTY(EditAnywhere, config, Category="Class Filtering")
	bool bShowWidgetsFromEngineContent;

	UPROPERTY(EditAnywhere, config, Category="Class Filtering")
	bool bShowWidgetsFromDeveloperContent;

	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category="Class Filtering")
	TArray<FString> CategoriesToHide;

	UPROPERTY(EditAnywhere, config, Category = "Class Filtering", meta = (MetaClass = "/Script/UMG.Widget"))
	TArray<FSoftClassPath> WidgetClassesToHide;

public:

	/** Enables a dialog that lets you select a root widget before creating a widget blueprint */
	UPROPERTY(EditAnywhere, config, Category = "Designer")
	bool bUseWidgetTemplateSelector;

	/** This list populates the common class section of the root widget selection dialog */
	UPROPERTY(EditAnywhere, config, Category = "Designer", meta = (EditCondition = "bUseWidgetTemplateSelector"))
	TArray<TSoftClassPtr<UPanelWidget>> CommonRootWidgetClasses;

	/** The panel widget to place at the root of all newly constructed widget blueprints. Can be empty. */
	UPROPERTY(EditAnywhere, config, Category = "Designer")
	TSubclassOf<UPanelWidget> DefaultRootWidget;

	/** Set true to filter all categories and widgets out in the palette, selectively enabling them later via permission lists. */
	UPROPERTY(EditAnywhere, config, Category = "Designer")
	bool bUseEditorConfigPaletteFiltering;

	/** Enables a dialog that lets you select the parent class in a tree view. */
	UPROPERTY(EditAnywhere, config, Category = "Designer")
	bool bUseUserWidgetParentClassViewerSelector;

	/** Enables a dialog that lets you select the parent class in a default view. */
	UPROPERTY(EditAnywhere, config, Category = "Designer")
	bool bUseUserWidgetParentDefaultClassViewerSelector;

	/** Set true to enable the Is Variable checkbox in the UMG editor DetailView. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableMakeVariable;

	/** Set true to hide widget animation related elements in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableWidgetAnimationEditor;

	/** Set true to enabled the Palette window in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnablePaletteWindow;

	/** Set true to enabled the Library window in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableLibraryWindow;

	/** Set true to enabled the Widget Hierarchy window in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableHierarchyWindow;
	
	/** Set true to enabled the Bind Widget window in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableBindWidgetWindow;

	/** Set true to enabled the Navigation Simulation window in the UMG editor. */
	UPROPERTY(EditAnywhere, Category = "Designer|Window")
	bool bEnableNavigationSimulationWindow;
	
	/** The default value of bCanCallInitializedWithoutPlayerContext. */
	UPROPERTY(EditAnywhere, Category = "Class Settings")
	bool bCanCallInitializedWithoutPlayerContext;

	UPROPERTY(EditAnywhere, Category = "Class Settings")
	bool bEnableUIComponentsProperty;

	/** Use the new version of the binding in the details view. */
	bool bUseNewBindingFromDetailsView = false;

	/** Use compact link button (just display the link) or complete one (with info about the current binding in the extension part of the detail view. */
	bool bUseCompactLinkButtonInExtension = false;

	/** Set the default widget animation display frame rate's numerator. */
	UPROPERTY(EditAnywhere, config, Category = "Designer|Animation", meta = (ClampMin = "1", UIMin = "1"))
	int32 DefaultWidgetAnimationFrameRate;

	/**
	 * The list of parent classes to choose from for newly constructed widget blueprints.
	 * The classes must have empty widget hierarchies.
	 */
	UPROPERTY(EditAnywhere, config, Category = Designer, meta = (AllowAbstract = "", EditCondition="bUseUserWidgetParentDefaultClassViewerSelector"))
	TArray<TSoftClassPtr<UUserWidget>> FavoriteWidgetParentClasses;

	UPROPERTY(EditAnywhere, config, Category=Designer)
	TArray<FDebugResolution> DebugResolutions;

	UE_API bool CompilerOption_AllowBlueprintTick(const class UWidgetBlueprint* WidgetBlueprint) const;
	UE_API bool CompilerOption_AllowBlueprintPaint(const class UWidgetBlueprint* WidgetBlueprint) const;
	UE_API EPropertyBindingPermissionLevel CompilerOption_PropertyBindingRule(const class UWidgetBlueprint* WidgetBlueprint) const;
	UE_API TArray<UWidgetCompilerRule*> CompilerOption_Rules(const class UWidgetBlueprint* WidgetBlueprint) const;

	/** Get the permission list that controls which categories are exposed in config palette filtering */
	UE_API FNamePermissionList& GetAllowedPaletteCategories();
	UE_API const FNamePermissionList& GetAllowedPaletteCategories() const;

	/** Get the permission list that controls which widgets are exposed in config palette filtering */
	UE_API FPathPermissionList& GetAllowedPaletteWidgets();
	UE_API const FPathPermissionList& GetAllowedPaletteWidgets() const;

private:
	template<typename ReturnType, typename T>
	ReturnType GetFirstCompilerOption(const class UWidgetBlueprint* WidgetBlueprint, T FWidgetCompilerOptions::* OptionMember, ReturnType bDefaultValue) const
	{
		ReturnType bValue = bDefaultValue;
		GetCompilerOptionsForWidget(WidgetBlueprint, [&bValue, OptionMember](const FWidgetCompilerOptions& Options) {
			bValue = Options.*OptionMember;
			return true;
		});
		return bValue;
	}

	UE_API void GetCompilerOptionsForWidget(const class UWidgetBlueprint* WidgetBlueprint, TFunctionRef<bool(const FWidgetCompilerOptions&)> Operator) const;

protected:
	UE_API virtual void PerformUpgradeStepForVersion(int32 ForVersion);

	UPROPERTY(config)
	int32 Version;

	/** This one is unsaved, we compare it on post init to see if the save matches real */
	int32 CurrentVersion;

	/** Palette categories to allow all widgets within when using permission list palette filtering */
	FNamePermissionList AllowedPaletteCategories;

	/** Individual widgets to always allow when using permission list palette filtering, regardless of category */
	FPathPermissionList AllowedPaletteWidgets;
};

#undef UE_API
