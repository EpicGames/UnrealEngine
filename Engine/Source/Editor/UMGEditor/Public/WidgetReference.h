// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"

#define UE_API UMGEDITOR_API

class FWidgetBlueprintEditor;
struct FWidgetReference;

struct FWidgetHandle
{
	friend FWidgetReference;
	friend FWidgetBlueprintEditor;

private:
	UE_API FWidgetHandle(UWidget* Widget);

	TWeakObjectPtr<UWidget> Widget;
};

/**
 * The Widget reference is a useful way to hold onto the selection in a way that allows for up to date access to the current preview object.
 * Because the designer could end up rebuilding the preview, it's best to hold onto an FWidgetReference.
 */
struct FWidgetReference
{
	friend FWidgetBlueprintEditor;

public:
	UE_API FWidgetReference();

	/** @returns true if both the template and the preview pointers are valid. */
	UE_API bool IsValid() const;

	/** @returns The template widget.  This is the widget that is serialized to disk */
	UE_API UWidget* GetTemplate() const;

	/** @returns the preview widget.  This is the transient representation of the template.  Constantly being destroyed and recreated.  Do not cache this pointer. */
	UE_API UWidget* GetPreview() const;

	/** @returns the preview slate widget.  This is the transient representation of the template.  Constantly being destroyed and recreated.  Do not cache this pointer. */
	TSharedPtr<SWidget> GetPreviewSlate() const
	{
		if ( IsValid() )
		{
			return GetPreview()->GetCachedWidget();
		}

		return TSharedPtr<SWidget>();
	}

	TSharedPtr<FWidgetBlueprintEditor> GetWidgetEditor() const { return WidgetEditor.Pin(); };

	/** Checks if widget reference is the same as another widget reference, based on the template pointers. */
	bool operator==( const FWidgetReference& Other ) const
	{
		if ( TemplateHandle.IsValid() && Other.TemplateHandle.IsValid() )
		{
			return TemplateHandle->Widget.Get() == Other.TemplateHandle->Widget.Get();
		}

		return false;
	}

	/** Checks if widget reference is the different from another widget reference, based on the template pointers. */
	bool operator!=( const FWidgetReference& Other ) const
	{
		return !operator==( Other );
	}

private:
	UE_API FWidgetReference(TSharedPtr<FWidgetBlueprintEditor> WidgetEditor, TSharedPtr< FWidgetHandle > TemplateHandle);

	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

private:
	TWeakPtr<FWidgetBlueprintEditor> WidgetEditor;

	TSharedPtr< FWidgetHandle > TemplateHandle;
};

inline uint32 GetTypeHash(const struct FWidgetReference& WidgetRef)
{
	return ::GetTypeHash(reinterpret_cast<void*>( WidgetRef.GetTemplate() ));
}

#undef UE_API
