// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateNoResource.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/IToolTip.h"

#define UE_API UMGEDITOR_API

class IToolTip;
class UWidget;
struct FSlateBrush;

/**
 * The widget template represents a widget or a set of widgets to create and spawn into the widget tree.
 * More complicated defaults could be created by deriving from this class and registering new templates with the module.
 */
class FWidgetTemplate : public TSharedFromThis<FWidgetTemplate>
{
public:
	/** Constructor */
	UE_API FWidgetTemplate();
	virtual ~FWidgetTemplate() = default;

	/** The category this template fits into. */
	virtual FText GetCategory() const = 0;

	/** Constructs the widget template. */
	virtual UWidget* Create(class UWidgetTree* Tree) = 0;

	/** Gets the icon to display for this widget template. */
	virtual const FSlateBrush* GetIcon() const
	{
		static FSlateNoResource NullBrush;
		return &NullBrush;
	}

	/** Gets tooltip widget for this widget template. */
	virtual TSharedRef<IToolTip> GetToolTip() const = 0;

	/** @param OutStrings - Returns an array of strings used for filtering/searching this widget template. */
	virtual void GetFilterStrings(TArray<FString>& OutStrings) const { OutStrings.Add(Name.ToString()); }

	/** The the action to perform when the template item is double clicked */
	virtual FReply OnDoubleClicked() { return FReply::Unhandled(); }

public:
	/** The name of the widget template. */
	FText Name;
};

#undef UE_API
