// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class UMGEditor : ModuleRules
{
	public UMGEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		OverridePackageType = PackageOverrideType.EngineDeveloper;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"FieldNotification",
				"SequencerCore",
				"Sequencer",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"AssetRegistry",
				"ClassViewer",
				"Core",
				"CoreUObject",
				"ContentBrowser",
				"ContentBrowserData",
				"ApplicationCore",
				"InputCore",
				"Engine",
				"AssetTools",
				"EditorConfig",
				"EditorSubsystem",
				"EditorFramework",
				"InteractiveToolsFramework",
				"UnrealEd", // for Asset Editor Subsystem
				"KismetWidgets",
				"EditorWidgets",
				"KismetCompiler",
				"BlueprintGraph",
				"Blutility",
				"GraphEditor",
				"Kismet",  // for FWorkflowCentricApplication
				"Projects",
				"PropertyPath",
				"PropertyEditor",
				"UMG",
				"EditorStyle",
				"Slate",
				"SlateCore",
				"SlateRHIRenderer",
				"StructViewer",
				"StructUtilsEditor",
				"StatusBar",
				"MessageLog",
				"MovieScene",
				"MovieSceneTools",
                "MovieSceneTracks",
				"DetailCustomizations",
                "Settings",
				"RenderCore",
                "TargetPlatform",
				"TimeManagement",
				"GameProjectGeneration",
				"PropertyPath",
				"ToolMenus",
				"SlateReflector",
				"DeveloperSettings",
				"ImageWrapper",
				"ToolWidgets",
				"WorkspaceMenuStructure"
			}
			);

		// Circular references that need to be cleaned up
		CircularlyReferencedDependentModules.AddRange(
		   [
				"Blutility",
		   ]);
	}
}
